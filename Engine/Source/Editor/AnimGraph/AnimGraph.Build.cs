// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AnimGraph : ModuleRules
{
	public AnimGraph(ReadOnlyTargetRules Target) : base(Target)
	{		
		OverridePackageType = PackageOverrideType.EngineDeveloper;

        PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core", 
				"CoreUObject", 
				"Engine", 
				"Slate",
				"AnimGraphRuntime",
				"BlueprintGraph",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"InputCore",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
                "GraphEditor",
				"PropertyEditor",
				
                "ContentBrowser",
				"KismetWidgets",
				"ToolMenus",
				"KismetCompiler",
				"Kismet",
				"EditorWidgets",
				"ToolWidgets",
				"AnimationEditMode",
				"DeveloperSettings"
			}
		);

        CircularlyReferencedDependentModules.AddRange(
			[
                "UnrealEd",
                "GraphEditor",
				"KismetCompiler",
				"Kismet",
				"BlueprintGraph",
				"ContentBrowser",
				"PropertyEditor",
			]
		);

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "AnimationBlueprintEditor",
            }
        );
	}
}
