// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_PropertyAccess.h"
#include "Animation/AnimNodeBase.h"
#include "IPropertyAccessCompiler.h"
#include "Animation/AnimSubsystem_Base.h"
#include "AnimBlueprintExtension_Base.generated.h"

class FCompilerResultsLog;
class UAnimGraphNode_Base;
class UEdGraphNode;
class UK2Node_CustomEvent;
class IAnimBlueprintCompilationContext;
class IAnimBlueprintGeneratedClassCompiledData;
class IAnimBlueprintCompilerCreationContext;
class IAnimBlueprintCompilationBracketContext;
class IAnimBlueprintPostExpansionStepContext;
class IAnimBlueprintCopyTermDefaultsContext;
struct FAnimGraphNodePropertyBinding;

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_Base : public UAnimBlueprintExtension
{
	GENERATED_BODY()

public:
	// Processes pose pins, building a map of pose links for later processing
	void ProcessPosePins(UAnimGraphNode_Base* InNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	enum class EPinProcessingFlags : int32
	{
		Constants				= (1 << 0),
		BlueprintHandlers		= (1 << 1),
		PropertyAccessBindings	= (1 << 3),
		PropertyAccessFastPath	= (1 << 4),

		All						= Constants | BlueprintHandlers | PropertyAccessBindings | PropertyAccessFastPath,
	};

	FRIEND_ENUM_CLASS_FLAGS(EPinProcessingFlags);

	// Processes a node's non-pose pins:
	// - Adds a map of struct eval handlers for the specified node
	// - Builds property binding data
	ANIMGRAPH_API void ProcessNonPosePins(UAnimGraphNode_Base* InNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData, EPinProcessingFlags InFlags);

	// Create an 'expanded' evaluation handler for the specified node, called in the compiler's node expansion step
	void CreateEvaluationHandlerForNode(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InNode);

private:
	// UAnimBlueprintExtension interface
	virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandleFinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandlePostExpansionStep(const UEdGraph* InGraph, IAnimBlueprintPostExpansionStepContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandleCopyTermDefaultsToDefaultObject(UObject* InDefaultObject, IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext) override;

	// Patch all node's evaluation handlers 
	void PatchEvaluationHandlers(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

private:
	/** Record of a single copy operation */
	struct FPropertyCopyRecord
	{
		FPropertyCopyRecord(UEdGraphPin* InDestPin, FProperty* InDestProperty, int32 InDestArrayIndex, TArray<FString>&& InDestPropertyPath)
			: DestPin(InDestPin)
			, DestProperty(InDestProperty)
			, DestArrayIndex(InDestArrayIndex)
			, DestPropertyPath(MoveTemp(InDestPropertyPath))
			, BindingContextId(NAME_None)
			, Operation(EPostCopyOperation::None)
			, bIsFastPath(true)
			, bOnlyUpdateWhenActive(false)
		{}

		FPropertyCopyRecord(const TArray<FString>& InSourcePropertyPath, const TArray<FString>& InDestPropertyPath)
			: DestPin(nullptr)
			, DestProperty(nullptr)
			, DestArrayIndex(INDEX_NONE)
			, SourcePropertyPath(InSourcePropertyPath)
			, DestPropertyPath(InDestPropertyPath)
			, BindingContextId(NAME_None)
			, Operation(EPostCopyOperation::None)
			, bIsFastPath(true)
			, bOnlyUpdateWhenActive(false)
		{}

		bool IsFastPath() const
		{
			return SourcePropertyPath.Num() > 0 && bIsFastPath;
		}

		void InvalidateFastPath()
		{
			bIsFastPath = false;
		}

		/** The destination pin we are copying to */
		UEdGraphPin* DestPin;

		/** The destination property we are copying to (on an animation node) */
		FProperty* DestProperty;

		/** The array index we use if the destination property is an array */
		int32 DestArrayIndex;

		/** The property path relative to the class */
		TArray<FString> SourcePropertyPath;

		/** The property path relative to the class */
		TArray<FString> DestPropertyPath;

		/** The handle of the copy in the property access compiler */
		FPropertyAccessHandle LibraryHandle;

		/** The handle of the copy in the property access library */
		FCompiledPropertyAccessHandle LibraryCompiledHandle;

		// Context in which a property binding occurs
		FName BindingContextId;
		
		/** Any operation we want to perform post-copy on the destination data */
		EPostCopyOperation Operation;

		/** Fast-path flag */
		bool bIsFastPath;

		bool bOnlyUpdateWhenActive;
	};

	// Context used to build fast-path copy records
	struct FCopyRecordGraphCheckContext
	{
		FCopyRecordGraphCheckContext(FPropertyCopyRecord& InCopyRecord, TArray<FPropertyCopyRecord>& InAdditionalCopyRecords, FCompilerResultsLog& InMessageLog)
			: CopyRecord(&InCopyRecord)
			, AdditionalCopyRecords(InAdditionalCopyRecords)
			, MessageLog(InMessageLog)
		{}

		// Copy record we are operating on
		FPropertyCopyRecord* CopyRecord;

		// Things like split input pins can add additional copy records
		TArray<FPropertyCopyRecord>& AdditionalCopyRecords;

		// Message log used to recover original nodes
		FCompilerResultsLog& MessageLog;
	};

	// Wireup record for a single anim node property (which might be an array)
	struct FAnimNodeSinglePropertyHandler
	{
		/** Copy records */
		TArray<FPropertyCopyRecord> CopyRecords;

		// If the anim instance is the container target instead of the node.
		bool bInstanceIsTarget;

		FAnimNodeSinglePropertyHandler()
			: bInstanceIsTarget(false)
		{
		}
	};

	/** BP execution handler for Anim node */
	struct FEvaluationHandlerRecord
	{
	public:
		// The Node this record came from
		UAnimGraphNode_Base* AnimGraphNode;

		// The node variable that the handler is in
		FStructProperty* NodeVariableProperty;

		// The specific evaluation handler inside the specified node
		int32 EvaluationHandlerIdx;

		// Whether or not our serviced properties are actually on the anim node 
		bool bServicesNodeProperties;

		// Whether or not our serviced properties are actually on the instance instead of the node
		bool bServicesInstanceProperties;

		// This eval handler has properties
		bool bHasProperties;

		// Set of properties serviced by this handler (Map from property name to the record for that property)
		TMap<FName, FAnimNodeSinglePropertyHandler> ServicedProperties;

		// The generated custom event node
		TArray<UEdGraphNode*> CustomEventNodes;

		// The resulting function name
		FName HandlerFunctionName;

	public:

		FEvaluationHandlerRecord()
			: AnimGraphNode(nullptr)
			, NodeVariableProperty(nullptr)
			, EvaluationHandlerIdx(INDEX_NONE)
			, bServicesNodeProperties(false)
			, bServicesInstanceProperties(false)
			, bHasProperties(false)
			, HandlerFunctionName(NAME_None)
		{}

		bool IsFastPath() const
		{
			for(TMap<FName, FAnimNodeSinglePropertyHandler>::TConstIterator It(ServicedProperties); It; ++It)
			{
				const FAnimNodeSinglePropertyHandler& AnimNodeSinglePropertyHandler = It.Value();
				for (const FPropertyCopyRecord& CopyRecord : AnimNodeSinglePropertyHandler.CopyRecords)
				{
					if (!CopyRecord.IsFastPath())
					{
						return false;
					}
				}
			}

			return true;
		}

		bool IsValid() const
		{
			return NodeVariableProperty != nullptr;
		}


		void PatchAnimNodeExposedValueHandler(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext) const;

		void RegisterPin(UEdGraphPin* DestPin, FProperty* AssociatedProperty, int32 AssociatedPropertyArrayIndex, bool bAllowFastPath);

		void RegisterPropertyBinding(FProperty* InProperty, const FAnimGraphNodePropertyBinding& InBinding);

		FStructProperty* GetHandlerNodeProperty() const { return NodeVariableProperty; }

		void BuildFastPathCopyRecords(IAnimBlueprintPostExpansionStepContext& InCompilationContext);

	private:

		bool CheckForVariableGet(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin);

		bool CheckForLogicalNot(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin);

		bool CheckForStructMemberAccess(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin);

		bool CheckForMemberOnlyAccess(FPropertyCopyRecord& Context, UEdGraphPin* DestPin);

		bool CheckForSplitPinAccess(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin);

		bool CheckForArrayAccess(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin);
	};
	
	// Create an evaluation handler for the specified node/record
	void CreateEvaluationHandler(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InNode, FEvaluationHandlerRecord& Record);

	// Redirect any property accesses that are affected by constant folding
	void RedirectPropertyAccesses(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InNode, FEvaluationHandlerRecord& InRecord);

private:
	// Records of pose pins for later patchup with an associated evaluation handler
	TMap<UAnimGraphNode_Base*, FEvaluationHandlerRecord> PerNodeStructEvalHandlers;

	// List of successfully created evaluation handlers
	TArray<FEvaluationHandlerRecord> ValidEvaluationHandlerList;
	TMap<UAnimGraphNode_Base*, int32> ValidEvaluationHandlerMap;

	// Set of used handler function names
	TSet<FName> HandlerFunctionNames;

	// Delegate handle for registering against library pre/post-compilation
	FDelegateHandle PreLibraryCompiledDelegateHandle;
	FDelegateHandle PostLibraryCompiledDelegateHandle;

	// Base subsystem data containing eval handlers
	UPROPERTY()
	FAnimSubsystem_Base Subsystem;
};

ENUM_CLASS_FLAGS(UAnimBlueprintExtension_Base::EPinProcessingFlags);