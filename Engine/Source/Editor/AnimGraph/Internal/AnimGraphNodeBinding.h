// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "Widgets/SNullWidget.h"
#include "AnimGraphNodeBinding.generated.h"

class UEdGraphPin;
struct FSearchTagDataPair;
class UClass;
class UBlueprint;
struct FMemberReference;

UCLASS(MinimalAPI, Abstract)
class UAnimGraphNodeBinding : public UObject
{
	GENERATED_BODY()

	friend class UAnimGraphNode_Base;
	friend class UAnimGraphNode_CustomProperty;
	friend class FAnimBlueprintCompilerContext;
	friend class UAnimBlueprintExtension;

	// Get the struct type for the runtime handling of this binding
	virtual UScriptStruct* GetAnimNodeHandlerStruct() const PURE_VIRTUAL(UAnimGraphNodeBinding::GetAnimNodeHandlerStruct, return nullptr;)

	// Override point for when pins are re-created on the hosting node
	virtual void OnInternalPinCreation(UAnimGraphNode_Base* InNode) {}

	// Override point for when the hosting node is reconstructed
	virtual void OnReconstructNode(UAnimGraphNode_Base* InNode) {}

	// Override point for when the hosting node is expanded during compilation
	virtual void OnExpandNode(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InNode, UEdGraph* InSourceGraph) {}

	// Check whether this binding binds to the specified name
	virtual bool HasBinding(FName InBindingName, bool bCheckArrayIndexName) const PURE_VIRTUAL(UAnimGraphNodeBinding::HasBinding, return false;)

	// Remove all bindings to the specified name
	virtual void RemoveBindings(FName InBindingName) PURE_VIRTUAL(UAnimGraphNodeBinding::RemoveBindings, )

	// Add any search metadata for pin bindings
	virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* InPin, FName InBindingName, TArray<FSearchTagDataPair>& OutTaggedMetaData) const {}

	virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) {}

	virtual void HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName) {}

	virtual void ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement) {}

	virtual bool ReferencesVariable(const FName& InVarName, const UStruct* InScope) const { return false; }

	virtual bool ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const { return false; }

	// Update function for binding names - if a binding string tghat differs from InOldName is 
	// returned from InModifierFunction then the binding will be replaced
	virtual void UpdateBindingNames(TFunctionRef<FString(const FString& InOldName)> InModifierFunction) PURE_VIRTUAL(UAnimGraphNodeBinding::UpdateBindingNames, )

	// Get any extensions that are needed to process this binding
	virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const {}

	// Process binding for this node during compilation
	virtual void ProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}

#if WITH_EDITOR
	// Create the binding widget for a pin/property. This is only called on the CDO. Use InArgs to determine the nodes to create the widget for.
	virtual TSharedRef<SWidget> MakePropertyBindingWidget(const UAnimGraphNode_Base::FAnimPropertyBindingWidgetArgs& InArgs) { return SNullWidget::NullWidget; }
#endif
};