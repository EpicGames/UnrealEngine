// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintCompilationContext.h"
#include "AnimBlueprintCompiler.h"
#include "AnimBlueprintExtension.h"

TUniquePtr<IAnimBlueprintCompilationContext> IAnimBlueprintCompilationContext::Get(FKismetCompilerContext& InKismetCompiler)
{
	return TUniquePtr<IAnimBlueprintCompilationContext>(new FAnimBlueprintCompilationContext(static_cast<FAnimBlueprintCompilerContext*>(&InKismetCompiler)));
}

void FAnimBlueprintCompilationContext::AddPoseLinkMappingRecordImpl(const FPoseLinkMappingRecord& InRecord)
{
	CompilerContext->ValidPoseLinkList.Add(InRecord);
}

void FAnimBlueprintCompilationContext::ProcessAnimationNodesImpl(TArray<UAnimGraphNode_Base*>& AnimNodeList)
{
	CompilerContext->ProcessAnimationNodes(AnimNodeList);
}

void FAnimBlueprintCompilationContext::PruneIsolatedAnimationNodesImpl(const TArray<UAnimGraphNode_Base*>& RootSet, TArray<UAnimGraphNode_Base*>& GraphNodes)
{
	CompilerContext->PruneIsolatedAnimationNodes(RootSet, GraphNodes);
}

void FAnimBlueprintCompilationContext::ExpansionStepImpl(UEdGraph* Graph, bool bAllowUbergraphExpansions)
{
	CompilerContext->ExpansionStep(Graph, bAllowUbergraphExpansions);
}

FCompilerResultsLog& FAnimBlueprintCompilationContext::GetMessageLogImpl() const
{
	return CompilerContext->MessageLog;
}

bool FAnimBlueprintCompilationContext::ValidateGraphIsWellFormedImpl(UEdGraph* Graph) const
{
	return CompilerContext->ValidateGraphIsWellFormed(Graph);
}

int32 FAnimBlueprintCompilationContext::GetAllocationIndexOfNodeImpl(UAnimGraphNode_Base* VisualAnimNode) const
{
	return CompilerContext->GetAllocationIndexOfNode(VisualAnimNode);
}

const UBlueprint* FAnimBlueprintCompilationContext::GetBlueprintImpl() const
{
	return CompilerContext->Blueprint;
}

const UAnimBlueprint* FAnimBlueprintCompilationContext::GetAnimBlueprintImpl() const
{
	return CompilerContext->AnimBlueprint;
}

UEdGraph* FAnimBlueprintCompilationContext::GetConsolidatedEventGraphImpl() const
{
	return CompilerContext->ConsolidatedEventGraph;
}

void FAnimBlueprintCompilationContext::GetLinkedAnimNodesImpl(UAnimGraphNode_Base* InGraphNode, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const
{
	CompilerContext->GetLinkedAnimNodes(InGraphNode, LinkedAnimNodes);
}

const TMap<UAnimGraphNode_Base*, int32>& FAnimBlueprintCompilationContext::GetAllocatedAnimNodeIndicesImpl() const
{
	return CompilerContext->AllocatedAnimNodeIndices;
}

const TMap<UAnimGraphNode_Base*, UAnimGraphNode_Base*>& FAnimBlueprintCompilationContext::GetSourceNodeToProcessedNodeMapImpl() const
{
	return CompilerContext->SourceNodeToProcessedNodeMap;
}

const TMap<int32, FProperty*>& FAnimBlueprintCompilationContext::GetAllocatedPropertiesByIndexImpl() const
{
	return CompilerContext->AllocatedPropertiesByIndex;
}

const TMap<UAnimGraphNode_Base*, FProperty*>& FAnimBlueprintCompilationContext::GetAllocatedPropertiesByNodeImpl() const
{
	return CompilerContext->AllocatedAnimNodes;
}

const TMap<UAnimGraphNode_Base*, FProperty*>& FAnimBlueprintCompilationContext::GetAllocatedHandlerPropertiesByNodeImpl() const
{
	return CompilerContext->AllocatedAnimNodeHandlers;
}

void FAnimBlueprintCompilationContext::AddAttributesToNodeImpl(UAnimGraphNode_Base* InNode, TArrayView<const FName> InAttributes) const
{
	CompilerContext->AddAttributesToNode(InNode, InAttributes);
}

TArrayView<const FName> FAnimBlueprintCompilationContext::GetAttributesFromNodeImpl(UAnimGraphNode_Base* InNode) const
{
	return CompilerContext->GetAttributesFromNode(InNode);
}

bool FAnimBlueprintCompilationContext::IsAnimGraphNodeFoldedImpl(UAnimGraphNode_Base* InNode) const
{
	return CompilerContext->IsAnimGraphNodeFolded(InNode);
}

const IAnimBlueprintCompilationContext::FFoldedPropertyRecord* FAnimBlueprintCompilationContext::GetFoldedPropertyRecordImpl(UAnimGraphNode_Base* InNode, FName InPropertyName) const
{
	return CompilerContext->GetFoldedPropertyRecord(InNode, InPropertyName);
}

const FStructProperty* FAnimBlueprintCompilationContext::GetMutableDataPropertyImpl() const
{
	return CompilerContext->NewMutablesProperty;
}

FKismetCompilerContext* FAnimBlueprintCompilationContext::GetKismetCompiler() const
{
	return static_cast<FKismetCompilerContext*>(CompilerContext);
}

FCompilerResultsLog& FAnimBlueprintCompilationBracketContext::GetMessageLogImpl() const
{
	return CompilerContext->MessageLog;
}

const TMap<UAnimGraphNode_Base*, int32>& FAnimBlueprintCompilationBracketContext::GetAllocatedAnimNodeIndicesImpl() const
{
	return CompilerContext->AllocatedAnimNodeIndices;
}

const TMap<UAnimGraphNode_Base*, FProperty*>& FAnimBlueprintCompilationBracketContext::GetAllocatedHandlerPropertiesByNodeImpl() const
{
	return CompilerContext->AllocatedAnimNodeHandlers;
}

FCompilerResultsLog& FAnimBlueprintCopyTermDefaultsContext::GetMessageLogImpl() const
{
	return CompilerContext->MessageLog;
}

const UAnimBlueprint* FAnimBlueprintCopyTermDefaultsContext::GetAnimBlueprintImpl() const
{
	return CompilerContext->AnimBlueprint;
}

FCompilerResultsLog& FAnimBlueprintPostExpansionStepContext::GetMessageLogImpl() const
{
	return CompilerContext->MessageLog;
}

UEdGraph* FAnimBlueprintPostExpansionStepContext::GetConsolidatedEventGraphImpl() const
{
	return CompilerContext->ConsolidatedEventGraph;
}

const FKismetCompilerOptions& FAnimBlueprintPostExpansionStepContext::GetCompileOptionsImpl() const
{
	return CompilerContext->CompileOptions;
}

const TMap<UAnimGraphNode_Base*, int32>& FAnimBlueprintPostExpansionStepContext::GetAllocatedAnimNodeIndicesImpl() const
{
	return CompilerContext->AllocatedAnimNodeIndices;
}