// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimBlueprintCompilationContext.h"
#include "IAnimBlueprintCompilationBracketContext.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "IAnimBlueprintPostExpansionStepContext.h"
#include "IAnimBlueprintNodeOverrideAssetsContext.h"
#include "Animation/AnimationAsset.h"

class FAnimBlueprintCompilerContext;
class UAnimBlueprintExtension;
struct FAnimNode_Base;

class FAnimBlueprintCompilationContext : public IAnimBlueprintCompilationContext
{
private:
	friend class FAnimBlueprintCompilerContext;
	friend class IAnimBlueprintCompilationContext;

	FAnimBlueprintCompilationContext(FAnimBlueprintCompilerContext* InCompilerContext)
		: CompilerContext(InCompilerContext)
	{}

	virtual void AddPoseLinkMappingRecordImpl(const FPoseLinkMappingRecord& InRecord) override;
	virtual void ProcessAnimationNodesImpl(TArray<UAnimGraphNode_Base*>& AnimNodeList) override;
	virtual void PruneIsolatedAnimationNodesImpl(const TArray<UAnimGraphNode_Base*>& RootSet, TArray<UAnimGraphNode_Base*>& GraphNodes) override;
	virtual void ExpansionStepImpl(UEdGraph* Graph, bool bAllowUbergraphExpansions) override;
	virtual FCompilerResultsLog& GetMessageLogImpl() const override;
	virtual bool ValidateGraphIsWellFormedImpl(UEdGraph* Graph) const override;
	virtual int32 GetAllocationIndexOfNodeImpl(UAnimGraphNode_Base* VisualAnimNode) const override;
	virtual const UBlueprint* GetBlueprintImpl() const override;
	virtual const UAnimBlueprint* GetAnimBlueprintImpl() const override;
	virtual UEdGraph* GetConsolidatedEventGraphImpl() const override;
	virtual void GetLinkedAnimNodesImpl(UAnimGraphNode_Base* InGraphNode, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const override;
	virtual const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndicesImpl() const override;
	virtual const TMap<UAnimGraphNode_Base*, UAnimGraphNode_Base*>& GetSourceNodeToProcessedNodeMapImpl() const override;
	virtual const TMap<int32, FProperty*>& GetAllocatedPropertiesByIndexImpl() const override;
	virtual const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedPropertiesByNodeImpl() const override;
	virtual const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedHandlerPropertiesByNodeImpl() const override;
	virtual void AddAttributesToNodeImpl(UAnimGraphNode_Base* InNode, TArrayView<const FName> InAttributes) const override;
	virtual TArrayView<const FName> GetAttributesFromNodeImpl(UAnimGraphNode_Base* InNode) const override;
	virtual bool IsAnimGraphNodeFoldedImpl(UAnimGraphNode_Base* InNode) const override;
	virtual const IAnimBlueprintCompilationContext::FFoldedPropertyRecord* GetFoldedPropertyRecordImpl(UAnimGraphNode_Base* InNode, FName InPropertyName) const override;
	virtual const FStructProperty* GetMutableDataPropertyImpl() const override;
	virtual FKismetCompilerContext* GetKismetCompiler() const override;

	FAnimBlueprintCompilerContext* CompilerContext;
};

class FAnimBlueprintCompilationBracketContext : public IAnimBlueprintCompilationBracketContext
{
private:
	friend class FAnimBlueprintCompilerContext;

	FAnimBlueprintCompilationBracketContext(FAnimBlueprintCompilerContext* InCompilerContext)
		: CompilerContext(InCompilerContext)
	{}

	virtual FCompilerResultsLog& GetMessageLogImpl() const override;
	virtual const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndicesImpl() const override;
	virtual const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedHandlerPropertiesByNodeImpl() const override;
	
	FAnimBlueprintCompilerContext* CompilerContext;
};

class FAnimBlueprintCopyTermDefaultsContext : public IAnimBlueprintCopyTermDefaultsContext
{
private:
	friend class FAnimBlueprintCompilerContext;

	FAnimBlueprintCopyTermDefaultsContext(FAnimBlueprintCompilerContext* InCompilerContext)
		: CompilerContext(InCompilerContext)
	{}

	virtual FCompilerResultsLog& GetMessageLogImpl() const override;
	virtual const UAnimBlueprint* GetAnimBlueprintImpl() const override;

	FAnimBlueprintCompilerContext* CompilerContext;
};

class FAnimBlueprintNodeCopyTermDefaultsContext : public IAnimBlueprintNodeCopyTermDefaultsContext
{
private:
	friend class FAnimBlueprintCompilerContext;

	FAnimBlueprintNodeCopyTermDefaultsContext(UObject* InCDO, const FProperty* InTargetProperty, uint8* InDestinationPtr, const uint8* InSourcePtr, int32 InNodePropertyIndex)
		: CDO(InCDO)
		, TargetProperty(InTargetProperty)
		, DestinationPtr(InDestinationPtr)
		, SourcePtr(InSourcePtr)
		, NodePropertyIndex(InNodePropertyIndex)
	{}

	virtual UObject* GetClassDefaultObjectImpl() const override { return CDO; }
	virtual const FProperty* GetTargetPropertyImpl() const override { return TargetProperty; }
	virtual uint8* GetDestinationPtrImpl() const override { return DestinationPtr; }
	virtual const uint8* GetSourcePtrImpl() const override { return SourcePtr; }
	virtual int32 GetNodePropertyIndexImpl() const override { return NodePropertyIndex; }

	UObject* CDO;
	const FProperty* TargetProperty;
	uint8* DestinationPtr;
	const uint8* SourcePtr;
	int32 NodePropertyIndex;
};

class FAnimBlueprintExtensionCopyTermDefaultsContext : public IAnimBlueprintExtensionCopyTermDefaultsContext
{
private:
	friend class FAnimBlueprintCompilerContext;

	FAnimBlueprintExtensionCopyTermDefaultsContext(UObject* InCDO, const FProperty* InTargetProperty, uint8* InDestinationPtr, const uint8* InSourcePtr, int32 InNodePropertyIndex)
		: CDO(InCDO)
		, TargetProperty(InTargetProperty)
		, DestinationPtr(InDestinationPtr)
		, SourcePtr(InSourcePtr)
		, NodePropertyIndex(InNodePropertyIndex)
	{}

	virtual UObject* GetClassDefaultObjectImpl() const override { return CDO; }
	virtual const FProperty* GetTargetPropertyImpl() const override { return TargetProperty; }
	virtual uint8* GetDestinationPtrImpl() const override { return DestinationPtr; }
	virtual const uint8* GetSourcePtrImpl() const override { return SourcePtr; }
	virtual int32 GetNodePropertyIndexImpl() const override { return NodePropertyIndex; }

	UObject* CDO;
	const FProperty* TargetProperty;
	uint8* DestinationPtr;
	const uint8* SourcePtr;
	int32 NodePropertyIndex;
};

class FAnimBlueprintPostExpansionStepContext : public IAnimBlueprintPostExpansionStepContext
{
private:
	friend class FAnimBlueprintCompilerContext;

	FAnimBlueprintPostExpansionStepContext(FAnimBlueprintCompilerContext* InCompilerContext)
		: CompilerContext(InCompilerContext)
	{}

	virtual FCompilerResultsLog& GetMessageLogImpl() const override;
	virtual UEdGraph* GetConsolidatedEventGraphImpl() const override;
	virtual const FKismetCompilerOptions& GetCompileOptionsImpl() const override;
	virtual const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndicesImpl() const override;

	FAnimBlueprintCompilerContext* CompilerContext;
};

class FAnimBlueprintNodeOverrideAssetsContext : public IAnimBlueprintNodeOverrideAssetsContext
{
private:
	friend class FAnimBlueprintCompilerContext;
	
	FAnimBlueprintNodeOverrideAssetsContext(FAnimNode_Base* InAnimNode, UScriptStruct* InAnimNodeType)
		: AnimNode(InAnimNode)
		, AnimNodeType(InAnimNodeType)
	{}

	// Add an asset to this context
	void AddAsset(UObject* InAsset) { Assets.Add(InAsset); }

	virtual FAnimNode_Base* GetAnimNodeImpl() const override { return AnimNode; }
	virtual UScriptStruct* GetAnimNodeTypeImpl() const override { return AnimNodeType; }
	virtual TArrayView<UObject* const> GetAssetsImpl() const override { return MakeArrayView(Assets); }

	// The anim node we are targeting
	FAnimNode_Base* AnimNode;

	// The type of the anim node we are targeting
	UScriptStruct* AnimNodeType;
	
	// All the assets to override
	TArray<UObject*> Assets;
};