// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintCompiler.h"
#include "Animation/AnimInstance.h"
#include "EdGraphUtilities.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Knot.h"
#include "K2Node_StructMemberSet.h"
#include "K2Node_VariableGet.h"

#include "AnimationGraphSchema.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet/KismetArrayLibrary.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "AnimGraphNode_Root.h"
#include "Animation/AnimNode_CustomProperty.h"
#include "AnimationEditorUtils.h"
#include "AnimationGraph.h"
#include "AnimBlueprintPostCompileValidation.h" 
#include "AnimGraphNode_LinkedInputPose.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "AnimGraphNode_LinkedAnimLayer.h"
#include "String/ParseTokens.h"
#include "Algo/Sort.h"
#include "IClassVariableCreator.h"
#include "AnimBlueprintGeneratedClassCompiledData.h"
#include "AnimBlueprintCompilationContext.h"
#include "AnimBlueprintCompilerCreationContext.h"
#include "AnimBlueprintVariableCreationContext.h"
#include "Animation/AnimSubsystemInstance.h"
#include "AnimBlueprintExtension.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "AnimGraphNodeBinding.h"

#define LOCTEXT_NAMESPACE "AnimBlueprintCompiler"

DECLARE_CYCLE_STAT(TEXT("Merge Ubergraph Pages In"), EAnimBlueprintCompilerStats_MergeUbergraphPagesIn, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Process All Animation Nodes"), EAnimBlueprintCompilerStats_ProcessAllAnimationNodes, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Process Animation Nodes"), EAnimBlueprintCompilerStats_ProcessAnimationNodes, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Expand Split Pins"), EAnimBlueprintCompilerStats_ExpandSplitPins, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Move Graphs"), EAnimBlueprintCompilerStats_MoveGraphs, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Clone Graph"), EAnimBlueprintCompilerStats_CloneGraph, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Process Animation Node"), EAnimBlueprintCompilerStats_ProcessAnimationNode, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Gather Fold Records"), EAnimBlueprintCompilerStats_GatherFoldRecords, STATGROUP_KismetCompiler);

//////////////////////////////////////////////////////////////////////////
// FAnimBlueprintCompiler

FAnimBlueprintCompilerContext::FAnimBlueprintCompilerContext(UAnimBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
	: FKismetCompilerContext(SourceSketch, InMessageLog, InCompileOptions)
	, NewAnimBlueprintConstants(nullptr)
	, NewAnimBlueprintMutables(nullptr)
	, NewMutablesProperty(nullptr)
	, AnimBlueprint(SourceSketch)
	, OldSparseClassDataStruct(nullptr)
	, bIsDerivedAnimBlueprint(false)
{
	// Add the animation graph schema to skip default function processing on them
	KnownGraphSchemas.AddUnique(UAnimationGraphSchema::StaticClass());

	// Make sure the skeleton has finished preloading
	if (AnimBlueprint->TargetSkeleton != nullptr)
	{
		if (FLinkerLoad* Linker = AnimBlueprint->TargetSkeleton->GetLinker())
		{
			Linker->Preload(AnimBlueprint->TargetSkeleton);
		}
	}

	// If we need to, refresh all extensions here
	if(AnimBlueprint->bRefreshExtensions)
	{
		UAnimBlueprintExtension::RefreshExtensions(AnimBlueprint);
		AnimBlueprint->bRefreshExtensions = false;
	}
	
	if (AnimBlueprint->HasAnyFlags(RF_NeedPostLoad))
	{
		//Compilation during loading .. need to verify node guids as some anim blueprints have duplicated guids

		TArray<UEdGraph*> ChildGraphs;
		ChildGraphs.Reserve(20);

		TMap<FGuid, UEdGraphNode*> GuidMap;
		GuidMap.Reserve(200);

		// Tracking to see if we need to warn for deterministic cooking
		bool bNodeGuidsRegenerated = false;

		auto CheckGraph = [&bNodeGuidsRegenerated, &GuidMap, &ChildGraphs](UEdGraph* InGraph)
		{
			if (AnimationEditorUtils::IsAnimGraph(InGraph))
			{
				ChildGraphs.Reset();
				AnimationEditorUtils::FindChildGraphsFromNodes(InGraph, ChildGraphs);

				for (int32 Index = 0; Index < ChildGraphs.Num(); ++Index) // Not ranged for as we modify array within the loop
				{
					UEdGraph* ChildGraph = ChildGraphs[Index];

					// Get subgraphs before continuing 
					AnimationEditorUtils::FindChildGraphsFromNodes(ChildGraph, ChildGraphs);

					for (UEdGraphNode* Node : ChildGraph->Nodes)
					{
						if (Node)
						{
							if (UEdGraphNode** FoundNode = GuidMap.Find(Node->NodeGuid))
							{
								if (*FoundNode != Node)
								{
									bNodeGuidsRegenerated = true;
								
									Node->CreateNewGuid(); // GUID is already being used, create a new one.
								}
							}
							else
							{
								GuidMap.Add(Node->NodeGuid, Node);
							}
						}
					}
				}
			}
		};

		for (UEdGraph* Graph : AnimBlueprint->FunctionGraphs)
		{
			CheckGraph(Graph);
		}

		for(FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
		{
			for(UEdGraph* Graph : InterfaceDesc.Graphs)
			{
				CheckGraph(Graph);
			} 
		}

		if(bNodeGuidsRegenerated)
		{
			UE_LOGF(LogAnimation, Warning, "Animation Blueprint %ls has nodes with invalid node guids that have been regenerated. This blueprint "
				"will not cook deterministically until it is resaved.", *AnimBlueprint->GetPathName());
		}
	}

	FAnimBlueprintCompilerCreationContext CreationContext(this);
	UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [&CreationContext](UAnimBlueprintExtension* InExtension)
	{
		InExtension->BeginCompilation(CreationContext);
	});

	// Determine if there is an anim blueprint in the ancestry of this class
	bIsDerivedAnimBlueprint = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint) != NULL;

	// Regenerate temporary stub functions
	// We do this here to catch the standard and 'fast' (compilation manager) compilation paths
	CreateAnimGraphStubFunctions();

	// Stash any existing sparse class data struct here, as we may regenerate it
	if (AnimBlueprint->GeneratedClass && AnimBlueprint->GeneratedClass->GetSparseClassDataStruct())
	{
		OldSparseClassDataStruct = AnimBlueprint->GeneratedClass->GetSparseClassDataStruct();
	}
}

FAnimBlueprintCompilerContext::~FAnimBlueprintCompilerContext()
{
	DestroyAnimGraphStubFunctions();

	UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [](UAnimBlueprintExtension* InExtension)
	{
	    InExtension->EndCompilation();
	});
}

void FAnimBlueprintCompilerContext::ForAllSubGraphs(UEdGraph* InGraph, TFunctionRef<void(UEdGraph*)> InPerGraphFunction)
{
	TArray<UEdGraph*> AllGraphs;
	AllGraphs.Add(InGraph);
	InGraph->GetAllChildrenGraphs(AllGraphs);

	for(UEdGraph* CurrGraph : AllGraphs)
	{
		InPerGraphFunction(CurrGraph);
	}
};

void FAnimBlueprintCompilerContext::CreateClassVariablesFromBlueprint()
{
	FKismetCompilerContext::CreateClassVariablesFromBlueprint();

	if(!bIsDerivedAnimBlueprint)
	{
		auto ProcessGraph = [this](UEdGraph* InGraph)
		{
			TArray<IClassVariableCreator*> ClassVariableCreators;
			InGraph->GetNodesOfClass(ClassVariableCreators);
			FAnimBlueprintVariableCreationContext CreationContext(this);
			for(IClassVariableCreator* ClassVariableCreator : ClassVariableCreators)
			{
				ClassVariableCreator->CreateClassVariablesFromBlueprint(CreationContext);
			}
		};

		for (UEdGraph* UbergraphPage : Blueprint->UbergraphPages)
		{
			ForAllSubGraphs(UbergraphPage, ProcessGraph);
		}

		for (UEdGraph* Graph : Blueprint->FunctionGraphs)
		{
			ForAllSubGraphs(Graph, ProcessGraph);
		}

		for(FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
		{
			for(UEdGraph* Graph : InterfaceDesc.Graphs)
			{
				ForAllSubGraphs(Graph, ProcessGraph);
			}
		}
	}
}


UEdGraphSchema_K2* FAnimBlueprintCompilerContext::CreateSchema()
{
	AnimSchema = NewObject<UAnimationGraphSchema>();
	return AnimSchema;
}

void FAnimBlueprintCompilerContext::ProcessAnimationNode(UAnimGraphNode_Base* VisualAnimNode)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_ProcessAnimationNode);
	
	// Early out if this node has already been processed
	if (AllocatedAnimNodes.Contains(VisualAnimNode))
	{
		return;
	}

	// Make sure the visual node has a runtime node template
	const UScriptStruct* NodeType = VisualAnimNode->GetFNodeType();
	if (NodeType == nullptr)
	{
		MessageLog.Error(TEXT("@@ has no animation node member"), VisualAnimNode);
		return;
	}

	// Give the visual node a chance to do validation
	{
		const int32 PreValidationErrorCount = MessageLog.NumErrors;
		VisualAnimNode->ValidateAnimNodeDuringCompilation(AnimBlueprint->TargetSkeleton, MessageLog);
		VisualAnimNode->BakeDataDuringCompilation(MessageLog);
		if (MessageLog.NumErrors != PreValidationErrorCount)
		{
			return;
		}
	}

	// Create a property for the node
	const FString NodeVariableName = ClassScopeNetNameMap.MakeValidName(VisualAnimNode);

	const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();

	FEdGraphPinType NodeVariableType;
	NodeVariableType.PinCategory = UAnimationGraphSchema::PC_Struct;
	NodeVariableType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UScriptStruct*>(NodeType));

	FStructProperty* NewProperty = CastField<FStructProperty>(CreateVariable(FName(*NodeVariableName), NodeVariableType));

	if (NewProperty == nullptr)
	{
		MessageLog.Error(TEXT("ICE: Failed to create node property for @@"), VisualAnimNode);
	}

	// Create a handler property in constants
	UScriptStruct* HandlerStruct = nullptr;
	if(const UAnimGraphNodeBinding* Binding = VisualAnimNode->GetBinding())
	{
		HandlerStruct = Binding->GetAnimNodeHandlerStruct();
		check(HandlerStruct->IsChildOf(FAnimNodeExposedValueHandler::StaticStruct()));
	}
	else
	{
		HandlerStruct = FAnimNodeExposedValueHandler::StaticStruct();
	}

	FEdGraphPinType HandlerVariableType;
	HandlerVariableType.PinCategory = UAnimationGraphSchema::PC_Struct;
	HandlerVariableType.PinSubCategoryObject = MakeWeakObjectPtr(HandlerStruct);

	FStructProperty* NewHandlerProperty = CastField<FStructProperty>(CreateStructVariable(NewAnimBlueprintConstants, FName(*NodeVariableName), HandlerVariableType));
	if (NewHandlerProperty == nullptr)
	{
		MessageLog.Error(TEXT("ICE: Failed to create node handler property for @@"), VisualAnimNode);
	}

	GatherFoldRecordsForAnimationNode(NodeType, NewProperty, VisualAnimNode);

	// Register this node with the compile-time data structures
	const int32 AllocatedIndex = AllocateNodeIndexCounter++;
	AllocatedAnimNodes.Add(VisualAnimNode, NewProperty);
	AllocatedAnimNodeHandlers.Add(VisualAnimNode, NewHandlerProperty);
	AllocatedNodePropertiesToNodes.Add(NewProperty, VisualAnimNode);
	AllocatedAnimNodeIndices.Add(VisualAnimNode, AllocatedIndex);
	AllocatedPropertiesByIndex.Add(AllocatedIndex, NewProperty);

	UAnimGraphNode_Base* TrueSourceObject = MessageLog.FindSourceObjectTypeChecked<UAnimGraphNode_Base>(VisualAnimNode);
	SourceNodeToProcessedNodeMap.Add(TrueSourceObject, VisualAnimNode);

	// Register the slightly more permanent debug information
	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();
	FAnimBlueprintDebugData& NewAnimBlueprintDebugData = NewAnimBlueprintClass->GetAnimBlueprintDebugData();
	NewAnimBlueprintDebugData.NodePropertyToIndexMap.Add(TrueSourceObject, AllocatedIndex);
	NewAnimBlueprintDebugData.NodeGuidToIndexMap.Add(TrueSourceObject->NodeGuid, AllocatedIndex);
	NewAnimBlueprintDebugData.NodePropertyIndexToNodeMap.Add(AllocatedIndex, TrueSourceObject);
	NewAnimBlueprintClass->GetDebugData().RegisterClassPropertyAssociation(TrueSourceObject, NewProperty);

	FAnimBlueprintGeneratedClassCompiledData CompiledData(NewAnimBlueprintClass);
	FAnimBlueprintCompilationContext CompilerContext(this);
	VisualAnimNode->ProcessDuringCompilation(CompilerContext, CompiledData);
}

void FAnimBlueprintCompilerContext::ProcessExtensions()
{
	TArray<UAnimBlueprintExtension*> Extensions = UAnimBlueprintExtension::GetExtensions(AnimBlueprint);
	
	// Sort extensions by class name (for determinism)
	Algo::Sort(Extensions, [](UAnimBlueprintExtension* InExtensionA, UAnimBlueprintExtension* InExtensionB)
	{
		return InExtensionA->GetClass()->GetName() < InExtensionB->GetClass()->GetName();
	});

	// Process all gathered class extensions
	for(UAnimBlueprintExtension* Extension : Extensions)
	{
		const FString ExtensionVariableName = ClassScopeNetNameMap.MakeValidName(Extension);

		const UScriptStruct* InstanceDataType = Extension->GetInstanceDataType();
		check(InstanceDataType->IsChildOf(FAnimSubsystemInstance::StaticStruct()));

		const UScriptStruct* ClassDataType = Extension->GetClassDataType();
		check(ClassDataType->IsChildOf(FAnimSubsystem::StaticStruct()));
		
		// Skip creating any properties if both are the default
		if(InstanceDataType != FAnimSubsystemInstance::StaticStruct() || ClassDataType != FAnimSubsystem::StaticStruct())
		{
			// Process instance data (mutable)
			{
				FEdGraphPinType ExtensionVariableType;
				ExtensionVariableType.PinCategory = UAnimationGraphSchema::PC_Struct;
				ExtensionVariableType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UScriptStruct*>(InstanceDataType));

				FStructProperty* NewProperty = CastField<FStructProperty>(CreateVariable(FName(*ExtensionVariableName), ExtensionVariableType));
				if (NewProperty == nullptr)
				{
					MessageLog.Error(*FText::Format(LOCTEXT("ExtensionPropertyCreationFailed", "Failed to create extension property for '{0}'"), FText::FromString(Extension->GetName())).ToString());
				}
				else
				{
					ExtensionToInstancePropertyMap.Add(Extension, NewProperty);
					InstancePropertyToExtensionMap.Add(NewProperty, Extension);
				}
			}

			// Process class data (constants)
			{
				FEdGraphPinType ExtensionVariableType;
				ExtensionVariableType.PinCategory = UAnimationGraphSchema::PC_Struct;
				ExtensionVariableType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UScriptStruct*>(ClassDataType));

				FStructProperty* NewProperty = CastField<FStructProperty>(CreateStructVariable(NewAnimBlueprintConstants, FName(*ExtensionVariableName), ExtensionVariableType));
				if (NewProperty == nullptr)
				{
					MessageLog.Error(*FText::Format(LOCTEXT("ExtensionPropertyCreationFailed", "Failed to create extension property for '{0}'"), FText::FromString(Extension->GetName())).ToString());
				}
				else
				{
					ExtensionToClassPropertyMap.Add(Extension, NewProperty);
					ClassPropertyToExtensionMap.Add(NewProperty, Extension);
				}
			}
		}
	}
}

void FAnimBlueprintCompilerContext::GatherFoldRecordsForAnimationNode(const UScriptStruct* InNodeType, FStructProperty* InNodeProperty, UAnimGraphNode_Base* InVisualAnimNode)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_GatherFoldRecords);

	static const FName NAME_FoldProperty("FoldProperty");

	// Run through node properties to see if any are eligible for folding
	for(TFieldIterator<FProperty> It(InNodeType); It; ++It)
	{
		FProperty* SubProperty = *It;

		if(FArrayProperty* ArrayProperty = CastField<FArrayProperty>(SubProperty))
		{
			bool bAllPinsExposed = true;
			bool bAllPinsDisconnected = true;
			const bool bAlwaysDynamic = InVisualAnimNode->AlwaysDynamicProperties.Contains(SubProperty->GetFName());

			// If a value is exposed on a pin but disconnected, push the value to the (intermediate) node here to simplify later logic
			if(SubProperty->HasAnyPropertyFlags(CPF_Edit|CPF_BlueprintVisible))
			{
				FStructProperty* AnimGraphNodeProperty = InVisualAnimNode->GetFNodeProperty();

				// Check the anim node property is contained in the anim graph node
				check(AnimGraphNodeProperty->GetOwner<UClass>() && InVisualAnimNode->GetClass()->IsChildOf(AnimGraphNodeProperty->GetOwner<UClass>()));
				const void* Node = AnimGraphNodeProperty->ContainerPtrToValuePtr<void>(InVisualAnimNode);

				// Check the anim node's property is contained in the anim node
				check(SubProperty->GetOwner<UStruct>() && AnimGraphNodeProperty->Struct->IsChildOf(SubProperty->GetOwner<UStruct>()));
				const void* TargetPtr = SubProperty->ContainerPtrToValuePtr<void>(Node);

				// Run through each array element - we can only fold array properties if all values are constants
				FScriptArrayHelper ArrayHelper(ArrayProperty, TargetPtr);

				for(int32 ArrayIndex = 0; ArrayIndex < ArrayHelper.Num(); ++ArrayIndex)
				{
					const FString ArrayElementPinName = SubProperty->GetName() + FString::Printf(TEXT("_%d"), ArrayIndex);
					UEdGraphPin* Pin = InVisualAnimNode->FindPin(ArrayElementPinName);
					const bool bExposedOnPin = Pin != nullptr;
					const bool bPinConnected = InVisualAnimNode->IsPinExposedAndLinked(ArrayElementPinName, EGPD_Input);
					const bool bPinBound = InVisualAnimNode->IsPinExposedAndBound(ArrayElementPinName, EGPD_Input);
					
					if(bExposedOnPin)
					{
						if(!(bPinConnected || bPinBound))
						{
							check(Pin);

							if(!FBlueprintEditorUtils::PropertyValueFromString_Direct(ArrayProperty->Inner, Pin->GetDefaultAsString(), ArrayHelper.GetRawPtr(ArrayIndex)))
							{
								MessageLog.Warning(TEXT("Unable to push default value for array pin @@ on @@"), Pin, InVisualAnimNode);
							}
						}
						else
						{
							bAllPinsDisconnected = false;
						}
					}
					else
					{
						bAllPinsExposed = false;
					}
				}
			}

			if(SubProperty->HasMetaData(NAME_FoldProperty))
			{
				// Add folding candidate
				AddFoldedPropertyRecord(InVisualAnimNode, InNodeProperty, SubProperty, bAllPinsExposed, !bAllPinsDisconnected, bAlwaysDynamic);
			}
		}
		else
		{
			UEdGraphPin* Pin = InVisualAnimNode->FindPin(SubProperty->GetName());
			const bool bExposedOnPin = Pin != nullptr;
			const bool bPinConnected = InVisualAnimNode->IsPinExposedAndLinked(SubProperty->GetName(), EGPD_Input);
			const bool bPinBound = InVisualAnimNode->IsPinExposedAndBound(SubProperty->GetName(), EGPD_Input);
			const bool bAlwaysDynamic = InVisualAnimNode->AlwaysDynamicProperties.Contains(SubProperty->GetFName());

			// If a value is exposed on a pin but disconnected, push the value to the (intermediate) node here to simplify later logic
			if(SubProperty->HasAnyPropertyFlags(CPF_Edit|CPF_BlueprintVisible))
			{
				if(bExposedOnPin && !(bPinConnected || bPinBound))
				{
					check(Pin);

					FStructProperty* AnimGraphNodeProperty = InVisualAnimNode->GetFNodeProperty();

					// Check the anim node property is contained in the anim graph node
					check(AnimGraphNodeProperty->GetOwner<UClass>() && InVisualAnimNode->GetClass()->IsChildOf(AnimGraphNodeProperty->GetOwner<UClass>()));
					const void* Node = AnimGraphNodeProperty->ContainerPtrToValuePtr<void>(InVisualAnimNode);

					// Check the anim node's property is contained in the anim node
					check(SubProperty->GetOwner<UStruct>() && AnimGraphNodeProperty->Struct->IsChildOf(SubProperty->GetOwner<UStruct>()));
					const void* TargetPtr = SubProperty->ContainerPtrToValuePtr<void>(Node);

					if(!FBlueprintEditorUtils::PropertyValueFromString_Direct(SubProperty, Pin->GetDefaultAsString(), (uint8*)TargetPtr))
					{
						MessageLog.Warning(TEXT("Unable to push default value for pin @@ on @@"), Pin, InVisualAnimNode);
					}
				}
			}

			if(SubProperty->HasMetaData(NAME_FoldProperty))
			{
				// Add folding candidate
				AddFoldedPropertyRecord(InVisualAnimNode, InNodeProperty, SubProperty, bExposedOnPin, bPinConnected || bPinBound, bAlwaysDynamic);
			}
		}
	}
}

int32 FAnimBlueprintCompilerContext::GetAllocationIndexOfNode(UAnimGraphNode_Base* VisualAnimNode)
{
	ProcessAnimationNode(VisualAnimNode);
	int32* pResult = AllocatedAnimNodeIndices.Find(VisualAnimNode);
	return (pResult != NULL) ? *pResult : INDEX_NONE;
}

bool FAnimBlueprintCompilerContext::ShouldForceKeepNode(const UEdGraphNode* Node) const
{
	// Force keep anim nodes during the standard pruning step. Isolated ones will then be removed via PruneIsolatedAnimationNodes.
	// Anim graph nodes are always culled at their expansion step anyways.
	return Node->IsA<UAnimGraphNode_Base>();
}

void FAnimBlueprintCompilerContext::PostExpansionStep(const UEdGraph* Graph)
{
	FAnimBlueprintGeneratedClassCompiledData CompiledData(GetNewAnimBlueprintClass());
	FAnimBlueprintPostExpansionStepContext CompilerContext(this);
	UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [Graph, &CompiledData, &CompilerContext](UAnimBlueprintExtension* InExtension)
	{
		InExtension->PostExpansionStep(Graph, CompilerContext, CompiledData);
	});
}

void FAnimBlueprintCompilerContext::PruneIsolatedAnimationNodes(const TArray<UAnimGraphNode_Base*>& RootSet, TArray<UAnimGraphNode_Base*>& GraphNodes)
{
	struct FNodeVisitorDownPoseWires
	{
		TSet<UEdGraphNode*> VisitedNodes;
		const UAnimationGraphSchema* Schema;

		FNodeVisitorDownPoseWires()
		{
			Schema = GetDefault<UAnimationGraphSchema>();
		}

		void TraverseNodes(UEdGraphNode* Node)
		{
			VisitedNodes.Add(Node);

			// Follow every exec output pin
			for (int32 i = 0; i < Node->Pins.Num(); ++i)
			{
				UEdGraphPin* MyPin = Node->Pins[i];

				if ((MyPin->Direction == EGPD_Input) && (Schema->IsPosePin(MyPin->PinType)))
				{
					for (int32 j = 0; j < MyPin->LinkedTo.Num(); ++j)
					{
						UEdGraphPin* OtherPin = MyPin->LinkedTo[j];
						UEdGraphNode* OtherNode = OtherPin->GetOwningNode();
						if (!VisitedNodes.Contains(OtherNode))
						{
							TraverseNodes(OtherNode);
						}
					}
				}
			}
		}
	};

	// Prune the nodes that aren't reachable via an animation pose link
	FNodeVisitorDownPoseWires Visitor;

	for (auto RootIt = RootSet.CreateConstIterator(); RootIt; ++RootIt)
	{
		UAnimGraphNode_Base* RootNode = *RootIt;
		Visitor.TraverseNodes(RootNode);
	}

	for (int32 NodeIndex = 0; NodeIndex < GraphNodes.Num(); ++NodeIndex)
	{
		UAnimGraphNode_Base* Node = GraphNodes[NodeIndex];

		// We cant prune linked input poses as even if they are not linked to the root, they are needed for the dynamic link phase at runtime
		if (!Visitor.VisitedNodes.Contains(Node) && !IsNodePure(Node) && !Node->IsA<UAnimGraphNode_LinkedInputPose>())
		{
			Node->BreakAllNodeLinks();
			GraphNodes.RemoveAtSwap(NodeIndex);
			--NodeIndex;
		}
	}
}

void FAnimBlueprintCompilerContext::ProcessAnimationNodes(TArray<UAnimGraphNode_Base*>& AnimNodeList)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_ProcessAnimationNodes);
	
	// Process the remaining nodes
	for (UAnimGraphNode_Base* AnimNode : AnimNodeList)
	{
		ProcessAnimationNode(AnimNode);
	}
}

void FAnimBlueprintCompilerContext::GetLinkedAnimNodes(UAnimGraphNode_Base* InGraphNode, TArray<UAnimGraphNode_Base*> &LinkedAnimNodes) const
{
	for(UEdGraphPin* Pin : InGraphNode->Pins)
	{
		if(Pin->Direction == EEdGraphPinDirection::EGPD_Input &&
		   Pin->PinType.PinCategory == TEXT("struct"))
		{
			if(UScriptStruct* Struct = Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get()))
			{
				if(Struct->IsChildOf(FPoseLinkBase::StaticStruct()))
				{
					GetLinkedAnimNodes_TraversePin(Pin, LinkedAnimNodes);
				}
			}
		}
	}
}

void FAnimBlueprintCompilerContext::GetLinkedAnimNodes_TraversePin(UEdGraphPin* InPin, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const
{
	if(!InPin)
	{
		return;
	}

	for(UEdGraphPin* LinkedPin : InPin->LinkedTo)
	{
		if(!LinkedPin)
		{
			continue;
		}
		
		UEdGraphNode* OwningNode = LinkedPin->GetOwningNode();

		if(UK2Node_Knot* InnerKnot = Cast<UK2Node_Knot>(OwningNode))
		{
			GetLinkedAnimNodes_TraversePin(InnerKnot->GetInputPin(), LinkedAnimNodes);
		}
		else if(UAnimGraphNode_Base* AnimNode = Cast<UAnimGraphNode_Base>(OwningNode))
		{
			GetLinkedAnimNodes_ProcessAnimNode(AnimNode, LinkedAnimNodes);
		}
	}
}

void FAnimBlueprintCompilerContext::GetLinkedAnimNodes_ProcessAnimNode(UAnimGraphNode_Base* AnimNode, TArray<UAnimGraphNode_Base *>& LinkedAnimNodes) const
{
	if(!AllocatedAnimNodes.Contains(AnimNode))
	{
		UAnimGraphNode_Base* TrueSourceNode = MessageLog.FindSourceObjectTypeChecked<UAnimGraphNode_Base>(AnimNode);

		if(UAnimGraphNode_Base*const* AllocatedNode = SourceNodeToProcessedNodeMap.Find(TrueSourceNode))
		{
			LinkedAnimNodes.Add(*AllocatedNode);
		}
		else
		{
			FString ErrorString = FText::Format(LOCTEXT("MissingLinkFmt", "Missing allocated node for {0} while searching for node links - likely due to the node having outstanding errors."), FText::FromString(AnimNode->GetName())).ToString();
			MessageLog.Error(*ErrorString);
		}
	}
	else
	{
		LinkedAnimNodes.Add(AnimNode);
	}
}

void FAnimBlueprintCompilerContext::ProcessAllAnimationNodes()
{
	BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_ProcessAllAnimationNodes);
	
	// Validate that we have a skeleton
	if ((AnimBlueprint->TargetSkeleton == nullptr) && !AnimBlueprint->bIsNewlyCreated && !AnimBlueprint->bIsTemplate)
	{
		MessageLog.Error(*LOCTEXT("NoSkeleton", "@@ - The skeleton asset for this animation Blueprint is missing, so it cannot be compiled!").ToString(), AnimBlueprint);
		return;
	}

	// Build the raw node lists
	TArray<UAnimGraphNode_Base*> RootAnimNodeList;
	ConsolidatedEventGraph->GetNodesOfClass<UAnimGraphNode_Base>(RootAnimNodeList);

	// We recursively build the node lists for pre- and post-processing phases to make sure
	// we catch any handler-relevant nodes in sub-graphs
	TArray<UAnimGraphNode_Base*> AllSubGraphsAnimNodeList;
	ForAllSubGraphs(ConsolidatedEventGraph, [&AllSubGraphsAnimNodeList](UEdGraph* InGraph)
	{
		InGraph->GetNodesOfClass<UAnimGraphNode_Base>(AllSubGraphsAnimNodeList);
	});

	// Find the root nodes
	TArray<UAnimGraphNode_Base*> RootSet;

	AllocateNodeIndexCounter = 0;

	for (UAnimGraphNode_Base* SourceNode : RootAnimNodeList)
	{
		UAnimGraphNode_Base* TrueNode = MessageLog.FindSourceObjectTypeChecked<UAnimGraphNode_Base>(SourceNode);
		TrueNode->BlueprintUsage = EBlueprintUsage::NoProperties;

		if(SourceNode->IsNodeRootSet())
		{
			RootSet.Add(SourceNode);
		}
	}

	if (RootAnimNodeList.Num() > 0)
	{
		// Prune any anim nodes (they will be skipped by PruneIsolatedNodes above)
		PruneIsolatedAnimationNodes(RootSet, RootAnimNodeList);

		// Validate the graph
		{
			TGuardValue<bool> SilentModeGuard(MessageLog.bSilentMode, true);
			TGuardValue<bool> AnnotateMentionedNodesGuard(MessageLog.bAnnotateMentionedNodes, false);
			
			ValidateGraphIsWellFormed(ConsolidatedEventGraph);
			
			MessageLog.Messages.Empty();
		}
		
		FAnimBlueprintGeneratedClassCompiledData CompiledData(GetNewAnimBlueprintClass());
		FAnimBlueprintCompilationContext CompilerContext(this);
		UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [&AllSubGraphsAnimNodeList, &CompiledData, &CompilerContext](UAnimBlueprintExtension* InExtension)
		{
			InExtension->PreProcessAnimationNodes(AllSubGraphsAnimNodeList, CompilerContext, CompiledData);
		});

		// Process the animation nodes
		ProcessAnimationNodes(RootAnimNodeList);

		// Process any extensions
		ProcessExtensions();
		
		// Fold any constants
		ProcessFoldedPropertyRecords();

		UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [&AllSubGraphsAnimNodeList, &CompiledData, &CompilerContext](UAnimBlueprintExtension* InExtension)
		{
			InExtension->PostProcessAnimationNodes(AllSubGraphsAnimNodeList, CompilerContext, CompiledData);
		});	
	}
	else
	{
		MessageLog.Error(*LOCTEXT("ExpectedAFunctionEntry_Error", "Expected at least one animation node, but did not find any").ToString());
	}
}

void FAnimBlueprintCompilerContext::CopyTermDefaultsToDefaultObject(UObject* DefaultObject)
{
	Super::CopyTermDefaultsToDefaultObject(DefaultObject);

	UAnimInstance* DefaultAnimInstance = Cast<UAnimInstance>(DefaultObject);
	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();

	if (bIsDerivedAnimBlueprint && DefaultAnimInstance)
	{
		//Need To have a sparse class data struct for BuildConstantProperties below 
		if (NewAnimBlueprintClass->GetSparseClassDataStruct() == nullptr)
		{
			RecreateSparseClassData();
		}
		else
		{
			// Ensure we have constant properties & anim node data rebuilt
			NewAnimBlueprintClass->BuildConstantProperties();
		}
		CopyAnimNodeDataFromRoot();
		
		// If we are a derived animation graph; apply any stored overrides.
		// Restore values from the root class to catch values where the override has been removed.
		UAnimBlueprintGeneratedClass* RootAnimClass = NewAnimBlueprintClass;
		while (UAnimBlueprintGeneratedClass* NextClass = Cast<UAnimBlueprintGeneratedClass>(RootAnimClass->GetSuperClass()))
		{
			RootAnimClass = NextClass;
		}
		
		UObject* RootDefaultObject = RootAnimClass->GetDefaultObject();

		for (TFieldIterator<FProperty> It(RootAnimClass); It; ++It)
		{
			FProperty* RootProp = *It;

			if (FStructProperty* RootStructProp = CastField<FStructProperty>(RootProp))
			{
				if (RootStructProp->Struct->IsChildOf(FAnimNode_Base::StaticStruct()))
				{
					FStructProperty* ChildStructProp = FindFProperty<FStructProperty>(NewAnimBlueprintClass, *RootStructProp->GetName());
					check(ChildStructProp);
					uint8* SourcePtr = RootStructProp->ContainerPtrToValuePtr<uint8>(RootDefaultObject);
					uint8* DestPtr = ChildStructProp->ContainerPtrToValuePtr<uint8>(DefaultAnimInstance);
					check(SourcePtr && DestPtr);
					RootStructProp->CopyCompleteValue(DestPtr, SourcePtr);
				}
			}
		}

		// Copy from root sparse class data to our new class
		if(NewAnimBlueprintClass->GetConstantNodeData() && RootAnimClass->GetConstantNodeData())
		{
			checkf(NewAnimBlueprintClass->GetSparseClassDataStruct()->IsChildOf(RootAnimClass->GetSparseClassDataStruct()), TEXT("SparseClassDataStruct for AnimBPClass '%s' (Struct:%s) is not a child of SparseClassDataStruct for AnimBPClass '%s' (Struct:%s)"), *GetPathNameSafe(NewAnimBlueprintClass), *GetFullNameSafe(NewAnimBlueprintClass->GetSparseClassDataStruct()), *GetPathNameSafe(RootAnimClass), *GetFullNameSafe(RootAnimClass->GetSparseClassDataStruct()));

			for (TFieldIterator<FProperty> PropertyIt(RootAnimClass->GetSparseClassDataStruct()); PropertyIt; ++PropertyIt)
			{
				FProperty* RootProperty = *PropertyIt;
				FProperty* ChildProperty = FindFProperty<FProperty>(NewAnimBlueprintClass->GetSparseClassDataStruct(), *RootProperty->GetName());
				check(ChildProperty);
				
				const uint8* SourcePtr = RootProperty->ContainerPtrToValuePtr<uint8>(RootAnimClass->GetConstantNodeData());
				uint8* DestPtr = const_cast<uint8*>(ChildProperty->ContainerPtrToValuePtr<uint8>(NewAnimBlueprintClass->GetConstantNodeData()));
				check(SourcePtr && DestPtr);
				RootProperty->CopyCompleteValue(DestPtr, SourcePtr);
			}
		}
		
		// Copy from root mutable data to our new mutable data
		if (NewAnimBlueprintClass->GetMutableNodeData(DefaultObject) && RootAnimClass->GetMutableNodeData(RootDefaultObject))
		{
			// These properties should have been linked and cached by now
			check(RootAnimClass->MutableNodeDataProperty);
			check(RootAnimClass->MutableNodeDataProperty->Struct);
			check(NewAnimBlueprintClass->MutableNodeDataProperty);
			check(NewAnimBlueprintClass->MutableNodeDataProperty->Struct);
			
			for (TFieldIterator<FProperty> PropertyIt(RootAnimClass->MutableNodeDataProperty->Struct); PropertyIt; ++PropertyIt)
			{
				FProperty* RootProperty = *PropertyIt;
				FProperty* ChildProperty = FindFProperty<FProperty>(NewAnimBlueprintClass->MutableNodeDataProperty->Struct, *RootProperty->GetName());
				check(ChildProperty != nullptr);
		 
				const uint8* SourcePtr = RootProperty->ContainerPtrToValuePtr<uint8>(RootAnimClass->GetMutableNodeData(RootDefaultObject));
				uint8* DestPtr = ChildProperty->ContainerPtrToValuePtr<uint8>(NewAnimBlueprintClass->GetMutableNodeData(DefaultObject));
				check(SourcePtr != nullptr && DestPtr != nullptr);
				RootProperty->CopyCompleteValue(DestPtr, SourcePtr);
			}
		}
		
		// Re-initialize node data tables (they would be overwritten in the loop above)
		NewAnimBlueprintClass->InitializeAnimNodeData(DefaultObject, true);
	}

	// Give game-specific logic a chance to replace animations
	if(DefaultAnimInstance)
	{
		DefaultAnimInstance->ApplyAnimOverridesToCDO(MessageLog);
	}

	if (bIsDerivedAnimBlueprint && DefaultAnimInstance)
	{
		// Patch the overridden values into the CDO
		TArray<FAnimParentNodeAssetOverride*> AssetOverrides;
		AnimBlueprint->GetAssetOverrides(AssetOverrides);
		for (FAnimParentNodeAssetOverride* Override : AssetOverrides)
		{
			if (Override->NewAsset)
			{
				int32 NodeIndex = NewAnimBlueprintClass->GetNodeIndexFromGuid(Override->ParentNodeGuid, EPropertySearchMode::Hierarchy);
				if(NodeIndex != INDEX_NONE)
				{
					const UAnimGraphNode_Base* GraphNode = Cast<UAnimGraphNode_Base>(NewAnimBlueprintClass->GetVisualNodeFromNodePropertyIndex(NodeIndex, EPropertySearchMode::Hierarchy));
					FAnimNode_Base* BaseNode = NewAnimBlueprintClass->GetPropertyInstance<FAnimNode_Base>(DefaultAnimInstance, Override->ParentNodeGuid, EPropertySearchMode::Hierarchy);
					if (GraphNode && BaseNode)
					{
						FAnimBlueprintNodeOverrideAssetsContext Context(BaseNode, GraphNode->GetFNodeType());
						Context.AddAsset(Override->NewAsset);
						GraphNode->OverrideAssets(Context);
					}
				}
			}
		}

		return;
	}

	if(DefaultAnimInstance)
	{
		int32 LinkIndexCount = 0;
		TMap<UAnimGraphNode_Base*, int32> LinkIndexMap;
		TMap<UAnimGraphNode_Base*, uint8*> NodeBaseAddresses;

		FAnimBlueprintGeneratedClassCompiledData CompiledData(NewAnimBlueprintClass);
		FAnimBlueprintCopyTermDefaultsContext CompilerContext(this);

		// Initialize animation nodes from their templates
		for (TFieldIterator<FProperty> It(DefaultAnimInstance->GetClass(), EFieldIteratorFlags::ExcludeSuper); It; ++It)
		{
			FProperty* TargetProperty = *It;

			if (UAnimGraphNode_Base* VisualAnimNode = AllocatedNodePropertiesToNodes.FindRef(TargetProperty))
			{
				const FStructProperty* SourceNodeProperty = VisualAnimNode->GetFNodeProperty();
				check(SourceNodeProperty != NULL);
				check(CastFieldChecked<FStructProperty>(TargetProperty)->Struct == SourceNodeProperty->Struct);

				uint8* DestinationPtr = TargetProperty->ContainerPtrToValuePtr<uint8>(DefaultAnimInstance);
				const uint8* SourcePtr = SourceNodeProperty->ContainerPtrToValuePtr<uint8>(VisualAnimNode);

				FAnimBlueprintNodeCopyTermDefaultsContext NodeContext(DefaultObject, TargetProperty, DestinationPtr, SourcePtr, LinkIndexCount);
				UAnimGraphNode_Base* OriginalAnimNode = Cast<UAnimGraphNode_Base>(MessageLog.FindSourceObject(VisualAnimNode));
				OriginalAnimNode->CopyTermDefaultsToDefaultObject(CompilerContext, NodeContext, CompiledData);

				LinkIndexMap.Add(VisualAnimNode, LinkIndexCount);
				NodeBaseAddresses.Add(VisualAnimNode, DestinationPtr);
				++LinkIndexCount;
			}
		}

		// Applies a set of folded property records to a data area (i.e. the constant or mutable structs)
		auto PatchDataArea = [](void* InData, UScriptStruct* InStruct, const TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>>& InRecords)
		{
			for(const TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>& Record : InRecords)
			{
				if(Record->GeneratedProperty != nullptr)
				{
					FStructProperty* AnimGraphNodeProperty = Record->AnimGraphNode->GetFNodeProperty();

					// Check the anim node property is contained in the anim graph node
					check(AnimGraphNodeProperty->GetOwner<UClass>() && Record->AnimGraphNode->GetClass()->IsChildOf(AnimGraphNodeProperty->GetOwner<UClass>()));
					const void* Node = AnimGraphNodeProperty->ContainerPtrToValuePtr<void>(Record->AnimGraphNode);

					// Check the anim node's property is contained in the anim node
					check(Record->Property->GetOwner<UStruct>() && AnimGraphNodeProperty->Struct->IsChildOf(Record->Property->GetOwner<UStruct>()));
					const void* SourcePtr = Record->Property->ContainerPtrToValuePtr<void>(Node);

					// Check the generated property is a member of the constants struct
					check(Record->GeneratedProperty->GetOwner<UStruct>() && InStruct->IsChildOf(Record->GeneratedProperty->GetOwner<UStruct>()));
					void* TargetPtr = Record->GeneratedProperty->ContainerPtrToValuePtr<void>(InData);

					// Extract underlying property for enums
					FProperty* PropertyToCopyWith = Record->GeneratedProperty;
					if(const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Record->GeneratedProperty))
					{
						PropertyToCopyWith = EnumProperty->GetUnderlyingProperty();
					}

					PropertyToCopyWith->CopyCompleteValue(TargetPtr, SourcePtr);
				}
			}
		};

		if(void* Constants = NewAnimBlueprintClass->GetOrCreateSparseClassData())
		{
 			UScriptStruct* ConstantsStruct = NewAnimBlueprintClass->GetSparseClassDataStruct();
 			check(ConstantsStruct == NewAnimBlueprintConstants);

			// Initialize extensions from their templates
			for (TFieldIterator<FStructProperty> It(ConstantsStruct, EFieldIteratorFlags::ExcludeSuper); It; ++It)
			{
				FStructProperty* TargetProperty = *It;

				if (UAnimBlueprintExtension* Extension = ClassPropertyToExtensionMap.FindRef(TargetProperty))
				{
					if(const FStructProperty* SourceExtensionProperty = Extension->GetClassDataProperty())
					{
						check(SourceExtensionProperty != NULL);
						check(TargetProperty->Struct == SourceExtensionProperty->Struct);

						uint8* DestinationPtr = TargetProperty->ContainerPtrToValuePtr<uint8>(Constants);
						const uint8* SourcePtr = SourceExtensionProperty->ContainerPtrToValuePtr<uint8>(Extension);
					
						FAnimBlueprintExtensionCopyTermDefaultsContext NodeContext(DefaultObject, TargetProperty, DestinationPtr, SourcePtr, LinkIndexCount);
						Extension->CopyTermDefaultsToSparseClassData(CompilerContext, NodeContext);
					}
				}
			}

			// Patch constants
			PatchDataArea(Constants, ConstantsStruct, ConstantPropertyRecords);
		}

		if(NewMutablesProperty)
		{
			check(NewMutablesProperty->GetOwner<UClass>() && DefaultObject->GetClass()->IsChildOf(NewMutablesProperty->GetOwner<UClass>()));
			void* Mutables = NewMutablesProperty->ContainerPtrToValuePtr<void>(DefaultObject);
			UScriptStruct* MutablesStruct = NewMutablesProperty->Struct;
			check(MutablesStruct == NewAnimBlueprintMutables);

			// Patch mutables
			PatchDataArea(Mutables, MutablesStruct, MutablePropertyRecords);
		}
		
		// Initialize extensions from their templates
		for (TFieldIterator<FStructProperty> It(DefaultAnimInstance->GetClass(), EFieldIteratorFlags::ExcludeSuper); It; ++It)
		{
			FStructProperty* TargetProperty = *It;

			if (UAnimBlueprintExtension* Extension = InstancePropertyToExtensionMap.FindRef(TargetProperty))
			{
				uint8* DestinationPtr = TargetProperty->ContainerPtrToValuePtr<uint8>(DefaultAnimInstance);
				const uint8* SourcePtr = nullptr;
				const FStructProperty* SourceExtensionProperty = Extension->GetInstanceDataProperty();

				if (SourceExtensionProperty)
				{
					check(TargetProperty->Struct == SourceExtensionProperty->Struct);
					SourcePtr = SourceExtensionProperty->ContainerPtrToValuePtr<uint8>(Extension);
				}

				FAnimBlueprintExtensionCopyTermDefaultsContext ExtensionContext(DefaultObject, TargetProperty, DestinationPtr, SourcePtr, LinkIndexCount);
				Extension->CopyTermDefaultsToDefaultObject(DefaultAnimInstance, CompilerContext, ExtensionContext);
			}
		}

		// And wire up node links
		for (auto PoseLinkIt = ValidPoseLinkList.CreateIterator(); PoseLinkIt; ++PoseLinkIt)
		{
			FPoseLinkMappingRecord& Record = *PoseLinkIt;

			UAnimGraphNode_Base* LinkingNode = Record.GetLinkingNode();
			UAnimGraphNode_Base* LinkedNode = Record.GetLinkedNode();
		
			// @TODO this is quick solution for crash - if there were previous errors and some nodes were not added, they could still end here -
			// this check avoids that and since there are already errors, compilation won't be successful.
			// but I'd prefer stopping compilation earlier to avoid getting here in first place
			if (LinkIndexMap.Contains(LinkingNode) && LinkIndexMap.Contains(LinkedNode))
			{
				const int32 SourceNodeIndex = LinkIndexMap.FindChecked(LinkingNode);
				const int32 LinkedNodeIndex = LinkIndexMap.FindChecked(LinkedNode);
				uint8* DestinationPtr = NodeBaseAddresses.FindChecked(LinkingNode);

				Record.PatchLinkIndex(DestinationPtr, LinkedNodeIndex, SourceNodeIndex);
			}
		}   

		UAnimBlueprintGeneratedClass* AnimBlueprintGeneratedClass = CastChecked<UAnimBlueprintGeneratedClass>(NewClass);
		
		// copy threaded update flag to CDO
		DefaultAnimInstance->bUseMultiThreadedAnimationUpdate = AnimBlueprint->bUseMultiThreadedAnimationUpdate;

		// Verify thread-safety
		if(GetDefault<UEngine>()->bAllowMultiThreadedAnimationUpdate && DefaultAnimInstance->bUseMultiThreadedAnimationUpdate)
		{
			// If we are a child anim BP, check parent classes & their CDOs
			if (UAnimBlueprintGeneratedClass* ParentClass = Cast<UAnimBlueprintGeneratedClass>(AnimBlueprintGeneratedClass->GetSuperClass()))
			{
				UAnimBlueprint* ParentAnimBlueprint = Cast<UAnimBlueprint>(ParentClass->ClassGeneratedBy);
				if (ParentAnimBlueprint && !ParentAnimBlueprint->bUseMultiThreadedAnimationUpdate)
				{
					DefaultAnimInstance->bUseMultiThreadedAnimationUpdate = false;
				}

				UAnimInstance* ParentDefaultObject = Cast<UAnimInstance>(ParentClass->GetDefaultObject(false));
				if (ParentDefaultObject && !ParentDefaultObject->bUseMultiThreadedAnimationUpdate)
				{
					DefaultAnimInstance->bUseMultiThreadedAnimationUpdate = false;
				}
			}

			// iterate all properties to determine validity
			for (FStructProperty* Property : TFieldRange<FStructProperty>(AnimBlueprintGeneratedClass, EFieldIteratorFlags::IncludeSuper))
			{
				if(Property->Struct->IsChildOf(FAnimNode_Base::StaticStruct()))
				{
					FAnimNode_Base* AnimNode = Property->ContainerPtrToValuePtr<FAnimNode_Base>(DefaultAnimInstance);
					if(!AnimNode->CanUpdateInWorkerThread())
					{
						MessageLog.Warning(*FText::Format(LOCTEXT("HasIncompatibleNode", "Found incompatible node \"{0}\" in blend graph. Disable threaded update or use member variable access."), FText::FromName(Property->Struct->GetFName())).ToString())
							->AddToken(FDocumentationToken::Create(TEXT("Engine/Animation/AnimBlueprints/AnimGraph")));;

						DefaultAnimInstance->bUseMultiThreadedAnimationUpdate = false;
					}
				}
			}

			if (FunctionList.Num() > 0)
			{
				// find the ubergraph in the function list
				FKismetFunctionContext* UbergraphFunctionContext = nullptr;
				for (FKismetFunctionContext& FunctionContext : FunctionList)
				{
					if (FunctionList[0].Function->GetName().StartsWith(TEXT("ExecuteUbergraph")))
					{
						UbergraphFunctionContext = &FunctionContext;
						break;
					}
				}

				if (UbergraphFunctionContext)
				{
					// run through the per-node compiled statements looking for struct-sets used by anim nodes
					for (const TPair<UEdGraphNode*, TArray<FBlueprintCompiledStatement*>>& StatementPair : UbergraphFunctionContext->StatementsPerNode)
					{
						if (UK2Node_StructMemberSet* StructMemberSetNode = Cast<UK2Node_StructMemberSet>(StatementPair.Key))
						{
							UK2Node* SourceNode = CastChecked<UK2Node>(MessageLog.FindSourceObject(StructMemberSetNode));

							if (SourceNode && (StructMemberSetNode->StructType->IsChildOf(FAnimNode_Base::StaticStruct()) || StructMemberSetNode->StructType->IsChildOf(FAnimBlueprintMutableData::StaticStruct())))
							{
								UEdGraph* SourceGraph = SourceNode->GetGraph();
								
								const bool bEmitErrors = false;
								bool bIsThreadSafe = true;

								static const FBoolConfigValueHelper UseLegacyAnimBlueprintThreadSafetyChecks(TEXT("Kismet"), TEXT("bUseLegacyAnimBlueprintThreadSafetyChecks"), GEngineIni);
								if(UseLegacyAnimBlueprintThreadSafetyChecks)
								{
									for (FBlueprintCompiledStatement* Statement : StatementPair.Value)
									{
										if (Statement->Type == KCST_CallFunction && Statement->FunctionToCall)
										{
											// pure function?
											const bool bPureFunctionCall = Statement->FunctionToCall->HasAnyFunctionFlags(FUNC_BlueprintPure);

											// function called on something other than function library or anim instance?
											UClass* FunctionClass = CastChecked<UClass>(Statement->FunctionToCall->GetOuter());
											const bool bFunctionLibraryCall = FunctionClass->IsChildOf<UBlueprintFunctionLibrary>();
											const bool bAnimInstanceCall = FunctionClass->IsChildOf<UAnimInstance>();

											// Allowed/denied? Some functions are not really 'pure', so we give people the opportunity to mark them up.
											// Mark up the class if it is generally thread safe, then unsafe functions can be marked up individually. We assume
											// that classes are unsafe by default, as well as if they are marked up NotBlueprintThreadSafe.
											const bool bClassThreadSafe = FunctionClass->HasMetaData(TEXT("BlueprintThreadSafe"));
											const bool bClassNotThreadSafe = FunctionClass->HasMetaData(TEXT("NotBlueprintThreadSafe")) || !FunctionClass->HasMetaData(TEXT("BlueprintThreadSafe"));
											const bool bFunctionThreadSafe = Statement->FunctionToCall->HasMetaData(TEXT("BlueprintThreadSafe"));
											const bool bFunctionNotThreadSafe = Statement->FunctionToCall->HasMetaData(TEXT("NotBlueprintThreadSafe"));

											const bool bThreadSafe = (bClassThreadSafe && !bFunctionNotThreadSafe) || (bClassNotThreadSafe && bFunctionThreadSafe);

											const bool bValidForUsage = bPureFunctionCall && bThreadSafe && (bFunctionLibraryCall || bAnimInstanceCall);

											if (!bValidForUsage)
											{
												UEdGraphNode* FunctionNode = nullptr;
												if (Statement->FunctionContext && Statement->FunctionContext->SourcePin)
												{
													FunctionNode = Statement->FunctionContext->SourcePin->GetOwningNode();
												}
												else if (Statement->LHS && Statement->LHS->SourcePin)
												{
													FunctionNode = Statement->LHS->SourcePin->GetOwningNode();
												}

												if (FunctionNode)
												{
													MessageLog.Warning(*LOCTEXT("NotThreadSafeWarningNodeContext", "Node @@ uses potentially thread-unsafe call @@. Disable threaded update or use a thread-safe call. Function may need BlueprintThreadSafe metadata adding.").ToString(), SourceNode, FunctionNode)
														->AddToken(FDocumentationToken::Create(TEXT("Engine/Animation/AnimBlueprints/AnimGraph")));
												}
												else if (Statement->FunctionToCall)
												{
													MessageLog.Warning(*FText::Format(LOCTEXT("NotThreadSafeWarningFunctionContext", "Node @@ uses potentially thread-unsafe call {0}. Disable threaded update or use a thread-safe call. Function may need BlueprintThreadSafe metadata adding."), Statement->FunctionToCall->GetDisplayNameText()).ToString(), SourceNode)
														->AddToken(FDocumentationToken::Create(TEXT("Engine/Animation/AnimBlueprints/AnimGraph")));
												}
												else
												{
													MessageLog.Warning(*LOCTEXT("NotThreadSafeWarningUnknownContext", "Node @@ uses potentially thread-unsafe call. Disable threaded update or use a thread-safe call.").ToString(), SourceNode)
														->AddToken(FDocumentationToken::Create(TEXT("Engine/Animation/AnimBlueprints/AnimGraph")));
												}

												bIsThreadSafe = false;
											}
										}
									}
								}
								else
								{
									bIsThreadSafe = FKismetCompilerUtilities::CheckFunctionCompiledStatementsThreadSafety(SourceNode, SourceGraph, StatementPair.Value, MessageLog, bEmitErrors);
								}
								
								DefaultAnimInstance->bUseMultiThreadedAnimationUpdate = bIsThreadSafe;
							}
						}
					}
				}
			}
		}
	}
}

void FAnimBlueprintCompilerContext::ExpandSplitPins(UEdGraph* InGraph)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_ExpandSplitPins);

	for (auto NodeIt = InGraph->Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		UK2Node* K2Node = Cast<UK2Node>(*NodeIt);
		if (K2Node != nullptr)
		{
			K2Node->ExpandSplitPins(*this, InGraph);
		}
	}
}

// Merges in any all ubergraph pages into the gathering ubergraph
void FAnimBlueprintCompilerContext::MergeUbergraphPagesIn(UEdGraph* Ubergraph)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_MergeUbergraphPagesIn);

	Super::MergeUbergraphPagesIn(Ubergraph);

	if (bIsDerivedAnimBlueprint)
	{
		// Skip any work related to an anim graph, it's all done by the parent class
		// We do need to make sure that anim node data is correctly copied & remapped to this class
		RecreateSparseClassData();
		CopyAnimNodeDataFromRoot();
	}
	else
	{
		RecreateSparseClassData();
		RecreateMutables();
	
		{
			BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_MoveGraphs);
			
			// Move all animation graph nodes and associated pure logic chains into the consolidated event graph
			auto MoveGraph = [this](UEdGraph* InGraph)
			{
				if (InGraph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()))
				{
					UEdGraph* ClonedGraph;
					{
						BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_CloneGraph);
					
						// Merge all the animation nodes, contents, etc... into the ubergraph
						ClonedGraph = FEdGraphUtilities::CloneGraph(InGraph, NULL, &MessageLog, true);
					}

					// Prune the graph up-front
					const bool bIncludePotentialRootNodes = false;
					PruneIsolatedNodes(ClonedGraph, bIncludePotentialRootNodes);

					const bool bIsLoading = Blueprint->bIsRegeneratingOnLoad || IsAsyncLoading();
					const bool bIsCompiling = Blueprint->bBeingCompiled;
					ClonedGraph->MoveNodesToAnotherGraph(ConsolidatedEventGraph, bIsLoading, bIsCompiling);

					// Move subgraphs too
					ConsolidatedEventGraph->SubGraphs.Append(ClonedGraph->SubGraphs);
				}
			};

			for (UEdGraph* Graph : Blueprint->FunctionGraphs)
			{
				MoveGraph(Graph);
			}

			for(FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
			{
				for(UEdGraph* Graph : InterfaceDesc.Graphs)
				{
					MoveGraph(Graph);
				}
			}
		}

		// Make sure we expand any split pins here before we process animation nodes.
		ForAllSubGraphs(ConsolidatedEventGraph, [this](UEdGraph* InGraph)
		{
			ExpandSplitPins(InGraph);
		});

		// Compile the animation graph
		ProcessAllAnimationNodes();
	}
}

void FAnimBlueprintCompilerContext::CopyAnimNodeDataFromRoot() const
{
	check(bIsDerivedAnimBlueprint);

	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();
	UAnimBlueprintGeneratedClass* RootAnimClass = NewAnimBlueprintClass;
	while (UAnimBlueprintGeneratedClass* NextClass = Cast<UAnimBlueprintGeneratedClass>(RootAnimClass->GetSuperClass()))
	{
		RootAnimClass = NextClass;
	}
	
	// Copy constant/folded data from root class, remapping the class
	NewAnimBlueprintClass->AnimNodeData = RootAnimClass->AnimNodeData;
	NewAnimBlueprintClass->NodeTypeMap = RootAnimClass->NodeTypeMap;
	for(FAnimNodeData& NodeData : NewAnimBlueprintClass->AnimNodeData)
	{
		NodeData.AnimClassInterface = NewAnimBlueprintClass;
	}
}

void FAnimBlueprintCompilerContext::ProcessOneFunctionGraph(UEdGraph* SourceGraph, bool bInternalFunction)
{
	if(!KnownGraphSchemas.FindByPredicate([SourceGraph](const TSubclassOf<UEdGraphSchema>& InSchemaClass)
	{
		return SourceGraph->Schema->IsChildOf(InSchemaClass.Get());
	}))
	{
		// Not known as a schema that this compiler looks at, pass to the default
		Super::ProcessOneFunctionGraph(SourceGraph, bInternalFunction);
	}
}

void FAnimBlueprintCompilerContext::EnsureProperGeneratedClass(UClass*& TargetUClass)
{
	if( TargetUClass && !((UObject*)TargetUClass)->IsA(UAnimBlueprintGeneratedClass::StaticClass()) )
	{
		FKismetCompilerUtilities::ConsignToOblivion(TargetUClass, Blueprint->bIsRegeneratingOnLoad);
		TargetUClass = NULL;
	}
}

void FAnimBlueprintCompilerContext::RecreateSparseClassData()
{
	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();

	// Set up our sparse class data struct
	if (bIsDerivedAnimBlueprint)
	{
		// Get parent class
		UAnimBlueprint* ParentAnimBP = UAnimBlueprint::GetParentAnimBlueprint(AnimBlueprint);
		UAnimBlueprintGeneratedClass* ParentAnimClass = Cast<UAnimBlueprintGeneratedClass>(ParentAnimBP->GeneratedClass);
		
		check(ParentAnimClass);
		if (ParentAnimClass->TargetSkeleton)
		{
			// If our parent has a skeleton it should compiled enough to where it has a sparse class data struct
			check(ParentAnimClass->GetSparseClassDataStruct());
		}

		// Create new sparse class data struct
		NewAnimBlueprintConstants = NewObject<UScriptStruct>(NewAnimBlueprintClass, UAnimBlueprintGeneratedClass::GetConstantsStructName(), RF_Public);

		// Inherit from parent archetype struct if there is any (Just like below for non-derived, it may not exist. Ex: Deleted skeleton).
		UScriptStruct* ParentStruct = ParentAnimClass->GetSparseClassDataStruct();
		UScriptStruct* SuperStruct = ParentStruct ? ParentStruct : FAnimBlueprintConstantData::StaticStruct();
		NewAnimBlueprintConstants->SetSuperStruct(SuperStruct);

		// Just link & assign sparse class data struct here, no additional members are added
		NewAnimBlueprintConstants->StaticLink(true);
		NewAnimBlueprintClass->SetSparseClassDataStruct(NewAnimBlueprintConstants);
		NewAnimBlueprintClass->GetOrCreateSparseClassData();
		NewAnimBlueprintClass->BuildConstantProperties();
	}
	else
	{
		UClass* ParentClass = NewAnimBlueprintClass->GetSuperClass();
		check(ParentClass);
		
	    // Create new sparse class data struct
		NewAnimBlueprintConstants = NewObject<UScriptStruct>(NewAnimBlueprintClass, UAnimBlueprintGeneratedClass::GetConstantsStructName(), RF_Public);

		// Inherit from archetype struct if there is any 
		UScriptStruct* ParentStruct = ParentClass->GetSparseClassDataStruct();
		UScriptStruct* SuperStruct = ParentStruct ? ParentStruct : FAnimBlueprintConstantData::StaticStruct();
		NewAnimBlueprintConstants->SetSuperStruct(SuperStruct);
	}

	if(OldSparseClassDataStruct)
	{
		FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(OldSparseClassDataStruct, NewAnimBlueprintConstants);
	}
}

void FAnimBlueprintCompilerContext::RecreateMutables()
{
	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();

	UScriptStruct* OldMutablesStruct = FindObject<UScriptStruct>(NewAnimBlueprintClass, *UAnimBlueprintGeneratedClass::GetMutablesStructName().ToString());
	
	// Set up our mutables struct
	NewAnimBlueprintMutables = NewObject<UScriptStruct>(NewAnimBlueprintClass, UAnimBlueprintGeneratedClass::GetMutablesStructName(), RF_Public);
	NewAnimBlueprintMutables->SetSuperStruct(FAnimBlueprintMutableData::StaticStruct());

	if(OldMutablesStruct)
	{
		FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(OldMutablesStruct, NewAnimBlueprintMutables);
	}
}

void FAnimBlueprintCompilerContext::SpawnNewClass(const FString& NewClassName)
{
	NewClass = FindObject<UAnimBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName);

	if (NewClass == NULL)
	{
		NewClass = NewObject<UAnimBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName, RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		FBlueprintCompileReinstancer::Create(NewClass);
	}
	
	FAnimBlueprintGeneratedClassCompiledData CompiledData(GetNewAnimBlueprintClass());
	FAnimBlueprintCompilationBracketContext CompilerContext(this);
	UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [this, &CompiledData, &CompilerContext](UAnimBlueprintExtension* InExtension)
    {
		InExtension->StartCompilingClass(GetNewAnimBlueprintClass(), CompilerContext, CompiledData);
	});
}

void FAnimBlueprintCompilerContext::OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& Context)
{
	if (Context.bIsSkeletonOnly)
	{
		return;
	}

	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();
	NewAnimBlueprintClass->OnPostLoadDefaults(NewAnimBlueprintClass->GetDefaultObject(false));
}

void FAnimBlueprintCompilerContext::OnNewClassSet(UBlueprintGeneratedClass* ClassToUse)
{
	NewClass = CastChecked<UAnimBlueprintGeneratedClass>(ClassToUse);
}

void FAnimBlueprintCompilerContext::CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOldCDO)
{
	UAnimBlueprintGeneratedClass* AnimBlueprintClassToClean = CastChecked<UAnimBlueprintGeneratedClass>(ClassToClean);

	UScriptStruct* CurrentSparseClassDataStruct = AnimBlueprintClassToClean->GetSparseClassDataStruct();
	if(CurrentSparseClassDataStruct && CurrentSparseClassDataStruct->GetOuter() == AnimBlueprintClassToClean)
	{
		// Only 'clear' (which renames the struct aside) if we own the sparse class data
		// We do this because this could be a parent classes struct and we dont want to be 
		// altering other classes during compilation
		AnimBlueprintClassToClean->ClearSparseClassDataStruct(Blueprint->bIsRegeneratingOnLoad);
	}
	else
	{
		AnimBlueprintClassToClean->SetSparseClassDataStruct(nullptr);
	}
	
	// Calling super will set this classes superclass, which will reset its sparse class data to the parent (archetype)
	Super::CleanAndSanitizeClass(ClassToClean, InOldCDO);

	// Clear reference to archetype sparse class data (set in the super call above), we will be regenerating it
	AnimBlueprintClassToClean->SetSparseClassDataStruct(nullptr);
	
	AnimBlueprintClassToClean->AnimBlueprintDebugData = FAnimBlueprintDebugData();

	// Reset the baked data
	//@TODO: Move this into PurgeClass
	AnimBlueprintClassToClean->BakedStateMachines.Empty();
	AnimBlueprintClassToClean->AnimNotifies.Empty();
	AnimBlueprintClassToClean->AnimBlueprintFunctions.Empty();
	AnimBlueprintClassToClean->OrderedSavedPoseIndicesMap.Empty();
	AnimBlueprintClassToClean->AnimNodeProperties.Empty();
	AnimBlueprintClassToClean->LinkedAnimGraphNodeProperties.Empty();
	AnimBlueprintClassToClean->LinkedAnimLayerNodeProperties.Empty();
	AnimBlueprintClassToClean->PreUpdateNodeProperties.Empty();
	AnimBlueprintClassToClean->DynamicResetNodeProperties.Empty();
	AnimBlueprintClassToClean->StateMachineNodeProperties.Empty();
	AnimBlueprintClassToClean->InitializationNodeProperties.Empty();
	AnimBlueprintClassToClean->GraphAssetPlayerInformation.Empty();
	AnimBlueprintClassToClean->GraphBlendOptions.Empty();
	AnimBlueprintClassToClean->AnimNodeData.Empty();
	AnimBlueprintClassToClean->NodeTypeMap.Empty();

	// Copy over runtime data from the blueprint to the class
	AnimBlueprintClassToClean->TargetSkeleton = AnimBlueprint->TargetSkeleton;

	UAnimBlueprint* RootAnimBP = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint);
	bIsDerivedAnimBlueprint = RootAnimBP != NULL;
	
	FAnimBlueprintGeneratedClassCompiledData CompiledData(AnimBlueprintClassToClean);
	FAnimBlueprintCompilationBracketContext CompilerContext(this);
	UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [this, &CompiledData, &CompilerContext, AnimBlueprintClassToClean](UAnimBlueprintExtension* InExtension)
	{
		InExtension->StartCompilingClass(AnimBlueprintClassToClean, CompilerContext, CompiledData);
	});
}

void FAnimBlueprintCompilerContext::FinishCompilingClass(UClass* Class)
{
	const UAnimBlueprint* PossibleRoot = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint);
	const UAnimBlueprint* Src = PossibleRoot ? PossibleRoot : AnimBlueprint;

	UAnimBlueprintGeneratedClass* AnimBlueprintGeneratedClass = CastChecked<UAnimBlueprintGeneratedClass>(Class);
	AnimBlueprintGeneratedClass->SyncGroupNames.Reset();
	AnimBlueprintGeneratedClass->SyncGroupNames.Reserve(Src->Groups.Num());
	for (const FAnimGroupInfo& GroupInfo : Src->Groups)
	{
		AnimBlueprintGeneratedClass->SyncGroupNames.Add(GroupInfo.Name);
	}

	// Add graph blend options to class if blend values were actually customized
	auto AddBlendOptions = [AnimBlueprintGeneratedClass](UEdGraph* Graph)
	{
		UAnimationGraph* AnimGraph = Cast<UAnimationGraph>(Graph);
		if (AnimGraph && (AnimGraph->BlendOptions.BlendInTime >= 0.0f || AnimGraph->BlendOptions.BlendOutTime >= 0.0f))
		{
			AnimBlueprintGeneratedClass->GraphBlendOptions.Add(AnimGraph->GetFName(), AnimGraph->BlendOptions);
		}
	};


	for (UEdGraph* Graph : Blueprint->FunctionGraphs)
	{
		AddBlendOptions(Graph);
	}

	for (FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
	{
		if (InterfaceDesc.Interface != nullptr && InterfaceDesc.Interface->IsChildOf<UAnimLayerInterface>())
		{
			for (UEdGraph* Graph : InterfaceDesc.Graphs)
			{
				AddBlendOptions(Graph);
			}
		}
	}

	FAnimBlueprintGeneratedClassCompiledData CompiledData(GetNewAnimBlueprintClass());
	FAnimBlueprintCompilationBracketContext CompilerContext(this);
	UAnimBlueprintExtension::ForEachExtension(AnimBlueprint, [this, &CompiledData, &CompilerContext](UAnimBlueprintExtension* InExtension)
	{
		InExtension->FinishCompilingClass(GetNewAnimBlueprintClass(), CompilerContext, CompiledData);
	});

	Super::FinishCompilingClass(Class);
}

void FAnimBlueprintCompilerContext::PostCompile()
{
	Super::PostCompile();

	for (UPoseWatch* PoseWatch : AnimBlueprint->PoseWatches)
	{
		AnimationEditorUtils::SetPoseWatch(PoseWatch, AnimBlueprint);
	}

	UAnimBlueprintGeneratedClass* AnimBlueprintGeneratedClass = CastChecked<UAnimBlueprintGeneratedClass>(NewClass);
	if(UAnimInstance* DefaultAnimInstance = Cast<UAnimInstance>(AnimBlueprintGeneratedClass->GetDefaultObject()))
	{
		// iterate all anim node and call PostCompile
		if(const USkeleton* CurrentSkeleton = AnimBlueprint->TargetSkeleton)
		{
			for (FStructProperty* Property : TFieldRange<FStructProperty>(AnimBlueprintGeneratedClass, EFieldIteratorFlags::IncludeSuper))
			{
				if (Property->Struct->IsChildOf(FAnimNode_Base::StaticStruct()))
				{
					FAnimNode_Base* AnimNode = Property->ContainerPtrToValuePtr<FAnimNode_Base>(DefaultAnimInstance);
					AnimNode->PostCompile(CurrentSkeleton);
				}
			}
		}
	}
}

void FAnimBlueprintCompilerContext::PostCompileDiagnostics()
{
	FKismetCompilerContext::PostCompileDiagnostics();

	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();
	
#if WITH_EDITORONLY_DATA // ANIMINST_PostCompileValidation
	// See if AnimInstance implements a PostCompileValidation Class. 
	// If so, instantiate it, and let it perform Validation of our newly compiled AnimBlueprint.
	if (const UAnimInstance* const DefaultAnimInstance = Cast<UAnimInstance>(NewAnimBlueprintClass->GetDefaultObject()))
	{
		if (DefaultAnimInstance->PostCompileValidationClassName.IsValid())
		{
			UClass* PostCompileValidationClass = LoadClass<UObject>(nullptr, *DefaultAnimInstance->PostCompileValidationClassName.ToString());
			if (PostCompileValidationClass)
			{
				UAnimBlueprintPostCompileValidation* PostCompileValidation = NewObject<UAnimBlueprintPostCompileValidation>(GetTransientPackage(), PostCompileValidationClass);
				if (PostCompileValidation)
				{
					FAnimBPCompileValidationParams PCV_Params(DefaultAnimInstance, NewAnimBlueprintClass, MessageLog, AllocatedNodePropertiesToNodes);
					PostCompileValidation->DoPostCompileValidation(PCV_Params);
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	if (!bIsDerivedAnimBlueprint)
	{
		bool bUsingCopyPoseFromMesh = false;

		// Run thru all nodes and make sure they like the final results
		for (auto NodeIt = AllocatedAnimNodeIndices.CreateConstIterator(); NodeIt; ++NodeIt)
		{
			if (UAnimGraphNode_Base* Node = NodeIt.Key())
			{
				Node->ValidateAnimNodePostCompile(MessageLog, NewAnimBlueprintClass, NodeIt.Value());
				bUsingCopyPoseFromMesh = bUsingCopyPoseFromMesh || Node->UsingCopyPoseFromMesh();
			}
		}

		// Update CDO
		if (UAnimInstance* const DefaultAnimInstance = Cast<UAnimInstance>(NewAnimBlueprintClass->GetDefaultObject()))
		{
			DefaultAnimInstance->bUsingCopyPoseFromMesh = bUsingCopyPoseFromMesh;
		}
	}
}

void FAnimBlueprintCompilerContext::CreateAnimGraphStubFunctions()
{
	TArray<UEdGraph*> NewGraphs;

	auto CreateStubForGraph = [this, &NewGraphs](UEdGraph* InGraph)
	{
		if(InGraph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()))
		{
			// Check to see if we are implementing an interface, and if so, use the signature from that graph instead
			// as we may not have yet been conformed to it (it happens later in compilation)
			UEdGraph* GraphToUseforSignature = InGraph;
			for(const FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
			{
				UClass* InterfaceClass = InterfaceDesc.Interface;
				if(InterfaceClass)
				{
					if(UAnimBlueprint* InterfaceAnimBlueprint = Cast<UAnimBlueprint>(InterfaceClass->ClassGeneratedBy))
					{
						TArray<UEdGraph*> AllGraphs;
						InterfaceAnimBlueprint->GetAllGraphs(AllGraphs);
						UEdGraph** FoundSourceGraph = AllGraphs.FindByPredicate([InGraph](UEdGraph* InGraphToCheck){ return InGraphToCheck->GetFName() == InGraph->GetFName(); });
						if(FoundSourceGraph)
						{
							GraphToUseforSignature = *FoundSourceGraph;
							break;
						}
					}
				}
			}

			// Find the root and linked input pose nodes
			TArray<UAnimGraphNode_Root*> Roots;
			GraphToUseforSignature->GetNodesOfClass(Roots);

			TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
			GraphToUseforSignature->GetNodesOfClass(LinkedInputPoseNodes);

			if(Roots.Num() > 0)
			{
				UAnimGraphNode_Root* RootNode = Roots[0];

				// Make sure there was only one root node
				for (int32 RootIndex = 1; RootIndex < Roots.Num(); ++RootIndex)
				{
					MessageLog.Error(
						*LOCTEXT("ExpectedOneRoot_Error", "Expected only one root node in graph @@, but found both @@ and @@").ToString(),
						InGraph,
						RootNode,
						Roots[RootIndex]
					);
				}

				// Verify no duplicate inputs
				for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode0 : LinkedInputPoseNodes)
				{
					for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode1 : LinkedInputPoseNodes)
					{
						if(LinkedInputPoseNode0 != LinkedInputPoseNode1)
						{
							if(LinkedInputPoseNode0->Node.Name == LinkedInputPoseNode1->Node.Name)
							{
								MessageLog.Error(
									*LOCTEXT("DuplicateInputNode_Error", "Found duplicate input node @@ in graph @@").ToString(),
									LinkedInputPoseNode1,
									InGraph
								);
							}
						}
					}
				}

				// Create a simple generated graph for our anim 'function'. Decorate it to avoid naming conflicts with the original graph.
				FName NewGraphName(*(GraphToUseforSignature->GetName() + ANIM_FUNC_DECORATOR));

				UEdGraph* StubGraph = NewObject<UEdGraph>(Blueprint, NewGraphName);
				NewGraphs.Add(StubGraph);
				StubGraph->Schema = UEdGraphSchema_K2::StaticClass();
				StubGraph->SetFlags(RF_Transient);

				// Add an entry node
				UK2Node_FunctionEntry* EntryNode = SpawnIntermediateNode<UK2Node_FunctionEntry>(RootNode, StubGraph);
				EntryNode->NodePosX = -200;
				EntryNode->CustomGeneratedFunctionName = GraphToUseforSignature->GetFName();	// Note that the function generated from this temporary graph is undecorated

				const bool bIsDefaultAnimGraph = GraphToUseforSignature->GetFName() == UEdGraphSchema_K2::GN_AnimGraph;
				EntryNode->MetaData.Category = ((RootNode->Node.GetGroup() == NAME_None) || (bIsDefaultAnimGraph && RootNode->Node.GetGroup() == FAnimNode_Root::DefaultSharedGroup)) 
					? FText::GetEmpty()
					: FText::FromName(RootNode->Node.GetGroup());

				// Add linked input poses as parameters
				for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode : LinkedInputPoseNodes)
				{
					// Add user defined pins for each linked input pose
					TSharedPtr<FUserPinInfo> PosePinInfo = MakeShared<FUserPinInfo>();
					PosePinInfo->PinType = UAnimationGraphSchema::MakeLocalSpacePosePin();
					PosePinInfo->PinName = LinkedInputPoseNode->Node.Name;
					PosePinInfo->DesiredPinDirection = EGPD_Output;
					EntryNode->UserDefinedPins.Add(PosePinInfo);

					// Add user defined pins for each linked input pose parameter
					for(UEdGraphPin* LinkedInputPoseNodePin : LinkedInputPoseNode->Pins)
					{
						if(!LinkedInputPoseNodePin->bOrphanedPin && LinkedInputPoseNodePin->Direction == EGPD_Output && !UAnimationGraphSchema::IsPosePin(LinkedInputPoseNodePin->PinType))
						{
							TSharedPtr<FUserPinInfo> ParameterPinInfo = MakeShared<FUserPinInfo>();
							ParameterPinInfo->PinType = LinkedInputPoseNodePin->PinType;
							ParameterPinInfo->PinName = LinkedInputPoseNodePin->PinName;
							ParameterPinInfo->DesiredPinDirection = EGPD_Output;
							EntryNode->UserDefinedPins.Add(ParameterPinInfo);
						}
					}
				}
				EntryNode->AllocateDefaultPins();

				UEdGraphPin* EntryExecPin = EntryNode->FindPinChecked(UEdGraphSchema_K2::PN_Then, EGPD_Output);

				UK2Node_FunctionResult* ResultNode = SpawnIntermediateNode<UK2Node_FunctionResult>(RootNode, StubGraph);
				ResultNode->NodePosX = 200;

				// Add root as the 'return value'
				TSharedPtr<FUserPinInfo> PinInfo = MakeShared<FUserPinInfo>();
				PinInfo->PinType = UAnimationGraphSchema::MakeLocalSpacePosePin();
				PinInfo->PinName = GraphToUseforSignature->GetFName();
				PinInfo->DesiredPinDirection = EGPD_Input;
				ResultNode->UserDefinedPins.Add(PinInfo);
	
				ResultNode->AllocateDefaultPins();

				UEdGraphPin* ResultExecPin = ResultNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute, EGPD_Input);

				// Link up entry to exit
				EntryExecPin->MakeLinkTo(ResultExecPin);
			}
			else
			{
				MessageLog.Error(*LOCTEXT("NoRootNodeFound_Error", "Could not find a root node for the graph @@").ToString(), InGraph);
			}
		}	
	};

	for(UEdGraph* Graph : Blueprint->FunctionGraphs)
	{
		CreateStubForGraph(Graph);
	}

	for(FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
	{
		for(UEdGraph* Graph : InterfaceDesc.Graphs)
		{
			CreateStubForGraph(Graph);
		}
	}

	Blueprint->FunctionGraphs.Append(NewGraphs);
	GeneratedStubGraphs.Append(NewGraphs);
}

void FAnimBlueprintCompilerContext::DestroyAnimGraphStubFunctions()
{
	Blueprint->FunctionGraphs.RemoveAll([this](UEdGraph* InGraph)
	{
		return GeneratedStubGraphs.Contains(InGraph);
	});

	GeneratedStubGraphs.Empty();
}

void FAnimBlueprintCompilerContext::PrecompileFunction(FKismetFunctionContext& Context, EInternalCompilerFlags InternalFlags)
{
	Super::PrecompileFunction(Context, InternalFlags);

	if(Context.Function)
	{
		auto CompareEntryPointName =
		[Function = Context.Function](UEdGraph* InGraph)
		{
			if(InGraph)
			{
				TArray<UK2Node_FunctionEntry*> EntryPoints;
				InGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryPoints);
				if(EntryPoints.Num() == 1 && EntryPoints[0])
				{
					return EntryPoints[0]->CustomGeneratedFunctionName == Function->GetFName(); 
				}
			}
			return true;
		};

		if(GeneratedStubGraphs.ContainsByPredicate(CompareEntryPointName))
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_BlueprintInternalUseOnly, TEXT("true"));
			Context.Function->SetMetaData(FBlueprintMetadata::MD_AnimBlueprintFunction, TEXT("true"));
		}
	}
}

void FAnimBlueprintCompilerContext::SetCalculatedMetaDataAndFlags(UFunction* Function, UK2Node_FunctionEntry* EntryNode, const UEdGraphSchema_K2* K2Schema)
{
	Super::SetCalculatedMetaDataAndFlags(Function, EntryNode, K2Schema);

	if(Function)
	{
		auto CompareEntryPointName =
		[Function](UEdGraph* InGraph)
		{
			if(InGraph)
			{
				TArray<UK2Node_FunctionEntry*> EntryPoints;
				InGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryPoints);
				if(EntryPoints.Num() == 1 && EntryPoints[0])
				{
					return EntryPoints[0]->CustomGeneratedFunctionName == Function->GetFName(); 
				}
			}
			return true;
		};

		// Match by name to generated graph's entry points
		if(GeneratedStubGraphs.ContainsByPredicate(CompareEntryPointName))
		{
			Function->SetMetaData(FBlueprintMetadata::MD_BlueprintInternalUseOnly, TEXT("true"));
			Function->SetMetaData(FBlueprintMetadata::MD_AnimBlueprintFunction, TEXT("true"));
		}
	}
}

void FAnimBlueprintCompilerContext::AddAttributesToNode(UAnimGraphNode_Base* InNode, TArrayView<const FName> InAttributes) const
{
	if(UAnimGraphNode_Base* OriginalNode = CastChecked<UAnimGraphNode_Base>(MessageLog.FindSourceObject(InNode)))
	{
		TArray<FName>& AttributeSet = GetNewAnimBlueprintClass()->GetAnimBlueprintDebugData().NodeAttributes.FindOrAdd(OriginalNode);
		AttributeSet.Reserve(AttributeSet.Num() + InAttributes.Num());

		for(const FName& Attribute : InAttributes)
		{
			AttributeSet.AddUnique(Attribute);
		}
	}
}

TArrayView<const FName> FAnimBlueprintCompilerContext::GetAttributesFromNode(UAnimGraphNode_Base* InNode) const
{
	if(UAnimGraphNode_Base* OriginalNode = CastChecked<UAnimGraphNode_Base>(MessageLog.FindSourceObject(InNode)))
	{
		if(const TArray<FName>* AttributeSetPtr = GetNewAnimBlueprintClass()->GetAnimBlueprintDebugData().NodeAttributes.Find(OriginalNode))
		{
			return MakeArrayView(*AttributeSetPtr);
		}
	}

	return TArrayView<const FName>();
}

FProperty* FAnimBlueprintCompilerContext::CreateUniqueVariable(UObject* InForObject, const FEdGraphPinType& Type)
{
	const FString VariableName = ClassScopeNetNameMap.MakeValidName(InForObject);
	FProperty* Variable = CreateVariable(*VariableName, Type);
	Variable->SetMetaData(FBlueprintMetadata::MD_Private, TEXT("true"));
	return Variable;
}

FProperty* FAnimBlueprintCompilerContext::CreateStructVariable(UScriptStruct* InStruct, const FName VarName, const FEdGraphPinType& VarType)
{
	FProperty* NewProperty = FKismetCompilerUtilities::CreatePropertyOnScope(InStruct, VarName, VarType, nullptr, CPF_None, Schema, MessageLog);
	if (NewProperty != nullptr)
	{
		FKismetCompilerUtilities::LinkAddedProperty(InStruct, NewProperty);
	}
	else
	{
		MessageLog.Error(
			*FText::Format(
				LOCTEXT("VariableInvalidType_ErrorFmt", "The variable {0} declared in @@ has an invalid type {1}"),
				FText::FromName(VarName),
				UEdGraphSchema_K2::TypeToText(VarType)
			).ToString(),
			Blueprint
		);
	}

	return NewProperty;
}

void FAnimBlueprintCompilerContext::AddFoldedPropertyRecord(UAnimGraphNode_Base* InAnimGraphNode, FStructProperty* InAnimNodeProperty, FProperty* InProperty, bool bInExposedOnPin, bool bInPinConnected, bool bInAlwaysDynamic)
{
	const bool bConstant = !bInAlwaysDynamic && (!bInExposedOnPin || (bInExposedOnPin && !bInPinConnected));

	if(!InProperty->HasAnyPropertyFlags(CPF_EditorOnly))
	{
		MessageLog.Warning(*FString::Printf(TEXT("Property %s on @@ is foldable, but not editor only"), *InProperty->GetName()), InAnimGraphNode);
	}

	// Create record and add it our lookup map
	TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord> Record = MakeShared<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>(InAnimGraphNode, InAnimNodeProperty, InProperty, bConstant);
	TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>>& Array = NodeToFoldedPropertyRecordMap.FindOrAdd(InAnimGraphNode);
	Array.Add(Record);

	// Record it in the appropriate data area
	if(bConstant)
	{
		ConstantPropertyRecords.Add(Record);
	}
	else
	{
		MutablePropertyRecords.Add(Record);
	}
}

void FAnimBlueprintCompilerContext::ProcessFoldedPropertyRecords()
{
	UAnimBlueprintGeneratedClass* NewAnimBlueprintClass = GetNewAnimBlueprintClass();
	
	if(ConstantPropertyRecords.Num() > 0)
	{
		// Set constants struct as sparse class data
		check(NewAnimBlueprintConstants);

		auto GetRecordValue = [](const TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>& InRecord)
		{
			FStructProperty* AnimGraphNodeProperty = InRecord->AnimGraphNode->GetFNodeProperty();

			check(AnimGraphNodeProperty->GetOwner<UClass>() && InRecord->AnimGraphNode->GetClass()->IsChildOf(AnimGraphNodeProperty->GetOwner<UClass>()));
			const void* Node = AnimGraphNodeProperty->ContainerPtrToValuePtr<void>(InRecord->AnimGraphNode);
			check(InRecord->Property->GetOwner<UStruct>() && AnimGraphNodeProperty->Struct->IsChildOf(InRecord->Property->GetOwner<UStruct>()));
			return InRecord->Property->ContainerPtrToValuePtr<void>(Node);
		};

		// Reduce any constant properties before patching
		for(int32 RecordIndex0 = 0; RecordIndex0 < ConstantPropertyRecords.Num(); ++RecordIndex0)
		{
			const TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>& Record0 = ConstantPropertyRecords[RecordIndex0];
			if(Record0->FoldIndex == INDEX_NONE)
			{
				const void* Value0 = GetRecordValue(Record0);

				for(int32 RecordIndex1 = RecordIndex0 + 1; RecordIndex1 < ConstantPropertyRecords.Num(); ++RecordIndex1)
				{
					const TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>& Record1 = ConstantPropertyRecords[RecordIndex1];
					if(Record1->FoldIndex == INDEX_NONE)
					{
						if(Record1->Property->SameType(Record0->Property))
						{
							const void* Value1 = GetRecordValue(Record1);

							// same type, now test for equality
							if(Record0->Property->Identical(Value0, Value1))
							{
								// Values are the same - fold
								Record1->FoldIndex = RecordIndex0;
							}
						}
					}
				}
			}
		}
	}

	// Builds a 'data area', returns the total number of properties that were inserted into that area's struct
	auto BuildDataArea = [this, NewAnimBlueprintClass](TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>>& InRecords, UScriptStruct* InStruct)
	{
		int32 PropertyIndex = 0;

		if(InStruct)
		{
			const UAnimationGraphSchema* AnimationGraphSchema = GetDefault<UAnimationGraphSchema>();
			FAnimBlueprintDebugData& AnimBlueprintDebugData = NewAnimBlueprintClass->GetAnimBlueprintDebugData();
			
			for(const TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>& Record : InRecords)
			{
				// Skip folded records
				if(Record->FoldIndex == INDEX_NONE)
				{
					FEdGraphPinType VariableType;
					if(AnimationGraphSchema->ConvertPropertyToPinType(Record->Property, VariableType))
					{
						// Patch into sparse class data
						TStringBuilder<64> PropertyNameStringBuilder;

						// Add prefix for internal use
						PropertyNameStringBuilder.Append(TEXT("__"));

						// If we are an object property, we should be named according to the underlying class type to avoid
						// warnings if we tagged-property-serialize a colliding name with an updated struct layout
						if(const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Record->Property))
						{
							PropertyNameStringBuilder.Append(ObjectProperty->PropertyClass->GetName());
						}
						else
						{
							PropertyNameStringBuilder.Append(Record->Property->GetClass()->GetName());
						}

						const FName PropertyName = FName(PropertyNameStringBuilder.ToString(), InRecords.Num() - 1 - PropertyIndex);
						Record->GeneratedProperty = CreateStructVariable(InStruct, PropertyName, VariableType);
						if (Record->GeneratedProperty == nullptr)
						{
							MessageLog.Error(*FString::Printf(TEXT("Property %s on node @@ could not be patched into data area."), *Record->Property->GetName()), Record->AnimGraphNode);
						}
						else
						{
							// Propagate some relevant property flags
							Record->GeneratedProperty->SetPropertyFlags(Record->Property->GetPropertyFlags() & CPF_EditFixedSize);

							// Properties need to be BP visible to allow them to be set by the generated exec chains in CreateEvaluationHandlerForNode
							Record->GeneratedProperty->SetPropertyFlags(CPF_BlueprintVisible);
							Record->PropertyIndex = PropertyIndex++;

							// Get the source graph node of the given one as the debug utilities and other users of the graph pin to folded property map
							// are working with the source graph nodes.
							UEdGraphNode* GraphNode = CastChecked<UEdGraphNode>(MessageLog.FindSourceObject(Record->AnimGraphNode));
							if (GraphNode)
							{
								UEdGraphPin* GraphPin = GraphNode->FindPin(Record->Property->GetName());
								if (GraphPin)
								{
									// Link the graph pin with the generated (folded) property.
									AnimBlueprintDebugData.GraphPinToFoldedPropertyMap.Add(GraphPin, Record->GeneratedProperty);

									// Also add all linked pins directly here. This is needed for pins on nodes that are not processed by the compiler as e.g. get variable nodes.
									for (UEdGraphPin* LinkedToPin : GraphPin->LinkedTo)
									{									
										AnimBlueprintDebugData.GraphPinToFoldedPropertyMap.Add(LinkedToPin, Record->GeneratedProperty);
									}
								}
							}
						}
					}
					else
					{
						MessageLog.Error(*FString::Printf(TEXT("Property %s on node @@ could not be patched into data area."), *Record->Property->GetName()), Record->AnimGraphNode);
					}
				}
			}

			if(InRecords.Num() > 0)
			{
				InStruct->StaticLink(true);
			}
		}

		return PropertyIndex;
	};

	// Next, patch into the relevant data areas
	const int32 NumConstantProperties = BuildDataArea(ConstantPropertyRecords, NewAnimBlueprintConstants);

	// Set constants as sparse class data
	if(ConstantPropertyRecords.Num() > 0)
	{
		NewAnimBlueprintClass->SetSparseClassDataStruct(NewAnimBlueprintConstants);
	}

	const int32 NumMutableProperties = BuildDataArea(MutablePropertyRecords, NewAnimBlueprintMutables);

	// Create the property for our mutables, if we have any
	if(MutablePropertyRecords.Num() > 0)
	{
		const FName MutablesStructName("__AnimBlueprintMutables");
		FEdGraphPinType MutablesPinType;
		MutablesPinType.PinCategory = UAnimationGraphSchema::PC_Struct;
		MutablesPinType.PinSubCategoryObject = MakeWeakObjectPtr(NewAnimBlueprintMutables);
		NewMutablesProperty = CastFieldChecked<FStructProperty>(CreateVariable(MutablesStructName, MutablesPinType));
		NewMutablesProperty->SetMetaData(TEXT("BlueprintCompilerGeneratedDefaults"), TEXT("true"));
	}
	else
	{
		// No more references to this struct, as it was not used to generate the property above, so mark it as garbage to ensure it doesnt get saved into the package
		NewAnimBlueprintMutables->MarkAsGarbage();
	}

	// Set up per-node mappings
	NewAnimBlueprintClass->AnimNodeData.Empty();
	NewAnimBlueprintClass->AnimNodeData.SetNum(AllocatedAnimNodeIndices.Num());
	NewAnimBlueprintClass->NodeTypeMap.Empty();
	
	// First index & setup the node data
	for(const TPair<int32, FProperty*>& IndexPropertyPair : AllocatedPropertiesByIndex)
	{
		int32 NodeIndex = AllocatedPropertiesByIndex.Num() - 1 - IndexPropertyPair.Key;
		FStructProperty* NodeStructProperty = CastFieldChecked<FStructProperty>(IndexPropertyPair.Value);
		const UScriptStruct* Struct = NodeStructProperty->Struct;
		const FAnimNodeStructData AnimNodeStructData = NewAnimBlueprintClass->NodeTypeMap.Add(Struct, FAnimNodeStructData(Struct));
		const int32 NumProperties = AnimNodeStructData.GetNumProperties();

		// Add any super-structs as values can be accessed via base classes
		const UScriptStruct* SuperStruct = Cast<UScriptStruct>(Struct->GetSuperStruct());
		while(SuperStruct)
		{
			NewAnimBlueprintClass->NodeTypeMap.Add(SuperStruct, FAnimNodeStructData(SuperStruct));
			SuperStruct = Cast<UScriptStruct>(SuperStruct->GetSuperStruct());
		}

		FAnimNodeData& NodeData = NewAnimBlueprintClass->AnimNodeData[NodeIndex];
		NodeData.AnimClassInterface = NewAnimBlueprintClass;
		NodeData.NodeIndex = NodeIndex;
		check(NumProperties >= 0);
		NodeData.Entries.SetNum(NumProperties);
		for(uint32& Entry : NodeData.Entries)
		{
			Entry = ANIM_NODE_DATA_INVALID_ENTRY;
		}
	}

	auto BuildAnimNodeData = [this, &NewAnimBlueprintClass](const TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>>& InRecords, int32 InTotalPropertyCount)
	{
		const int32 NumNodes = AllocatedAnimNodeIndices.Num();

		for(const TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>& Record : InRecords)
		{
			int32 NodeIndex = NumNodes - 1 - AllocatedAnimNodeIndices.FindChecked(Record->AnimGraphNode);
			
			FAnimNodeData& NodeData = NewAnimBlueprintClass->AnimNodeData[NodeIndex];
			
			int32 PropertyIndex = Record->PropertyIndex;
			if(Record->FoldIndex != INDEX_NONE)
			{
				PropertyIndex = InRecords[Record->FoldIndex]->PropertyIndex;
			}
			check(PropertyIndex >= 0 && PropertyIndex < InTotalPropertyCount);
			PropertyIndex = InTotalPropertyCount - 1 - PropertyIndex;

			uint32 PropertyEntry = PropertyIndex;
			if(!Record->bIsOnClass)
			{
				PropertyEntry |= ANIM_NODE_DATA_INSTANCE_DATA_FLAG;
			}

			const FAnimNodeStructData& AnimNodeStructData = NewAnimBlueprintClass->NodeTypeMap.FindChecked(Record->AnimNodeProperty->Struct);
			const int32 EntryIndex = AnimNodeStructData.GetPropertyIndex(Record->Property->GetFName());
			NodeData.Entries[EntryIndex] = PropertyEntry;
		}
	};

	BuildAnimNodeData(ConstantPropertyRecords, NumConstantProperties);
	BuildAnimNodeData(MutablePropertyRecords, NumMutableProperties);

	const int32 NumNodes = AllocatedAnimNodeIndices.Num();
	
	// Patch anim node data flags
	for(const TPair<UAnimGraphNode_Base*, int32>& GraphNodeIndexPair : AllocatedAnimNodeIndices)
	{
		UAnimGraphNode_Base* AnimGraphNode = GraphNodeIndexPair.Key;
		const int32 NodeIndex = NumNodes - 1 - GraphNodeIndexPair.Value;
		FAnimNodeData& NodeData = NewAnimBlueprintClass->AnimNodeData[NodeIndex];
		UClass* ClassToUse = AnimGraphNode->GetBlueprintClassFromNode();
		if(AnimGraphNode->InitialUpdateFunction.ResolveMember<UFunction>(ClassToUse))
		{
			NodeData.SetNodeFlags(EAnimNodeDataFlags::HasInitialUpdateFunction);
		}
		if(AnimGraphNode->UpdateFunction.ResolveMember<UFunction>(ClassToUse))
		{
			NodeData.SetNodeFlags(EAnimNodeDataFlags::HasUpdateFunction);
		}
		if(AnimGraphNode->BecomeRelevantFunction.ResolveMember<UFunction>(ClassToUse))
		{
			NodeData.SetNodeFlags(EAnimNodeDataFlags::HasBecomeRelevantFunction);
		}
	}
}

bool FAnimBlueprintCompilerContext::IsAnimGraphNodeFolded(UAnimGraphNode_Base* InNode) const
{
	return NodeToFoldedPropertyRecordMap.Find(InNode) != nullptr;
}

const IAnimBlueprintCompilationContext::FFoldedPropertyRecord* FAnimBlueprintCompilerContext::GetFoldedPropertyRecord(UAnimGraphNode_Base* InNode, FName InPropertyName) const
{
	if(const TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>>* FoundRecordsPtr = NodeToFoldedPropertyRecordMap.Find(InNode))
	{
		for(const TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>& Record : *FoundRecordsPtr)
		{
			if(Record->Property->GetFName() == InPropertyName)
			{
				return &Record.Get();
			}
		}
	}

	return nullptr;
}

void FAnimBlueprintCompilerContext::PreCompileUpdateBlueprintOnLoad(UBlueprint* BP)
{
	if (UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(BP))
	{
		if (AnimBP->GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::AnimBlueprintSubgraphFix)
		{
			AnimationEditorUtils::RegenerateSubGraphArrays(AnimBP);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
