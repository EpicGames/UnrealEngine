// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetCompiler.h"
#include "Animation/AnimNodeBase.h"
#include "AnimGraphNode_Base.h"
#include "KismetCompilerModule.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "Containers/ArrayView.h"
#include "IAnimBlueprintCompilationContext.h"

class UAnimationGraphSchema;
class UAnimGraphNode_SaveCachedPose;
class UAnimGraphNode_StateMachineBase;
class UAnimGraphNode_StateResult;
class UAnimGraphNode_CustomProperty;

class UAnimGraphNode_UseCachedPose;
class UAnimStateTransitionNode;
class UK2Node_CallFunction;

//
// Forward declarations.
//
class UAnimGraphNode_SaveCachedPose;
class UAnimGraphNode_UseCachedPose;
class UAnimGraphNode_LinkedInputPose;
class UAnimGraphNode_LinkedAnimGraphBase;
class UAnimGraphNode_LinkedAnimGraph;
class UAnimGraphNode_Root;

class FStructProperty;
class UBlueprintGeneratedClass;
struct FPoseLinkMappingRecord;
struct FAnimGraphNodePropertyBinding;
class FAnimBlueprintCompilerContext;

//////////////////////////////////////////////////////////////////////////
// FAnimBlueprintCompilerContext
class FAnimBlueprintCompilerContext : public FKismetCompilerContext
{
	friend class FAnimBlueprintCompilerCreationContext;
	friend class FAnimBlueprintCompilationContext;
	friend class FAnimBlueprintVariableCreationContext;
	friend class FAnimBlueprintCompilationBracketContext;
	friend class FAnimBlueprintPostExpansionStepContext;
	friend class FAnimBlueprintCopyTermDefaultsContext;
	friend class UK2Node_AnimNodeReference;

protected:
	typedef FKismetCompilerContext Super;
public:
	FAnimBlueprintCompilerContext(UAnimBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions);
	virtual ~FAnimBlueprintCompilerContext();

protected:
	// Implementation of FKismetCompilerContext interface
	virtual void CreateClassVariablesFromBlueprint() override;
	virtual UEdGraphSchema_K2* CreateSchema() override;
	virtual void MergeUbergraphPagesIn(UEdGraph* Ubergraph) override;
	virtual void ProcessOneFunctionGraph(UEdGraph* SourceGraph, bool bInternalFunction = false) override;
	virtual void SpawnNewClass(const FString& NewClassName) override;
	virtual void OnNewClassSet(UBlueprintGeneratedClass* ClassToUse) override;
	virtual void OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& Context) override;
	virtual void CopyTermDefaultsToDefaultObject(UObject* DefaultObject) override;
	virtual void PostCompile() override;
	virtual void PostCompileDiagnostics() override;
	virtual void EnsureProperGeneratedClass(UClass*& TargetClass) override;
	virtual void CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOldCDO) override;
	virtual void FinishCompilingClass(UClass* Class) override;
	virtual void PrecompileFunction(FKismetFunctionContext& Context, EInternalCompilerFlags InternalFlags) override;
	virtual void SetCalculatedMetaDataAndFlags(UFunction* Function, UK2Node_FunctionEntry* EntryNode, const UEdGraphSchema_K2* Schema ) override;
	virtual bool ShouldForceKeepNode(const UEdGraphNode* Node) const override;
	virtual void PostExpansionStep(const UEdGraph* Graph) override;
	virtual void PreCompileUpdateBlueprintOnLoad(UBlueprint* BP) override;
	// End of FKismetCompilerContext interface

protected:
	typedef TArray<UEdGraphPin*> UEdGraphPinArray;

protected:
	UScriptStruct* NewAnimBlueprintConstants;
	UScriptStruct* NewAnimBlueprintMutables;
	FStructProperty* NewMutablesProperty;
	UAnimBlueprint* AnimBlueprint;

	// Old sparse class data stored to patchup linker when doing a full compile
	UScriptStruct* OldSparseClassDataStruct;
	
	UAnimationGraphSchema* AnimSchema;

	// Map of allocated v3 nodes that are members of the class
	TMap<class UAnimGraphNode_Base*, FProperty*> AllocatedAnimNodes;
	TMap<class UAnimGraphNode_Base*, FProperty*> AllocatedAnimNodeHandlers;
	TMap<FProperty*, class UAnimGraphNode_Base*> AllocatedNodePropertiesToNodes;
	TMap<FProperty*, class UAnimGraphNode_Base*> AllocatedNodeConstantPropertiesToNodes;
	TMap<int32, FProperty*> AllocatedPropertiesByIndex;

	// Map of true source objects (user edited ones) to the cloned ones that are actually compiled
	TMap<class UAnimGraphNode_Base*, UAnimGraphNode_Base*> SourceNodeToProcessedNodeMap;

	// Index of the nodes (must match up with the runtime discovery process of nodes, which runs thru the property chain)
	int32 AllocateNodeIndexCounter;
	TMap<class UAnimGraphNode_Base*, int32> AllocatedAnimNodeIndices;

	// Map from pose link LinkID address
	//@TODO: Bad structure for a list of these
	TArray<FPoseLinkMappingRecord> ValidPoseLinkList;

	// Stub graphs we generated for animation graph functions
	TArray<UEdGraph*> GeneratedStubGraphs;

	// True if any parent class is also generated from an animation blueprint
	bool bIsDerivedAnimBlueprint;

	// Graph schema classes that this compiler is aware of - they will skip default function processing
	TArray<TSubclassOf<UEdGraphSchema>> KnownGraphSchemas;

	// Expose compile options to handlers
	using FKismetCompilerContext::CompileOptions;

	// Records of folded properties gleaned from nodes
	TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>> ConstantPropertyRecords;
	TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>> MutablePropertyRecords;

	// Allows lookups to see if a node participates in constant folding
	TMap<UAnimGraphNode_Base*, TArray<TSharedRef<IAnimBlueprintCompilationContext::FFoldedPropertyRecord>>> NodeToFoldedPropertyRecordMap;

	// Maps of extension <-> generated property on the instance
	TMap<UAnimBlueprintExtension*, FStructProperty*> ExtensionToInstancePropertyMap;
	TMap<FStructProperty*, UAnimBlueprintExtension*> InstancePropertyToExtensionMap;

	// Maps of extension <-> generated property on the class
	TMap<UAnimBlueprintExtension*, FStructProperty*> ExtensionToClassPropertyMap;
	TMap<FStructProperty*, UAnimBlueprintExtension*> ClassPropertyToExtensionMap;

private:
	// Get the generated class as an anim blueprint generated class
	UAnimBlueprintGeneratedClass* GetNewAnimBlueprintClass() const { return CastChecked<UAnimBlueprintGeneratedClass>(NewClass); };
	
	// Run a function on the passed-in graph and each subgraph of it
	void ForAllSubGraphs(UEdGraph* InGraph, TFunctionRef<void(UEdGraph*)> InPerGraphFunction);

	// Prunes any nodes that aren't reachable via a pose link
	void PruneIsolatedAnimationNodes(const TArray<UAnimGraphNode_Base*>& RootSet, TArray<UAnimGraphNode_Base*>& GraphNodes);

	// Compiles one animation node
	void ProcessAnimationNode(UAnimGraphNode_Base* VisualAnimNode);

	// Called during ProcessAnimationNode - gather property folding records for the node
	void GatherFoldRecordsForAnimationNode(const UScriptStruct* InNodeType, FStructProperty* InNodeProperty, UAnimGraphNode_Base* InVisualAnimNode);

	// Compiles an entire animation graph
	void ProcessAllAnimationNodes();

	// Processes all the supplied anim nodes
	void ProcessAnimationNodes(TArray<UAnimGraphNode_Base*>& AnimNodeList);

	// Process all the requested extensions
	void ProcessExtensions();
	
	// Gets all anim graph nodes that are piped into the provided node (traverses input pins)
	void GetLinkedAnimNodes(UAnimGraphNode_Base* InGraphNode, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const;
	void GetLinkedAnimNodes_TraversePin(UEdGraphPin* InPin, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const;
	void GetLinkedAnimNodes_ProcessAnimNode(UAnimGraphNode_Base* AnimNode, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const;

	// Returns the allocation index of the specified node, processing it if it was pending
	int32 GetAllocationIndexOfNode(UAnimGraphNode_Base* VisualAnimNode);

	// Create transient stub functions for each anim graph we are compiling
	void CreateAnimGraphStubFunctions();

	// Clean up transient stub functions
	void DestroyAnimGraphStubFunctions();

	// Expands split pins for a graph
	void ExpandSplitPins(UEdGraph* InGraph);

	// Add the specified compiled-in attribute uniquely to the specified node
	void AddAttributesToNode(UAnimGraphNode_Base* InNode, TArrayView<const FName> InAttributes) const;

	// Get the current compiled-in attributes uniquely assigned to the specified node
	TArrayView<const FName> GetAttributesFromNode(UAnimGraphNode_Base* InNode) const;

	// Called at the start of compilation to (re-) create the mutable struct
	void RecreateMutables();
	
	// (Re-)creates sparse class data structure. Called at the start of compilation to (re-) create the internal sparse 
	// class data. For derived anim BPs this is called just-in-time before CDO copy. This is to ensure that the sparse 
	// class data is always updated, as in some cases as the layout does not change a full compilation of a (data-only) 
	// derived anim BP may be skipped.  
	void RecreateSparseClassData();
	
	// Create a uniquely named variable corresponding to an object in the current class
	FProperty* CreateUniqueVariable(UObject* InForObject, const FEdGraphPinType& Type);

	/** Creates a variable on the specified struct */
	FProperty* CreateStructVariable(UScriptStruct* InStruct, const FName VarName, const FEdGraphPinType& VarType);

	/** Adds a record for a potentially-folded anim node property */
	void AddFoldedPropertyRecord(UAnimGraphNode_Base* InAnimGraphNode, FStructProperty* InAnimNodeProperty, FProperty* InProperty, bool bInExposedOnPin, bool bInPinConnected, bool bInAlwaysDynamic);

	/** Process any anim node properties that are 'foldable' */
	void ProcessFoldedPropertyRecords();

	// Check whether an anim node participates in constant folding
	bool IsAnimGraphNodeFolded(UAnimGraphNode_Base* InNode) const;

	// Copy the AnimNodeData array etc. from root anim BP class to a derived anim BP 
	void CopyAnimNodeDataFromRoot() const;
	
	// Get the folded property record, if any, for the supplied node & named property
	const IAnimBlueprintCompilationContext::FFoldedPropertyRecord* GetFoldedPropertyRecord(UAnimGraphNode_Base* InNode, FName InPropertyName) const;
};

