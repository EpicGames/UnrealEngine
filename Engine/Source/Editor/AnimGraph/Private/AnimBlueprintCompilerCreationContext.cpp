// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintCompilerCreationContext.h"
#include "AnimBlueprintCompiler.h"
#include "EdGraph/EdGraphSchema.h"

void FAnimBlueprintCompilerCreationContext::RegisterKnownGraphSchema(TSubclassOf<UEdGraphSchema> InGraphSchemaClass)
{
	CompilerContext->KnownGraphSchemas.AddUnique(InGraphSchemaClass);
}

