// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "Templates/SubclassOf.h"

class FAnimBlueprintCompilerContext;

class FAnimBlueprintCompilerCreationContext : public IAnimBlueprintCompilerCreationContext
{
private:
	friend class FAnimBlueprintCompilerContext;
	
	FAnimBlueprintCompilerCreationContext(FAnimBlueprintCompilerContext* InCompilerContext)
		: CompilerContext(InCompilerContext)
	{}

	// IAnimBlueprintCompilerCreationContext interface
	virtual void RegisterKnownGraphSchema(TSubclassOf<UEdGraphSchema> InGraphSchemaClass) override;

private:
	FAnimBlueprintCompilerContext* CompilerContext;
};