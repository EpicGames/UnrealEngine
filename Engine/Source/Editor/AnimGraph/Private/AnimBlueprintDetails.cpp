// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintDetails.h"

#include "Animation/AnimBlueprint.h"
#include "Containers/Array.h"
#include "DetailLayoutBuilder.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

#define LOCTEXT_NAMESPACE "AnimBlueprintDetails"

void FAnimBlueprintDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	UAnimBlueprint* Asset = CastChecked<UAnimBlueprint>(Objects[0].Get());

	if(Asset->bIsTemplate)
	{
		TSharedRef<IPropertyHandle> TargetSkeletonHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimBlueprint, TargetSkeleton));
		TargetSkeletonHandle->MarkHiddenByCustomization();
	}
}

#undef LOCTEXT_NAMESPACE