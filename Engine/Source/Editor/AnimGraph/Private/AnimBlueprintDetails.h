// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;

class FAnimBlueprintDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FAnimBlueprintDetails());
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};
