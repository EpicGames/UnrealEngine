// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension.h"
#include "Animation/AnimSubsystem.h"
#include "Animation/AnimSubsystemInstance.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "Animation/AnimBlueprint.h"
#include "Templates/SubclassOf.h"
#include "AnimBlueprintExtension_Base.h"
#include "AnimBlueprintExtension_Attributes.h"
#include "AnimBlueprintExtension_PropertyAccess.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimBlueprintExtension_NodeRelevancy.h"
#include "AnimBlueprintExtension_SharedLinkedAnimLayers.h"
#include "AnimBlueprintExtension_Tag.h"
#include "AnimGraphNodeBinding.h"

// Set used to refresh extensions. Checks that an extension has a reference from an anim node for each refresh.

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension)
static TSet<TSubclassOf<UAnimBlueprintExtension>> RefreshSet;
static bool GIsRefreshingExtensions = false;

UAnimBlueprintExtension* UAnimBlueprintExtension::RequestExtension(UAnimBlueprint* InAnimBlueprint, TSubclassOf<UAnimBlueprintExtension> InExtensionType)
{
	if(GIsRefreshingExtensions)
	{
		RefreshSet.Add(InExtensionType);
	}
	
	// Look for an existing extension
	if(UAnimBlueprintExtension* ExistingExtension = GetExtension(InAnimBlueprint, InExtensionType))
	{
		return ExistingExtension;
	}

	// Do not use RequestExtension when a blueprint is being compiled. Extensions should be consistent throughout all compilation stages.
	ensure(!InAnimBlueprint->bBeingCompiled);

	// Not found, create one
	UAnimBlueprintExtension* NewExtension = NewObject<UAnimBlueprintExtension>(InAnimBlueprint, InExtensionType.Get());
	InAnimBlueprint->AddExtension(NewExtension);
	return NewExtension;
}

UAnimBlueprintExtension* UAnimBlueprintExtension::GetExtension(UAnimBlueprint* InAnimBlueprint, TSubclassOf<UAnimBlueprintExtension> InExtensionType)
{
	// Look for an existing extension
	for(TObjectPtr<UBlueprintExtension> Extension : InAnimBlueprint->GetExtensions())
	{
		if(Extension && Extension->GetClass() == InExtensionType)
		{
			return CastChecked<UAnimBlueprintExtension>(Extension);
		}
	}

	return nullptr;
}

TArray<UAnimBlueprintExtension*> UAnimBlueprintExtension::GetExtensions(UAnimBlueprint* InAnimBlueprint)
{
	TArray<UAnimBlueprintExtension*> Extensions;
	
	for(TObjectPtr<UBlueprintExtension> Extension : InAnimBlueprint->GetExtensions())
	{
		if(Extension && Extension->GetClass()->IsChildOf(UAnimBlueprintExtension::StaticClass()))
		{
			Extensions.Add(CastChecked<UAnimBlueprintExtension>(Extension));
		}
	}
	
	return Extensions;
}

void UAnimBlueprintExtension::RequestExtensionsForNode(UAnimGraphNode_Base* InAnimGraphNode)
{
	if(UAnimBlueprint* AnimBlueprint = InAnimGraphNode->GetAnimBlueprint())
	{
		TArray<TSubclassOf<UAnimBlueprintExtension>> ExtensionClasses =
		{
			UAnimBlueprintExtension_Base::StaticClass(),
            UAnimBlueprintExtension_Attributes::StaticClass(),
            UAnimBlueprintExtension_PropertyAccess::StaticClass()
        };

		if (AnimBlueprint->bEnableLinkedAnimLayerInstanceSharing)
		{
			ExtensionClasses.Add(UAnimBlueprintExtension_SharedLinkedAnimLayers::StaticClass());
		}

		// As this can be called when we have not regenerated the skeleton class, we need to be less conservative
		// and request extensions if it *looks* like we have a valid reference, rather than a verified concrete ref.
		// Later validation code will do the actual validation for us anyways and fail compilation if the reference is
		// actually invalid.
		if(UAnimGraphNode_Base::IsPotentiallyBoundFunction(InAnimGraphNode->InitialUpdateFunction) ||
			UAnimGraphNode_Base::IsPotentiallyBoundFunction(InAnimGraphNode->BecomeRelevantFunction))
		{
			ExtensionClasses.Add(UAnimBlueprintExtension_NodeRelevancy::StaticClass());
		}

		if(InAnimGraphNode->Tag != NAME_None)
		{
			ExtensionClasses.Add(UAnimBlueprintExtension_Tag::StaticClass());
		}
		
		InAnimGraphNode->GetRequiredExtensions(ExtensionClasses);

		if (const UAnimGraphNodeBinding* Binding = InAnimGraphNode->GetBinding())
		{
			Binding->GetRequiredExtensions(ExtensionClasses);
		}
	
		for(const TSubclassOf<UAnimBlueprintExtension>& ExtensionClass : ExtensionClasses)
		{	
			// Request any subsystem that we need to compile
			RequestExtension(AnimBlueprint, ExtensionClass);
		}
	}
}

void UAnimBlueprintExtension::RefreshExtensions(UAnimBlueprint* InAnimBlueprint)
{
	GIsRefreshingExtensions = true;
	RefreshSet.Empty();
	
	TArray<UAnimGraphNode_Base*> AllNodes;
	FBlueprintEditorUtils::GetAllNodesOfClass<UAnimGraphNode_Base>(InAnimBlueprint, AllNodes);
	for(UAnimGraphNode_Base* Node : AllNodes)
	{
		RequestExtensionsForNode(Node);
	}

	// Remove all extensions that are no longer needed
	InAnimBlueprint->RemoveAllExtension([](UBlueprintExtension* InExtension)
	{
		if(UAnimBlueprintExtension* AnimBlueprintExtension = Cast<UAnimBlueprintExtension>(InExtension))
		{
			return !RefreshSet.Contains(AnimBlueprintExtension->GetClass());
		}

		return false;
	});

	RefreshSet.Empty();
	GIsRefreshingExtensions = false;
}

void UAnimBlueprintExtension::ForEachExtension(UAnimBlueprint* InAnimBlueprint, TFunctionRef<void(UAnimBlueprintExtension*)> InFunction)
{
	for (const TObjectPtr<UBlueprintExtension>& BlueprintExtension : InAnimBlueprint->GetExtensions())
	{
		if(UAnimBlueprintExtension* AnimBlueprintExtension = Cast<UAnimBlueprintExtension>(BlueprintExtension))
		{
			InFunction(AnimBlueprintExtension);
		}
	}
}

const UScriptStruct* UAnimBlueprintExtension::GetInstanceDataType() const
{
	UScriptStruct* FoundStruct = FAnimSubsystemInstance::StaticStruct();

	for (TFieldIterator<FProperty> PropIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = CastField<FStructProperty>(*PropIt))
		{
			if (StructProp->Struct->IsChildOf(FAnimSubsystemInstance::StaticStruct()))
			{
				FoundStruct = StructProp->Struct;
				break;
			}
		}
	}

	return FoundStruct;
}

const UScriptStruct* UAnimBlueprintExtension::GetClassDataType() const
{
	UScriptStruct* FoundStruct = FAnimSubsystem::StaticStruct();

	for (TFieldIterator<FProperty> PropIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = CastField<FStructProperty>(*PropIt))
		{
			if (StructProp->Struct->IsChildOf(FAnimSubsystem::StaticStruct()))
			{
				FoundStruct = StructProp->Struct;
				break;
			}
		}
	}

	return FoundStruct;
}

const FStructProperty* UAnimBlueprintExtension::GetInstanceDataProperty() const
{
	for (TFieldIterator<FProperty> PropIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = CastField<FStructProperty>(*PropIt))
		{
			if (StructProp->Struct->IsChildOf(FAnimSubsystemInstance::StaticStruct()))
			{
				return StructProp;
			}
		}
	}

	return nullptr;
}

const FStructProperty* UAnimBlueprintExtension::GetClassDataProperty() const
{
	for (TFieldIterator<FProperty> PropIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = CastField<FStructProperty>(*PropIt))
		{
			if (StructProp->Struct->IsChildOf(FAnimSubsystem::StaticStruct()))
			{
				return StructProp;
			}
		}
	}

	return nullptr;
}

UAnimBlueprint* UAnimBlueprintExtension::GetAnimBlueprint() const
{
	return CastChecked<UAnimBlueprint>(GetOuter());
}

void* UAnimBlueprintExtension::GetClassDataInternal()
{
	if(const FStructProperty* Property = GetClassDataProperty())
	{
		return Property->ContainerPtrToValuePtr<void>(this);
	}

	static FAnimSubsystem Default;
	return &Default;
}

void* UAnimBlueprintExtension::GetInstanceDataInternal()
{
	if(const FStructProperty* Property = GetInstanceDataProperty())
	{
		return Property->ContainerPtrToValuePtr<void>(this);
	}

	static FAnimSubsystemInstance Default;
	return &Default;
}

void UAnimBlueprintExtension::BeginCompilation(IAnimBlueprintCompilerCreationContext& InCreationContext)
{
	HandleBeginCompilation(InCreationContext);
}

void UAnimBlueprintExtension::StartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	HandleStartCompilingClass(InClass, InCompilationContext, OutCompiledData);
}

void UAnimBlueprintExtension::PreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	HandlePreProcessAnimationNodes(InAnimNodes, InCompilationContext, OutCompiledData);
}

void UAnimBlueprintExtension::PostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	HandlePostProcessAnimationNodes(InAnimNodes, InCompilationContext, OutCompiledData);
}

void UAnimBlueprintExtension::FinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	HandleFinishCompilingClass(InClass, InCompilationContext, OutCompiledData);
}

void UAnimBlueprintExtension::PostExpansionStep(const UEdGraph* InGraph, IAnimBlueprintPostExpansionStepContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	HandlePostExpansionStep(InGraph, InCompilationContext, OutCompiledData);
}

void UAnimBlueprintExtension::CopyTermDefaultsToDefaultObject(UObject* InDefaultObject, IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext)
{
	if(InPerExtensionContext.GetTargetProperty() && InPerExtensionContext.GetDestinationPtr() && InPerExtensionContext.GetSourcePtr())
	{
		InPerExtensionContext.GetTargetProperty()->CopyCompleteValue(InPerExtensionContext.GetDestinationPtr(), InPerExtensionContext.GetSourcePtr());
	}

	HandleCopyTermDefaultsToDefaultObject(InDefaultObject, InCompilationContext, InPerExtensionContext);
}

void UAnimBlueprintExtension::CopyTermDefaultsToSparseClassData(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext)
{
	if(InPerExtensionContext.GetTargetProperty() && InPerExtensionContext.GetDestinationPtr() && InPerExtensionContext.GetSourcePtr())
	{
		InPerExtensionContext.GetTargetProperty()->CopyCompleteValue(InPerExtensionContext.GetDestinationPtr(), InPerExtensionContext.GetSourcePtr());
	}

	HandleCopyTermDefaultsToSparseClassData(InCompilationContext, InPerExtensionContext);
}

void UAnimBlueprintExtension::EndCompilation()
{
	HandleEndCompilation();
}
