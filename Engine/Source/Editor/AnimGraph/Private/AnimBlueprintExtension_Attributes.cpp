// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_Attributes.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "IAnimBlueprintCompilationContext.h"
#include "AnimationGraphSchema.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_SaveCachedPose.h"
#include "AnimGraphNode_UseCachedPose.h"
#include "AnimGraphNode_StateResult.h"
#include "AnimationStateGraph.h"
#include "AnimationStateMachineGraph.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimStateNode.h"
#include "AnimGraphNode_Root.h"
#include "AnimGraphAttributes.h"
#include "AnimGraphNode_TransitionResult.h"
#include "AnimGraphNode_CustomTransitionResult.h"
#include "AnimationCustomTransitionGraph.h"
#include "AnimStateTransitionNode.h"
#include "AnimGraphNode_BlendSpaceSampleResult.h"
#include "BlendSpaceGraph.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "AnimationBlendSpaceSampleGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_Attributes)

void UAnimBlueprintExtension_Attributes::HandlePostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{	
	using FNodeAttributeMap = TMap<UAnimGraphNode_Base*, UAnimGraphNode_Base::FNodeAttributeArray>;
	using FCachedPoseMap = TMultiMap<UAnimGraphNode_SaveCachedPose*, UAnimGraphNode_UseCachedPose*>;

	FNodeAttributeMap AttributeInputNodes;
	FNodeAttributeMap AttributeOutputNodes;
	FCachedPoseMap SaveCachedPoseMap;
	UAnimGraphNode_Base::FNodeAttributeArray Attributes;

	for(UAnimGraphNode_Base* Node : InAnimNodes)
	{
		// Establish links between save/used cached pose nodes
		if(UAnimGraphNode_UseCachedPose* UseCachedPoseNode = Cast<UAnimGraphNode_UseCachedPose>(Node))
		{
			if(UseCachedPoseNode->SaveCachedPoseNode.IsValid())
			{
				SaveCachedPoseMap.Add(UseCachedPoseNode->SaveCachedPoseNode.Get(), UseCachedPoseNode);
			}
		}
		
		// Get I/O attributes from all nodes
		Node->GetInputLinkAttributes(Attributes);
		if(Attributes.Num() > 0)
		{
			AttributeInputNodes.Add(Node, Attributes);
		}

		Attributes.Reset();

		Node->GetOutputLinkAttributes(Attributes);
		if(Attributes.Num() > 0)
		{
			AttributeOutputNodes.Add(Node, Attributes);
		}

		Attributes.Reset();
	}

	// Utility struct to check whether nodes with outputs reach nodes with inputs further towards the root
	struct FCheckOutputNodes
	{
	private:
		IAnimBlueprintCompilationContext& CompilationContext;
		const UAnimGraphNode_Base::FNodeAttributeArray& BaseAttributes;
		const FNodeAttributeMap& AttributeInputNodes;
		const FCachedPoseMap& SaveCachedPoseMap;
		UAnimGraphNode_Base* OutputNode;
		TSet<UEdGraphNode*> VisitedNodes;

	public:
		FCheckOutputNodes(IAnimBlueprintCompilationContext& InCompilationContext, const UAnimGraphNode_Base::FNodeAttributeArray& InAttributes, const FNodeAttributeMap& InAttributeInputNodes, const FCachedPoseMap& InSaveCachedPoseMap, UAnimGraphNode_Base* InNode)
			: CompilationContext(InCompilationContext)
			, BaseAttributes(InAttributes)
			, AttributeInputNodes(InAttributeInputNodes)
			, SaveCachedPoseMap(InSaveCachedPoseMap)
			, OutputNode(InNode)
		{
			UAnimGraphNode_Base::FNodeAttributeArray AbsorbedAttributes = TraverseNodes_Recursive(OutputNode, BaseAttributes);

			if(AbsorbedAttributes.Num() > 0)
			{
				// Push only absorbed attributes to the source node's internal set
				CompilationContext.AddAttributesToNode(OutputNode, AbsorbedAttributes);
			}
		}

	private:
		UAnimGraphNode_Base::FNodeAttributeArray TraverseNodes_Recursive_PerNode(UAnimGraphNode_Base* InLinkedNode, const UAnimGraphNode_Base::FNodeAttributeArray& InAttributes)
		{
			UAnimGraphNode_Base::FNodeAttributeArray AbsorbedAttributes;

			if (InLinkedNode)
			{
				if(!VisitedNodes.Contains(InLinkedNode))
				{
					// See if this node absorbs this attribute
					const UAnimGraphNode_Base::FNodeAttributeArray* AbsorbedAttributesPtr = AttributeInputNodes.Find(InLinkedNode);

					auto HasAttribute = [AbsorbedAttributesPtr, InLinkedNode](FName InAttribute)
					{
						return (AbsorbedAttributesPtr && AbsorbedAttributesPtr->Contains(InAttribute));
					};

					if(InLinkedNode->IsA<UAnimGraphNode_Root>() || InLinkedNode->IsA<UAnimGraphNode_TransitionResult>() || InAttributes.ContainsByPredicate(HasAttribute))
					{
						UAnimGraphNode_Base::FNodeAttributeArray ReducedAttributes;

						if(InLinkedNode->IsA<UAnimGraphNode_Root>() || InLinkedNode->IsA<UAnimGraphNode_TransitionResult>())
						{
							const UAnimGraphAttributes* AnimGraphAttributes = GetDefault<UAnimGraphAttributes>();

							auto IsAttributeBlendable = [AnimGraphAttributes](const FName& InAttribute)
							{
								const FAnimGraphAttributeDesc* Desc = AnimGraphAttributes->FindAttributeDesc(InAttribute);
								return Desc && Desc->Blend == EAnimGraphAttributeBlend::Blendable;
							};

							// Absorb all blendables at the root
							for(const FName& Attribute : InAttributes)
							{
								if(IsAttributeBlendable(Attribute))
								{
									AbsorbedAttributes.Add(Attribute);
								}
								else
								{
									ReducedAttributes.Add(Attribute);
								}
							}
						}
						else
						{
							// Reduce the set of attributes that we are using in this traversal
							for(const FName& Attribute : InAttributes)
							{
								if(HasAttribute(Attribute))
								{
									AbsorbedAttributes.Add(Attribute);
								}
								else
								{
									ReducedAttributes.Add(Attribute);
								}
							}
						}

						if(ReducedAttributes.Num() > 0)
						{
							UAnimGraphNode_Base::FNodeAttributeArray AbsorbedAttributesBelow = TraverseNodes_Recursive(InLinkedNode, ReducedAttributes);
							AbsorbedAttributes.Append(AbsorbedAttributesBelow);
						}
					}
					else
					{
						UAnimGraphNode_Base::FNodeAttributeArray AbsorbedAttributesBelow = TraverseNodes_Recursive(InLinkedNode, InAttributes);
						AbsorbedAttributes.Append(AbsorbedAttributesBelow);
					}
				}
				else
				{
					// already visited by another branch of a cached pose, see if any attributes got absorbed the last time we took this branch
					TArrayView<const FName> PreviouslyAbsorbedAttributes = CompilationContext.GetAttributesFromNode(InLinkedNode);
					AbsorbedAttributes.Append(PreviouslyAbsorbedAttributes.GetData(), PreviouslyAbsorbedAttributes.Num());
				}
			}

			// Post-recursion, we add any pass through attributes that got absorbed
			CompilationContext.AddAttributesToNode(InLinkedNode, AbsorbedAttributes);

			return AbsorbedAttributes;
		}

		UAnimGraphNode_Base::FNodeAttributeArray TraverseNodes_Recursive(UEdGraphNode* InNode, const UAnimGraphNode_Base::FNodeAttributeArray& InAttributes)
		{
			VisitedNodes.Add(InNode);

			for (UEdGraphPin* Pin : InNode->Pins)
			{
				if (UAnimationGraphSchema::IsPosePin(Pin->PinType) && Pin->Direction == EGPD_Output)
				{
					// Traverse pins
					for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
					{
						UEdGraphNode* OwningNode = LinkedPin->GetOwningNode();
						if(UAnimGraphNode_Base* LinkedNode = Cast<UAnimGraphNode_Base>(OwningNode))
						{
							return TraverseNodes_Recursive_PerNode(LinkedNode, InAttributes);
						}
						else if(OwningNode != nullptr)
						{
							// Its a pose link, but not on an anim node, likely a knot
							return TraverseNodes_Recursive(OwningNode, InAttributes);
						}
					}
				}
			}

			// Traverse saved cached pose->all use cached pose nodes
			if(UAnimGraphNode_SaveCachedPose* SaveCachedPoseNode = Cast<UAnimGraphNode_SaveCachedPose>(InNode))
			{
				TArray<UAnimGraphNode_UseCachedPose*> UseCachedPoseNodes;
				SaveCachedPoseMap.MultiFind(SaveCachedPoseNode, UseCachedPoseNodes);
				for(int32 UseCachedPoseIndex = 0; UseCachedPoseIndex < UseCachedPoseNodes.Num(); ++UseCachedPoseIndex)
				{
					UAnimGraphNode_UseCachedPose* UsedCachedPoseNode = UseCachedPoseNodes[UseCachedPoseIndex];
					UAnimGraphNode_Base::FNodeAttributeArray AbsorbedAttributesBelow = TraverseNodes_Recursive_PerNode(UsedCachedPoseNode, InAttributes);
					if(UseCachedPoseIndex == UseCachedPoseNodes.Num() - 1)
					{
						return AbsorbedAttributesBelow;
					}
				}
			}
			// Traverse out of custom transitions
			else if(UAnimGraphNode_CustomTransitionResult* TransitionResultNode = Cast<UAnimGraphNode_CustomTransitionResult>(InNode))
			{
				UAnimationCustomTransitionGraph* TransitionGraph = CastChecked<UAnimationCustomTransitionGraph>(TransitionResultNode->GetGraph());
				UAnimStateTransitionNode* TransitionNode = CastChecked<UAnimStateTransitionNode>(TransitionGraph->GetOuter());
				UAnimationStateMachineGraph* StateMachineGraph = CastChecked<UAnimationStateMachineGraph>(TransitionNode->GetOuter());
				UAnimGraphNode_StateMachineBase* StateMachineNode = CastChecked<UAnimGraphNode_StateMachineBase>(StateMachineGraph->GetOuter());
				return TraverseNodes_Recursive_PerNode(StateMachineNode, InAttributes);
			}
			// Traverse out of state machines
			else if(UAnimGraphNode_StateResult* StateResultNode = Cast<UAnimGraphNode_StateResult>(InNode))
			{
				UAnimationStateGraph* StateGraph = CastChecked<UAnimationStateGraph>(StateResultNode->GetGraph());
				UAnimStateNode* StateNode = CastChecked<UAnimStateNode>(StateGraph->GetOuter());
				UAnimationStateMachineGraph* StateMachineGraph = CastChecked<UAnimationStateMachineGraph>(StateNode->GetOuter());
				UAnimGraphNode_StateMachineBase* StateMachineNode = CastChecked<UAnimGraphNode_StateMachineBase>(StateMachineGraph->GetOuter());
				return TraverseNodes_Recursive_PerNode(StateMachineNode, InAttributes);
			}
			else if(UAnimGraphNode_BlendSpaceSampleResult* SampleResultNode = Cast<UAnimGraphNode_BlendSpaceSampleResult>(InNode))
			{
				UAnimationBlendSpaceSampleGraph* SampleGraph = CastChecked<UAnimationBlendSpaceSampleGraph>(SampleResultNode->GetGraph());
				UBlendSpaceGraph* BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(SampleGraph->GetOuter());
				UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceGraphNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());

				return TraverseNodes_Recursive_PerNode(BlendSpaceGraphNode, InAttributes);
			}

			return UAnimGraphNode_Base::FNodeAttributeArray();
		}
	};

	// Color connected nodes root-wise from output nodes
	for(auto& NodeAttributesPair : AttributeOutputNodes)
	{
		FCheckOutputNodes Checker(InCompilationContext, NodeAttributesPair.Value, AttributeInputNodes, SaveCachedPoseMap, NodeAttributesPair.Key);
	}
}
