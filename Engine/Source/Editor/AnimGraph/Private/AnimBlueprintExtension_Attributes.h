// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_Base.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_Attributes.generated.h"

class IAnimBlueprintCompilerCreationContext;
class IAnimBlueprintCompilationContext;
class IAnimBlueprintGeneratedClassCompiledData;

// Extension to propogate attributes from outputs to inputs and to build a static debug record of their path through the graph
UCLASS(MinimalAPI)
class UAnimBlueprintExtension_Attributes : public UAnimBlueprintExtension
{
	GENERATED_BODY()

private:
	// UAnimBlueprintExtension interface
	virtual void HandlePostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
};