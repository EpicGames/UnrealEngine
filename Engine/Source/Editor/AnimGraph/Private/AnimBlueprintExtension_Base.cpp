// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_Base.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_PropertyAccess.h"
#include "AnimGraphNode_Base.h"
#include "AnimationGraphSchema.h"
#include "AnimGraphNode_CustomProperty.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_VariableSet.h"
#include "K2Node_StructMemberSet.h"
#include "K2Node_StructMemberGet.h"
#include "K2Node_CallArrayFunction.h"
#include "Kismet/KismetArrayLibrary.h"
#include "K2Node_Knot.h"
#include "String/ParseTokens.h"
#include "K2Node_VariableGet.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_MakeStruct.h"
#include "Kismet/KismetMathLibrary.h"
#include "K2Node_TransitionRuleGetter.h"
#include "Animation/AnimNode_LinkedAnimGraph.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "IPropertyAccessEditor.h"
#include "IPropertyAccessCompiler.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "IAnimBlueprintCompilationContext.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "IAnimBlueprintPostExpansionStepContext.h"
#include "IAnimBlueprintCompilationBracketContext.h"
#include "Features/IModularFeatures.h"
#include "Animation/ExposedValueHandler.h"
#include "AnimGraphNodeBinding_Base.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_Base)

#define LOCTEXT_NAMESPACE "AnimBlueprintExtension_Base"

DECLARE_CYCLE_STAT(TEXT("Create Evaluation Handler"), EAnimBlueprintCompilerStats_CreateEvaluationHandler, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Create Evaluation Handler - Node Properties"), EAnimBlueprintCompilerStats_CreateEvaluationHandler_NodeProperties, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Create Evaluation Handler - Create Assignment Node"), EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateAssignmentNode, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Create Evaluation Handler - Create Instance Assignment Node"), EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateInstanceAssignmentNode, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Create Evaluation Handler - Create Instance Assignment Node - Build Property List"), EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateInstanceAssignmentNode_BuildPropertyList, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Create Evaluation Handler - Create Instance Assignment Node - Create Visible Pins"), EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateInstanceAssignmentNode_CreateVisiblePins, STATGROUP_KismetCompiler )

void UAnimBlueprintExtension_Base::HandleCopyTermDefaultsToDefaultObject(UObject* InDefaultObject, IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext)
{
	UAnimInstance* DefaultAnimInstance = Cast<UAnimInstance>(InDefaultObject);

	if(DefaultAnimInstance)
	{
		Subsystem.PatchValueHandlers(DefaultAnimInstance->GetClass());

		// Update blueprint usage of all graph nodes that have properties exposed
		for (const TPair<UAnimGraphNode_Base*, FEvaluationHandlerRecord>& EvaluationHandlerPair : PerNodeStructEvalHandlers)
		{
			if (EvaluationHandlerPair.Value.bHasProperties && EvaluationHandlerPair.Value.ServicedProperties.Num() == 0)
			{
				UAnimGraphNode_Base* Node = CastChecked<UAnimGraphNode_Base>(EvaluationHandlerPair.Key);
				UAnimGraphNode_Base* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_Base>(Node);
				TrueNode->BlueprintUsage = EBlueprintUsage::DoesNotUseBlueprint;
			}
		}

		for(const FEvaluationHandlerRecord& EvaluationHandler : ValidEvaluationHandlerList)
		{
			if(EvaluationHandler.AnimGraphNode)
			{
				UAnimGraphNode_Base* Node = CastChecked<UAnimGraphNode_Base>(EvaluationHandler.AnimGraphNode);
				UAnimGraphNode_Base* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_Base>(Node);	
				
				if(EvaluationHandler.EvaluationHandlerIdx != INDEX_NONE && EvaluationHandler.ServicedProperties.Num() > 0)
				{
					const FAnimNodeSinglePropertyHandler& Handler = EvaluationHandler.ServicedProperties.CreateConstIterator()->Value;
					check(Handler.CopyRecords.Num() > 0);

					bool bHandlerUsesBlueprint = false;
					if(Subsystem.ExposedValueHandlers[EvaluationHandler.EvaluationHandlerIdx].GetHandlerStruct()->IsChildOf(FAnimNodeExposedValueHandler_Base::StaticStruct()))
					{
						const FAnimNodeExposedValueHandler_Base* BaseHandler = static_cast<const FAnimNodeExposedValueHandler_Base*>(Subsystem.ExposedValueHandlers[EvaluationHandler.EvaluationHandlerIdx].GetHandler());
						bHandlerUsesBlueprint = BaseHandler->BoundFunction != NAME_None;
					}
					TrueNode->BlueprintUsage = bHandlerUsesBlueprint ? EBlueprintUsage::UsesBlueprint : EBlueprintUsage::DoesNotUseBlueprint;

#if WITH_EDITORONLY_DATA // ANIMINST_PostCompileValidation
					const bool bWarnAboutBlueprintUsage = InCompilationContext.GetAnimBlueprint()->bWarnAboutBlueprintUsage || DefaultAnimInstance->PCV_ShouldWarnAboutNodesNotUsingFastPath();
					const bool bNotifyAboutBlueprintUsage = DefaultAnimInstance->PCV_ShouldNotifyAboutNodesNotUsingFastPath();
#else
					const bool bWarnAboutBlueprintUsage = InCompilationContext.GetAnimBlueprint()->bWarnAboutBlueprintUsage;
					const bool bNotifyAboutBlueprintUsage = false;
#endif
					if ((TrueNode->BlueprintUsage == EBlueprintUsage::UsesBlueprint) && (bWarnAboutBlueprintUsage || bNotifyAboutBlueprintUsage))
					{
						const FString MessageString = LOCTEXT("BlueprintUsageWarning", "Node @@ uses Blueprint to update its values, access member variables directly or use a constant value for better performance.").ToString();
						if (bWarnAboutBlueprintUsage)
						{
							InCompilationContext.GetMessageLog().Warning(*MessageString, Node);
						}
						else
						{
							InCompilationContext.GetMessageLog().Note(*MessageString, Node);
						}
					}
				}
			}
		}
	}
}

void UAnimBlueprintExtension_Base::HandlePostExpansionStep(const UEdGraph* InGraph, IAnimBlueprintPostExpansionStepContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UEdGraph* ConsolidatedEventGraph = InCompilationContext.GetConsolidatedEventGraph();
	if(InGraph == ConsolidatedEventGraph)
	{
		// Skip fast-path generation if the property access system is unavailable.
		// Note that this wont prevent property access 'binding' copy records from running, only
		// old-style 'fast-path' records that are derived from BP pure chains
		if(IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
		{
			for(FEvaluationHandlerRecord& HandlerRecord : ValidEvaluationHandlerList)
			{
				HandlerRecord.BuildFastPathCopyRecords(InCompilationContext);

				if(HandlerRecord.IsFastPath())
				{
					for(UEdGraphNode* CustomEventNode : HandlerRecord.CustomEventNodes)
					{
						// Remove custom event nodes as we dont need it any more
						ConsolidatedEventGraph->RemoveNode(CustomEventNode);
					}
				}
			}
		}

		// Cull out all anim nodes as they dont contribute to execution at all
		for (int32 NodeIndex = 0; NodeIndex < ConsolidatedEventGraph->Nodes.Num(); ++NodeIndex)
		{
			if(UAnimGraphNode_Base* Node = Cast<UAnimGraphNode_Base>(ConsolidatedEventGraph->Nodes[NodeIndex]))
			{
				Node->BreakAllNodeLinks();
				ConsolidatedEventGraph->Nodes.RemoveAtSwap(NodeIndex);
				--NodeIndex;
			}
		}
	}
}

void UAnimBlueprintExtension_Base::PatchEvaluationHandlers(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	// Exposed value handlers indices must match the index of anim node properties, 
	// so we iterate over anim node properties here when patching up
	const int32 NumAllocatedNodes = InCompilationContext.GetAllocatedAnimNodeIndices().Num();

	for(const TPair<UAnimGraphNode_Base*, int32>& GraphNodePair : InCompilationContext.GetAllocatedAnimNodeIndices())
	{
		if(int32* EvaluationHandlerIndexPtr = ValidEvaluationHandlerMap.Find(GraphNodePair.Key))
		{
			// Indices here are in reverse order with respect to iterated properties as properties are prepended to the linked list when they are added
			const int32 NodePropertyIndex = NumAllocatedNodes - 1 - GraphNodePair.Value;

			FEvaluationHandlerRecord& EvaluationHandlerRecord = ValidEvaluationHandlerList[*EvaluationHandlerIndexPtr];
			EvaluationHandlerRecord.EvaluationHandlerIdx = NodePropertyIndex;
			EvaluationHandlerRecord.PatchAnimNodeExposedValueHandler(InClass, InCompilationContext);
		}
	}
}

void UAnimBlueprintExtension_Base::HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	PerNodeStructEvalHandlers.Empty();
	ValidEvaluationHandlerList.Empty();
	ValidEvaluationHandlerMap.Empty();
	HandlerFunctionNames.Empty();
	PreLibraryCompiledDelegateHandle.Reset();
	PostLibraryCompiledDelegateHandle.Reset();	

	UAnimBlueprintExtension_PropertyAccess* PropertyAccessExtension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_PropertyAccess>(GetAnimBlueprint());
	if(PropertyAccessExtension)
	{
		PreLibraryCompiledDelegateHandle = PropertyAccessExtension->OnPreLibraryCompiled().AddLambda([this, PropertyAccessExtension, InClass]()
		{
			if(IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
			{
				IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

				// Build the classes property access library before the library is compiled
				for(FEvaluationHandlerRecord& HandlerRecord : ValidEvaluationHandlerList)
				{
					for(TPair<FName, FAnimNodeSinglePropertyHandler>& PropertyHandler : HandlerRecord.ServicedProperties)
					{
						for(FPropertyCopyRecord& Record : PropertyHandler.Value.CopyRecords)
						{
							if(Record.IsFastPath())
							{
								Record.LibraryHandle = PropertyAccessExtension->AddCopy(Record.SourcePropertyPath, Record.DestPropertyPath, Record.BindingContextId, HandlerRecord.AnimGraphNode);
							}
						}
					}
				}
			}

			PropertyAccessExtension->OnPreLibraryCompiled().Remove(PreLibraryCompiledDelegateHandle);
		});

		PostLibraryCompiledDelegateHandle = PropertyAccessExtension->OnPostLibraryCompiled().AddLambda([this, PropertyAccessExtension, InClass](IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
		{
			for(FEvaluationHandlerRecord& HandlerRecord : ValidEvaluationHandlerList)
			{
				UAnimGraphNode_Base* OriginalNode = Cast<UAnimGraphNode_Base>(InCompilationContext.GetMessageLog().FindSourceObject(HandlerRecord.AnimGraphNode));
				
				// Map global copy index to batched indices
				for(TPair<FName, FAnimNodeSinglePropertyHandler>& PropertyHandler : HandlerRecord.ServicedProperties)
				{
					for(FPropertyCopyRecord& CopyRecord : PropertyHandler.Value.CopyRecords)
					{
						if(CopyRecord.IsFastPath())
						{
							CopyRecord.LibraryCompiledHandle = PropertyAccessExtension->GetCompiledHandle(CopyRecord.LibraryHandle);

							// Push compiled desc back to original node for feedback
							FName BindingName = CopyRecord.DestProperty->GetFName();
							if(CopyRecord.DestArrayIndex != INDEX_NONE)
							{
								BindingName.SetNumber(CopyRecord.DestArrayIndex + 1);
							}

							if(UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(OriginalNode->GetMutableBinding()))
							{
								if(FAnimGraphNodePropertyBinding* PropertyBinding = Binding->PropertyBindings.Find(BindingName))
								{
									if(CopyRecord.LibraryCompiledHandle.IsValid())
									{
										PropertyBinding->CompiledContext = UAnimBlueprintExtension_PropertyAccess::GetCompiledHandleContext(CopyRecord.LibraryCompiledHandle);
										PropertyBinding->CompiledContextDesc = UAnimBlueprintExtension_PropertyAccess::GetCompiledHandleContextDesc(CopyRecord.LibraryCompiledHandle);
									}
									else
									{
										PropertyBinding->CompiledContext = FText::GetEmpty();
										PropertyBinding->CompiledContextDesc = FText::GetEmpty();
									}
								}
							}
						}
					}
				}
			}

			PatchEvaluationHandlers(InClass, InCompilationContext, OutCompiledData);

			PropertyAccessExtension->OnPostLibraryCompiled().Remove(PostLibraryCompiledDelegateHandle);
		});
	}
}

void UAnimBlueprintExtension_Base::HandleFinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimBlueprintExtension_PropertyAccess* PropertyAccessExtension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_PropertyAccess>(GetAnimBlueprint());
	if(PropertyAccessExtension == nullptr)
	{
		// Without the property access system we need to patch generated function names here
		PatchEvaluationHandlers(InClass, InCompilationContext, OutCompiledData);
	}
}

void UAnimBlueprintExtension_Base::ProcessPosePins(UAnimGraphNode_Base* InNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();

	FStructProperty* NodeProperty = CastFieldChecked<FStructProperty>(InCompilationContext.GetAllocatedPropertiesByNode().FindChecked(InNode));

	for (auto SourcePinIt = InNode->Pins.CreateIterator(); SourcePinIt; ++SourcePinIt)
	{
		UEdGraphPin* SourcePin = *SourcePinIt;

		// Register pose links for future use
		if ((SourcePin->Direction == EGPD_Input) && (AnimGraphDefaultSchema->IsPosePin(SourcePin->PinType)))
		{
			// Input pose pin, going to need to be linked up
			FPoseLinkMappingRecord LinkRecord = InNode->GetLinkIDLocation(NodeProperty->Struct, SourcePin);
			if (LinkRecord.IsValid())
			{
				InCompilationContext.AddPoseLinkMappingRecord(LinkRecord);
			}
			else
			{
				//@TODO: ANIMREFACTOR: It's probably OK to have certain pins ignored eventually, but this is very helpful during development
				InCompilationContext.GetMessageLog().Note(TEXT("@@ was visible but ignored"), SourcePin);
			}
		}
	}
}

void UAnimBlueprintExtension_Base::ProcessNonPosePins(UAnimGraphNode_Base* InNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData, EPinProcessingFlags InFlags)
{
	const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();

	FStructProperty* NodeProperty = CastFieldChecked<FStructProperty>(InCompilationContext.GetAllocatedPropertiesByNode().FindChecked(InNode));

	for (auto SourcePinIt = InNode->Pins.CreateIterator(); SourcePinIt; ++SourcePinIt)
	{
		UEdGraphPin* SourcePin = *SourcePinIt;
		bool bConsumed = false;

		if ((SourcePin->Direction == EGPD_Input) && (AnimGraphDefaultSchema->IsPosePin(SourcePin->PinType)))
		{
			bConsumed = true;
		}
		else if(!InNode->ShouldCreateStructEvalHandlers() || !EnumHasAnyFlags(InFlags, EPinProcessingFlags::All))
		{
			bConsumed = true;
		}
		else
		{
			FEvaluationHandlerRecord& EvalHandler = PerNodeStructEvalHandlers.FindOrAdd(InNode);
			
			// The property source for our data, either the struct property for an anim node, or the
			// owning anim instance if using a linked instance node.
			FProperty* SourcePinProperty = nullptr;
			int32 SourceArrayIndex = INDEX_NONE;
			bool bInstancePropertyExists = false;

			// We have special handling below if we're targeting a linked instance instead of our own instance properties
			UAnimGraphNode_CustomProperty* CustomPropertyNode = Cast<UAnimGraphNode_CustomProperty>(InNode);

			InNode->GetPinAssociatedProperty(NodeProperty->Struct, SourcePin, /*out*/ SourcePinProperty, /*out*/ SourceArrayIndex);

			// Does this pin have an associated evaluation handler?
			if(!SourcePinProperty && CustomPropertyNode)
			{
				// Custom property nodes use instance properties not node properties as they aren't UObjects
				// and we can't store non-native properties there
				CustomPropertyNode->GetInstancePinProperty(InCompilationContext, SourcePin, SourcePinProperty);
				bInstancePropertyExists = true;
			}
			
			if (SourcePinProperty != NULL)
			{
				EvalHandler.bHasProperties = true;

				if (SourcePin->LinkedTo.Num() == 0)
				{
					if(EnumHasAnyFlags(InFlags, EPinProcessingFlags::Constants))
					{
						// Literal that can be pushed into the CDO instead of re-evaluated every frame
						bConsumed = true;
					}
				}
				else
				{
					if (EnumHasAnyFlags(InFlags, EPinProcessingFlags::BlueprintHandlers))
					{
						// Dynamic value that needs to be wired up and evaluated each frame
						const FString& EvaluationHandlerStr = SourcePinProperty->GetMetaData(AnimGraphDefaultSchema->NAME_OnEvaluate);
						FName EvaluationHandlerName(*EvaluationHandlerStr);
						if (EvaluationHandlerName != NAME_None)
						{
							// warn that NAME_OnEvaluate is deprecated:
							InCompilationContext.GetMessageLog().Warning(*LOCTEXT("OnEvaluateDeprecated", "OnEvaluate meta data is deprecated, found on @@").ToString(), SourcePinProperty);
						}
					
						ensure(EvalHandler.NodeVariableProperty == nullptr || EvalHandler.NodeVariableProperty == NodeProperty);
						EvalHandler.AnimGraphNode = InNode;
						EvalHandler.NodeVariableProperty = NodeProperty;
						EvalHandler.RegisterPin(SourcePin, SourcePinProperty, SourceArrayIndex, EnumHasAnyFlags(InFlags, EPinProcessingFlags::PropertyAccessFastPath));
						// if it's not instance property, ensure we mark it
						EvalHandler.bServicesNodeProperties = EvalHandler.bServicesNodeProperties || !bInstancePropertyExists;

						if (CustomPropertyNode)
						{
							EvalHandler.bServicesInstanceProperties = EvalHandler.bServicesInstanceProperties || bInstancePropertyExists;

							FAnimNodeSinglePropertyHandler* SinglePropHandler = EvalHandler.ServicedProperties.Find(SourcePinProperty->GetFName());
							check(SinglePropHandler); // Should have been added in RegisterPin

							// Flag that the target property is actually on the instance class and not the node
							SinglePropHandler->bInstanceIsTarget = bInstancePropertyExists;
						}

						bConsumed = true;
					}
				}

				UEdGraphPin* TrueSourcePin = InCompilationContext.GetMessageLog().FindSourcePin(SourcePin);
				if (TrueSourcePin)
				{
					OutCompiledData.GetBlueprintDebugData().RegisterClassPropertyAssociation(TrueSourcePin, SourcePinProperty);
				}
			}
		}

		if (!bConsumed && (SourcePin->Direction == EGPD_Input) && !AnimGraphDefaultSchema->IsPosePin(SourcePin->PinType))
		{
			//@TODO: ANIMREFACTOR: It's probably OK to have certain pins ignored eventually, but this is very helpful during development
			InCompilationContext.GetMessageLog().Note(TEXT("@@ was visible but ignored"), SourcePin);
		}
	}

	if (EnumHasAnyFlags(InFlags, EPinProcessingFlags::PropertyAccessBindings))
	{
		// Add any property bindings
		if (UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(InNode->GetMutableBinding()))
		{
			for(auto Iter = Binding->PropertyBindings.CreateIterator(); Iter; ++Iter)
			{
				if(Iter.Value().bIsBound)
				{
					FEvaluationHandlerRecord& EvalHandler = PerNodeStructEvalHandlers.FindOrAdd(InNode);
					EvalHandler.AnimGraphNode = InNode;

					// for array properties we need to account for the extra FName number 
					FName ComparisonName = Iter.Key();
					ComparisonName.SetNumber(0);

					if (FProperty* Property = FindFProperty<FProperty>(NodeProperty->Struct, ComparisonName))
					{
						EvalHandler.NodeVariableProperty = NodeProperty;
						EvalHandler.bServicesNodeProperties = true;
						EvalHandler.RegisterPropertyBinding(Property, Iter.Value());
					}
					else if(FProperty* ClassProperty = FindFProperty<FProperty>(InCompilationContext.GetBlueprint()->SkeletonGeneratedClass, Iter.Value().PropertyName))
					{
						EvalHandler.NodeVariableProperty = NodeProperty;
						EvalHandler.bServicesInstanceProperties = true;
						EvalHandler.RegisterPropertyBinding(ClassProperty, Iter.Value());
					}
					else
					{
						// Binding is no longer valid, remove it
						Iter.RemoveCurrent();
					}
				}
			}
		}
	}
}

void UAnimBlueprintExtension_Base::CreateEvaluationHandlerForNode(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InNode)
{
	if(FEvaluationHandlerRecord* RecordPtr = PerNodeStructEvalHandlers.Find(InNode))
	{
		// Generate a new event to update the value of these properties
		FEvaluationHandlerRecord& Record = *RecordPtr;

		if (Record.NodeVariableProperty)
		{
			CreateEvaluationHandler(InCompilationContext, InNode, Record);

			RedirectPropertyAccesses(InCompilationContext, InNode, Record);
			
			int32 NewIndex = ValidEvaluationHandlerList.Add(Record);
			ValidEvaluationHandlerMap.Add(InNode, NewIndex);
		}
	}
}

// Optional pin manager used to optimize the creation of internal struct member set nodes 
struct FInternalOptionalPinManager : public FOptionalPinManager
{
	FInternalOptionalPinManager(UAnimGraphNode_Base* InNode, FStructProperty* InNodeProperty, IAnimBlueprintCompilationContext& InCompilationContext)
		: Node(InNode)
		, NodeProperty(InNodeProperty)
		, CompilationContext(InCompilationContext)
	{}

	void BuildPropertyList(TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct)
	{
		// Build optional pins for all properties
		for(TFieldIterator<FProperty> It(SourceStruct); It; ++It)
		{
			FOptionalPinFromProperty& OptionalPin = Properties.AddDefaulted_GetRef();
			OptionalPin.PropertyName = It->GetFName();
		}

		// Then expose only those that have records for this node 
		for(TFieldIterator<FProperty> It(NodeProperty->Struct); It; ++It)
		{
			if(const IAnimBlueprintCompilationContext::FFoldedPropertyRecord* FoldedPropertyRecord = CompilationContext.GetFoldedPropertyRecord(Node, It->GetFName()))
			{
				// Dont expose array properties here - they are handled by a struct member get-by-ref
				if(!FoldedPropertyRecord->bIsOnClass && !FoldedPropertyRecord->GeneratedProperty->IsA<FArrayProperty>())
				{
					FOptionalPinFromProperty& OptionalPin = Properties[Properties.Num() - 1 - FoldedPropertyRecord->PropertyIndex];
				
					check(OptionalPin.PropertyName == FoldedPropertyRecord->GeneratedProperty->GetFName());
					OptionalPin.bShowPin = true;
				}
			}
		}
	}

	// Duplicated & re-worked from base class (because we are never re-creating) to optimize our case
	void CreateVisiblePinsEx(TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct, EEdGraphPinDirection Direction, UK2Node* TargetNode)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		for (FOptionalPinFromProperty& PropertyEntry : Properties)
		{
			if (PropertyEntry.bShowPin)
			{	
				if (FProperty* OuterProperty = FindFieldChecked<FProperty>(SourceStruct, PropertyEntry.PropertyName))
				{
					// Not an array property
					FEdGraphPinType PinType;
					if (Schema->ConvertPropertyToPinType(OuterProperty, /*out*/ PinType))
					{
						// Create the pin
						const FName PinName = PropertyEntry.PropertyName;
						UEdGraphPin* NewPin = TargetNode->CreatePin(Direction, PinType, PinName);
						NewPin->PinFriendlyName = FText::FromString(PropertyEntry.PropertyFriendlyName.IsEmpty() ? PinName.ToString() : PropertyEntry.PropertyFriendlyName);
						NewPin->bNotConnectable = !PropertyEntry.bIsSetValuePinVisible;
						NewPin->bDefaultValueIsIgnored = !PropertyEntry.bIsSetValuePinVisible;
						
						if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(OuterProperty))
						{
							NewPin->DefaultValue = EnumProperty->GetEnum()->GetNameStringByIndex(0);
						}
						else if (FByteProperty* ByteProperty = CastField<FByteProperty>(OuterProperty))
						{
							if (ByteProperty->IsEnum())
							{
								NewPin->DefaultValue = ByteProperty->GetIntPropertyEnum()->GetNameStringByIndex(0);
							}							
						}

						Schema->ConstructBasicPinTooltip(*NewPin, PropertyEntry.PropertyTooltip, NewPin->PinToolTip);
					}
				}
			}
		}
	}

	UAnimGraphNode_Base* Node;
	FStructProperty* NodeProperty;
	IAnimBlueprintCompilationContext& CompilationContext;
};

void UAnimBlueprintExtension_Base::CreateEvaluationHandler(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InNode, FEvaluationHandlerRecord& Record)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_CreateEvaluationHandler);

	// Shouldn't create a handler if there is nothing to work with
	check(Record.ServicedProperties.Num() > 0);
	check(Record.NodeVariableProperty != NULL);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();

	// Use the node GUID for a stable name across compiles
	FString FunctionName = FString::Printf(TEXT("%s_%s_%s_%s"), *AnimGraphDefaultSchema->DefaultEvaluationHandlerName.ToString(), *InNode->GetOuter()->GetName(), *InNode->GetClass()->GetName(), *InNode->NodeGuid.ToString());
	Record.HandlerFunctionName = FName(*FunctionName);

	// check function name isnt already used (data exists that can contain duplicate GUIDs) and apply a numeric extension until it is unique
	int32 ExtensionIndex = 0;
	FName* ExistingName = HandlerFunctionNames.Find(Record.HandlerFunctionName);
	while (ExistingName != nullptr)
	{
		FunctionName = FString::Printf(TEXT("%s_%s_%s_%s_%d"), *AnimGraphDefaultSchema->DefaultEvaluationHandlerName.ToString(), *InNode->GetOuter()->GetName(), *InNode->GetClass()->GetName(), *InNode->NodeGuid.ToString(), ExtensionIndex);
		Record.HandlerFunctionName = FName(*FunctionName);
		ExistingName = HandlerFunctionNames.Find(Record.HandlerFunctionName);
		ExtensionIndex++;
	}

	HandlerFunctionNames.Add(Record.HandlerFunctionName);

	// Add a custom event in the graph
	UK2Node_CustomEvent* CustomEventNode = InCompilationContext.SpawnIntermediateNode<UK2Node_CustomEvent>(InNode, InCompilationContext.GetConsolidatedEventGraph());
	CustomEventNode->bInternalEvent = true;
	CustomEventNode->CustomFunctionName = Record.HandlerFunctionName;
	CustomEventNode->AllocateDefaultPins();
	Record.CustomEventNodes.Add(CustomEventNode);

	// The ExecChain is the current exec output pin in the linear chain
	UEdGraphPin* ExecChain = K2Schema->FindExecutionPin(*CustomEventNode, EGPD_Output);
	if (Record.bServicesInstanceProperties)
	{
		// Need to create a variable set call for each serviced property in the handler
		for (TPair<FName, FAnimNodeSinglePropertyHandler>& PropHandlerPair : Record.ServicedProperties)
		{
			FAnimNodeSinglePropertyHandler& PropHandler = PropHandlerPair.Value;
			FName PropertyName = PropHandlerPair.Key;

			// We only want to deal with instance targets in here
			if (PropHandler.bInstanceIsTarget)
			{
				for (FPropertyCopyRecord& CopyRecord : PropHandler.CopyRecords)
				{
					if(CopyRecord.DestPin)
					{
						// New set node for the property
						UK2Node_VariableSet* VarAssignNode = InCompilationContext.SpawnIntermediateNode<UK2Node_VariableSet>(InNode, InCompilationContext.GetConsolidatedEventGraph());
						VarAssignNode->VariableReference.SetSelfMember(CopyRecord.DestProperty->GetFName());
						VarAssignNode->AllocateDefaultPins();
						Record.CustomEventNodes.Add(VarAssignNode);

						// Wire up the exec line, and update the end of the chain
						UEdGraphPin* ExecVariablesIn = K2Schema->FindExecutionPin(*VarAssignNode, EGPD_Input);
						ExecChain->MakeLinkTo(ExecVariablesIn);
						ExecChain = K2Schema->FindExecutionPin(*VarAssignNode, EGPD_Output);

						// Find the property pin on the set node and configure
						for (UEdGraphPin* TargetPin : VarAssignNode->Pins)
						{
							FName PinPropertyName(TargetPin->PinName);

							if (PinPropertyName == PropertyName)
							{
								// This is us, wire up the variable
								UEdGraphPin* DestPin = CopyRecord.DestPin;

								// Copy the data (link up to the source nodes)
								TargetPin->CopyPersistentDataFromOldPin(*DestPin);
								InCompilationContext.GetMessageLog().NotifyIntermediatePinCreation(TargetPin, DestPin);
								
								break;
							}
						}
					}
				}
			}
		}
	}

	if (Record.bServicesNodeProperties)
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_CreateEvaluationHandler_NodeProperties);
		
		UK2Node_StructMemberSet* AssignmentNode;
		{
			BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateAssignmentNode);
			
			// Create a struct member write node to store the parameters into the animation node
			AssignmentNode = InCompilationContext.SpawnIntermediateNode<UK2Node_StructMemberSet>(InNode, InCompilationContext.GetConsolidatedEventGraph());
			AssignmentNode->VariableReference.SetSelfMember(Record.NodeVariableProperty->GetFName());
			AssignmentNode->StructType = Record.NodeVariableProperty->Struct;
			AssignmentNode->AllocateExecPins();

			// Simple FOptionalPinManager that exposes all pins. The default used in UK2Node_StructMemberSet::AllocateDefaultPins will hide any that
			// are PinHiddenByDefault, so for this instance we dont want that as users may have exposed some pins
			struct FAssignmentNodeOptionalPinManager : public FOptionalPinManager
			{
				// FOptionalPinManager Interface
				virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override
				{
					// Pins are always visible
					Record.bCanToggleVisibility = true;
					Record.bShowPin = true;
				}
			};

			FAssignmentNodeOptionalPinManager OptionalPinManager;
			OptionalPinManager.RebuildPropertyList(AssignmentNode->ShowPinForProperties, AssignmentNode->StructType);
			OptionalPinManager.CreateVisiblePins(AssignmentNode->ShowPinForProperties, AssignmentNode->StructType, EGPD_Input, AssignmentNode);

			Record.CustomEventNodes.Add(AssignmentNode);
		}

		// If we have folded properties we will need to set members on the classes generated mutable data block
		const FStructProperty* MutableDataProperty = InCompilationContext.GetMutableDataProperty();
		UK2Node_StructMemberSet* InstanceAssignmentNode = nullptr;
		if(InCompilationContext.IsAnimGraphNodeFolded(InNode) && MutableDataProperty != nullptr)
		{
			BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateInstanceAssignmentNode);
			
			InstanceAssignmentNode = InCompilationContext.SpawnIntermediateNode<UK2Node_StructMemberSet>(InNode, InCompilationContext.GetConsolidatedEventGraph());
			InstanceAssignmentNode->VariableReference.SetSelfMember(MutableDataProperty->GetFName());
			InstanceAssignmentNode->StructType = MutableDataProperty->Struct;

			// We build this struct member set node using specialized logic to optimize its creation
			// as it can have 1000's of properties harvested from animation nodes
			InstanceAssignmentNode->AllocateExecPins();

			{
				FInternalOptionalPinManager OptionalPinManager(InNode, InNode->GetFNodeProperty(), InCompilationContext);
				{
					BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateInstanceAssignmentNode_BuildPropertyList);
					OptionalPinManager.BuildPropertyList(InstanceAssignmentNode->ShowPinForProperties, MutableDataProperty->Struct);
				}
				{
					BP_SCOPED_COMPILER_EVENT_STAT(EAnimBlueprintCompilerStats_CreateEvaluationHandler_CreateInstanceAssignmentNode_CreateVisiblePins);
					OptionalPinManager.CreateVisiblePinsEx(InstanceAssignmentNode->ShowPinForProperties, MutableDataProperty->Struct, EGPD_Input, InstanceAssignmentNode);
				}
			}
			
			Record.CustomEventNodes.Add(InstanceAssignmentNode);
		}

		// Run thru each property
		TSet<FName> PropertiesBeingSet;

		for (auto TargetPinIt = AssignmentNode->Pins.CreateIterator(); TargetPinIt; ++TargetPinIt)
		{
			UEdGraphPin* TargetPin = *TargetPinIt;
			FName PropertyName(TargetPin->PinName);

			// Does it get serviced by this handler?
			if (FAnimNodeSinglePropertyHandler* SourceInfo = Record.ServicedProperties.Find(PropertyName))
			{
				// Skip if the property is folded, we should have handled it above
				const IAnimBlueprintCompilationContext::FFoldedPropertyRecord* FoldedPropertyRecord = InCompilationContext.GetFoldedPropertyRecord(InNode, PropertyName);

				if(FoldedPropertyRecord != nullptr)
				{
					// We only support per-instance members here.
					if(!FoldedPropertyRecord->bIsOnClass)
					{
						// We must have created an assignment node for the mutable data block by now
						check(InstanceAssignmentNode != nullptr);

						// Redirect to the instance's mutable data area assignment node
						PropertyName = FoldedPropertyRecord->GeneratedProperty->GetFName();

						// We dont need to handle arrays with a member set, as they use a member get-by-ref
						if(!TargetPin->PinType.IsArray())
						{
							TargetPin = InstanceAssignmentNode->FindPinChecked(FoldedPropertyRecord->GeneratedProperty->GetFName());
						}
					}
				}

				if (TargetPin->PinType.IsArray())
				{
					// Grab the array that we need to set members for
					UK2Node_StructMemberGet* FetchArrayNode = InCompilationContext.SpawnIntermediateNode<UK2Node_StructMemberGet>(InNode, InCompilationContext.GetConsolidatedEventGraph());
					FetchArrayNode->VariableReference.SetSelfMember(FoldedPropertyRecord ? MutableDataProperty->GetFName() : Record.NodeVariableProperty->GetFName());
					FetchArrayNode->StructType = FoldedPropertyRecord ? MutableDataProperty->Struct : Record.NodeVariableProperty->Struct;
					FetchArrayNode->AllocatePinsForSingleMemberGet(PropertyName);
					Record.CustomEventNodes.Add(FetchArrayNode);

					UEdGraphPin* ArrayVariableNode = FetchArrayNode->FindPin(PropertyName);

					if (SourceInfo->CopyRecords.Num() > 0)
					{
						// Set each element in the array
						for (FPropertyCopyRecord& CopyRecord : SourceInfo->CopyRecords)
						{
							int32 ArrayIndex = CopyRecord.DestArrayIndex;
							if(UEdGraphPin* DestPin = CopyRecord.DestPin)
							{
								// Create an array element set node
								UK2Node_CallArrayFunction* ArrayNode = InCompilationContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(InNode, InCompilationContext.GetConsolidatedEventGraph());
								ArrayNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Set), UKismetArrayLibrary::StaticClass());
								ArrayNode->AllocateDefaultPins();
								Record.CustomEventNodes.Add(ArrayNode);

								// Connect the execution chain
								ExecChain->MakeLinkTo(ArrayNode->GetExecPin());
								ExecChain = ArrayNode->GetThenPin();

								// Connect the input array
								UEdGraphPin* TargetArrayPin = ArrayNode->FindPinChecked(TEXT("TargetArray"));
								TargetArrayPin->MakeLinkTo(ArrayVariableNode);
								ArrayNode->PinConnectionListChanged(TargetArrayPin);

								// Set the array index
								UEdGraphPin* TargetIndexPin = ArrayNode->FindPinChecked(TEXT("Index"));
								TargetIndexPin->DefaultValue = FString::FromInt(ArrayIndex);

								// Wire up the data input
								UEdGraphPin* TargetItemPin = ArrayNode->FindPinChecked(TEXT("Item"));
								TargetItemPin->CopyPersistentDataFromOldPin(*DestPin);
								InCompilationContext.GetMessageLog().NotifyIntermediatePinCreation(TargetItemPin, DestPin);
							}
						}
					}
				}
				else
				{
					// Single property
					if (SourceInfo->CopyRecords.Num() > 0 && SourceInfo->CopyRecords[0].DestPin != nullptr)
					{
						UEdGraphPin* DestPin = SourceInfo->CopyRecords[0].DestPin;

						PropertiesBeingSet.Add(PropertyName);
						TargetPin->CopyPersistentDataFromOldPin(*DestPin);
						InCompilationContext.GetMessageLog().NotifyIntermediatePinCreation(TargetPin, DestPin);
					}
				}
			}
		}

		// Remove any unused pins from the assignment nodes to avoid smashing constant values
		bool bAnyNodePropertiesSet = false;
		for (int32 PinIndex = 0; PinIndex < AssignmentNode->ShowPinForProperties.Num(); ++PinIndex)
		{
			FOptionalPinFromProperty& TestProperty = AssignmentNode->ShowPinForProperties[PinIndex];
			TestProperty.bShowPin = PropertiesBeingSet.Contains(TestProperty.PropertyName);
			bAnyNodePropertiesSet |= TestProperty.bShowPin;
		}

		if(bAnyNodePropertiesSet)
		{
			AssignmentNode->ReconstructNode();
			
			UEdGraphPin* ExecVariablesIn = K2Schema->FindExecutionPin(*AssignmentNode, EGPD_Input);
			ExecChain->MakeLinkTo(ExecVariablesIn);
			ExecChain = K2Schema->FindExecutionPin(*AssignmentNode, EGPD_Output);
		}

		if (InstanceAssignmentNode != nullptr)
		{
			for (int32 PinIndex = 0; PinIndex < InstanceAssignmentNode->ShowPinForProperties.Num(); ++PinIndex)
			{
				FOptionalPinFromProperty& TestProperty = InstanceAssignmentNode->ShowPinForProperties[PinIndex];
				if (TestProperty.bShowPin)
				{
					UEdGraphPin* ExecVariablesIn = K2Schema->FindExecutionPin(*InstanceAssignmentNode, EGPD_Input);
					ExecChain->MakeLinkTo(ExecVariablesIn);
					ExecChain = K2Schema->FindExecutionPin(*InstanceAssignmentNode, EGPD_Output);
					break;
				}
			}
		}
	}
}

void UAnimBlueprintExtension_Base::RedirectPropertyAccesses(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InNode, FEvaluationHandlerRecord& InRecord)
{
	const FStructProperty* MutableDataProperty = InCompilationContext.GetMutableDataProperty();
	if (InCompilationContext.IsAnimGraphNodeFolded(InNode) && MutableDataProperty != nullptr)
	{
		for (TPair<FName, FAnimNodeSinglePropertyHandler>& NamePropertyPair : InRecord.ServicedProperties)
		{
			if (const IAnimBlueprintCompilationContext::FFoldedPropertyRecord* FoldedPropertyRecord = InCompilationContext.GetFoldedPropertyRecord(InNode, NamePropertyPair.Key))
			{
				for (FPropertyCopyRecord& CopyRecord : NamePropertyPair.Value.CopyRecords)
				{
					if (CopyRecord.DestPropertyPath.Num() > 1)
					{
						// If this record writes to the node, switch it to the mutable data's property instead
						if (CopyRecord.DestPropertyPath[0] == InRecord.NodeVariableProperty->GetName())
						{
							CopyRecord.DestPropertyPath[0] = MutableDataProperty->GetName();

							FString DestPropertyPathTail = CopyRecord.DestPropertyPath[1];
							FString DestPropertyPathWithoutArray = DestPropertyPathTail;
							FString ArrayIndex;
							int32 ArrayDelim = INDEX_NONE;
							if (DestPropertyPathTail.FindChar(TEXT('['), ArrayDelim))
							{
								DestPropertyPathWithoutArray = DestPropertyPathTail.Left(ArrayDelim);
								ArrayIndex = DestPropertyPathTail.RightChop(ArrayDelim);
							}

							// Switch the destination property from the node's property to the generated one
							if (DestPropertyPathWithoutArray == FoldedPropertyRecord->Property->GetName())
							{
								CopyRecord.DestPropertyPath[1] = FoldedPropertyRecord->GeneratedProperty->GetName() + ArrayIndex;
							}
						}
					}
				}
			}
		}
	}
}

void UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::PatchAnimNodeExposedValueHandler(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext) const
{
	UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(const_cast<UClass*>(InClass));
	FStructProperty* HandlerProperty = CastFieldChecked<FStructProperty>(InCompilationContext.GetAllocatedHandlerPropertiesByNode().FindChecked(AnimGraphNode));
	check(HandlerProperty->GetOwner<UScriptStruct>() == AnimClass->GetSparseClassDataStruct());
	void* ConstantNodeData = const_cast<void*>(AnimClass->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull));
	check(ConstantNodeData);

	if (HandlerProperty->Struct == FAnimNodeExposedValueHandler_PropertyAccess::StaticStruct())
	{
		FAnimNodeExposedValueHandler_PropertyAccess* Handler = HandlerProperty->ContainerPtrToValuePtr<FAnimNodeExposedValueHandler_PropertyAccess>(ConstantNodeData);

		Handler->CopyRecords.Empty();

		for (const TPair<FName, FAnimNodeSinglePropertyHandler>& ServicedPropPair : ServicedProperties)
		{
			const FName& PropertyName = ServicedPropPair.Key;
			const FAnimNodeSinglePropertyHandler& PropertyHandler = ServicedPropPair.Value;

			for (const FPropertyCopyRecord& PropertyCopyRecord : PropertyHandler.CopyRecords)
			{
				// Only unbatched copies can be processed on a per-node basis
				// Skip invalid copy indices as these are usually the result of BP errors/warnings
				if (PropertyCopyRecord.LibraryCompiledHandle.IsValid() && PropertyCopyRecord.LibraryCompiledHandle.GetBatchId() == (int32)EAnimPropertyAccessCallSite::WorkerThread_Unbatched)
				{
					Handler->CopyRecords.Emplace(PropertyCopyRecord.LibraryCompiledHandle.GetId(), PropertyCopyRecord.Operation, PropertyCopyRecord.bOnlyUpdateWhenActive);
				}
			}
		}
	}

	if (HandlerProperty->Struct->IsChildOf(FAnimNodeExposedValueHandler_Base::StaticStruct()))
	{
		FAnimNodeExposedValueHandler_Base* Handler = HandlerProperty->ContainerPtrToValuePtr<FAnimNodeExposedValueHandler_Base>(ConstantNodeData);

		if (!IsFastPath())
		{
			// not all of our pins use copy records so we will need to call our exposed value handler
			Handler->BoundFunction = HandlerFunctionName;
		}
	}
}

static UEdGraphPin* FindFirstInputPin(UEdGraphNode* InNode)
{
	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();

	for(UEdGraphPin* Pin : InNode->Pins)
	{
		if(Pin && Pin->Direction == EGPD_Input && !Schema->IsExecPin(*Pin) && !Schema->IsSelfPin(*Pin))
		{
			return Pin;
		}
	}

	return nullptr;
}

static bool ForEachInputPin(UEdGraphNode* InNode, TFunctionRef<bool(UEdGraphPin*)> InFunction)
{
	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();
	bool bResult = false;

	for(UEdGraphPin* Pin : InNode->Pins)
	{
		if(Pin && Pin->Direction == EGPD_Input && !Schema->IsExecPin(*Pin) && !Schema->IsSelfPin(*Pin))
		{
			bResult |= InFunction(Pin);
		}
	}

	return bResult;
}

static UEdGraphNode* FollowKnots(UEdGraphPin* FromPin, UEdGraphPin*& ToPin)
{
	if (FromPin->LinkedTo.Num() == 0)
	{
		return nullptr;
	}

	UEdGraphPin* LinkedPin = FromPin->LinkedTo[0];
	ToPin = LinkedPin;
	if(LinkedPin)
	{
		UEdGraphNode* LinkedNode = LinkedPin->GetOwningNode();
		UK2Node_Knot* KnotNode = Cast<UK2Node_Knot>(LinkedNode);
		while(KnotNode)
		{
			if(UEdGraphPin* InputPin = FindFirstInputPin(KnotNode))
			{
				if (InputPin->LinkedTo.Num() > 0 && InputPin->LinkedTo[0])
				{
					ToPin = InputPin->LinkedTo[0];
					LinkedNode = InputPin->LinkedTo[0]->GetOwningNode();
					KnotNode = Cast<UK2Node_Knot>(LinkedNode);
				}
				else
				{
					KnotNode = nullptr;
				}
			}
		}
		return LinkedNode;
	}

	return nullptr;
}

void UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::RegisterPin(UEdGraphPin* DestPin, FProperty* AssociatedProperty, int32 AssociatedPropertyArrayIndex, bool bAllowFastPath)
{
	FAnimNodeSinglePropertyHandler& Handler = ServicedProperties.FindOrAdd(AssociatedProperty->GetFName());

	TArray<FString> DestPropertyPath;

	// Prepend the destination property with the node's member property if the property is not on a UClass
	if(Cast<UClass>(AssociatedProperty->Owner.ToUObject()) == nullptr)
	{
		DestPropertyPath.Add(NodeVariableProperty->GetName());
	}

	if(AssociatedPropertyArrayIndex != INDEX_NONE)
	{
		DestPropertyPath.Add(FString::Printf(TEXT("%s[%d]"), *AssociatedProperty->GetName(), AssociatedPropertyArrayIndex));
	}
	else
	{
		DestPropertyPath.Add(AssociatedProperty->GetName());
	}

	Handler.CopyRecords.Emplace(DestPin, AssociatedProperty, AssociatedPropertyArrayIndex, MoveTemp(DestPropertyPath));
	Handler.CopyRecords.Last().bIsFastPath = bAllowFastPath;
}

void UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::RegisterPropertyBinding(FProperty* InProperty, const FAnimGraphNodePropertyBinding& InBinding)
{
	FAnimNodeSinglePropertyHandler& Handler = ServicedProperties.FindOrAdd(InProperty->GetFName());

	TArray<FString> DestPropertyPath;

	// Prepend the destination property with the node's member property if the property is not on a UClass
	if(Cast<UClass>(InProperty->Owner.ToUObject()) == nullptr)
	{
		Handler.bInstanceIsTarget = false;
		DestPropertyPath.Add(NodeVariableProperty->GetName());
	}
	else
	{
		Handler.bInstanceIsTarget = true;
	}

	if(InBinding.ArrayIndex != INDEX_NONE)
	{
		DestPropertyPath.Add(FString::Printf(TEXT("%s[%d]"), *InProperty->GetName(), InBinding.ArrayIndex));
	}
	else
	{
		DestPropertyPath.Add(InProperty->GetName());
	}
	
	FPropertyCopyRecord& CopyRecord = Handler.CopyRecords.Emplace_GetRef(InBinding.PropertyPath, DestPropertyPath);
	CopyRecord.DestProperty = InProperty;
	CopyRecord.DestArrayIndex = InBinding.ArrayIndex;
	CopyRecord.BindingContextId = InBinding.ContextId;
	CopyRecord.bOnlyUpdateWhenActive = InBinding.bOnlyUpdateWhenActive;
}

void UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::BuildFastPathCopyRecords(IAnimBlueprintPostExpansionStepContext& InCompilationContext)
{
	typedef bool (UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::*GraphCheckerFunc)(FCopyRecordGraphCheckContext&, UEdGraphPin*);

	GraphCheckerFunc GraphCheckerFuncs[] =
	{
		&UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForSplitPinAccess,
		&UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForVariableGet,
		&UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForLogicalNot,
		&UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForStructMemberAccess,
		&UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForArrayAccess,
	};

	if (GetDefault<UEngine>()->bOptimizeAnimBlueprintMemberVariableAccess)
	{
		for (TPair<FName, FAnimNodeSinglePropertyHandler>& ServicedPropPair : ServicedProperties)
		{
			TArray<FPropertyCopyRecord> AllAdditionalCopyRecords;

			for (FPropertyCopyRecord& CopyRecord : ServicedPropPair.Value.CopyRecords)
			{
				if(CopyRecord.SourcePropertyPath.Num() == 0)
				{
					TArray<FPropertyCopyRecord> AdditionalCopyRecords;

					FCopyRecordGraphCheckContext Context(CopyRecord, AdditionalCopyRecords, InCompilationContext.GetMessageLog());

					for (GraphCheckerFunc& CheckFunc : GraphCheckerFuncs)
					{
						if ((this->*CheckFunc)(Context, CopyRecord.DestPin))
						{
							break;
						}
					}

					if(AdditionalCopyRecords.Num() > 0)
					{
						for(FPropertyCopyRecord& AdditionalCopyRecord : AdditionalCopyRecords)
						{
							CheckForMemberOnlyAccess(AdditionalCopyRecord, AdditionalCopyRecord.DestPin);
						}

						CopyRecord = AdditionalCopyRecords[0];

						for(int32 AdditionalRecordIndex = 1; AdditionalRecordIndex < AdditionalCopyRecords.Num(); ++AdditionalRecordIndex)
						{
							AllAdditionalCopyRecords.Add(AdditionalCopyRecords[AdditionalRecordIndex]);
						}
					}
					else
					{
						CheckForMemberOnlyAccess(CopyRecord, CopyRecord.DestPin);
					}
				}
			}

			// Append any additional copy records
			ServicedPropPair.Value.CopyRecords.Append(AllAdditionalCopyRecords);
		}
	}
}

static void GetFullyQualifiedPathFromPin(const UEdGraphPin* Pin, TArray<FString>& OutPath)
{
	FString PinName = Pin->PinName.ToString();
	while (Pin->ParentPin != nullptr)
	{
		PinName[Pin->ParentPin->PinName.GetStringLength()] = TEXT('.');
		Pin = Pin->ParentPin;
	}

	UE::String::ParseTokens(PinName, TEXT('.'), [&OutPath](FStringView InStringView)
	{
		OutPath.Add(FString(InStringView));
	});
}

bool UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForVariableGet(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin)
{
	if(DestPin)
	{
		UEdGraphPin* SourcePin = nullptr;
		if(UK2Node_VariableGet* VariableGetNode = Cast<UK2Node_VariableGet>(FollowKnots(DestPin, SourcePin)))
		{
			if(VariableGetNode && VariableGetNode->IsNodePure() && VariableGetNode->VariableReference.IsSelfContext())
			{
				if(SourcePin)
				{
					GetFullyQualifiedPathFromPin(SourcePin, Context.CopyRecord->SourcePropertyPath);
					return true;
				}
			}
		}
	}

	return false;
}

bool UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForLogicalNot(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin)
{
	if(DestPin)
	{
		UEdGraphPin* SourcePin = nullptr;
		UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(FollowKnots(DestPin, SourcePin));
		if(CallFunctionNode && CallFunctionNode->FunctionReference.GetMemberName() == FName(TEXT("Not_PreBool")))
		{
			// find and follow input pin
			if(UEdGraphPin* InputPin = FindFirstInputPin(CallFunctionNode))
			{
				check(InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
				if(CheckForVariableGet(Context, InputPin) || CheckForStructMemberAccess(Context, InputPin) || CheckForArrayAccess(Context, InputPin))
				{
					check(Context.CopyRecord->SourcePropertyPath.Num() > 0);	// this should have been filled in by CheckForVariableGet() or CheckForStructMemberAccess() above
					Context.CopyRecord->Operation = EPostCopyOperation::LogicalNegateBool;
					return true;
				}
			}
		}
	}

	return false;
}

/** The functions that we can safely native-break */
static const FName NativeBreakFunctionNameAllowList[] =
{
	FName(TEXT("BreakVector")),
	FName(TEXT("BreakVector2D")),
	FName(TEXT("BreakRotator")),
};

/** Check whether a native break function can be safely used in the fast-path copy system (ie. source and dest data will be the same) */
static bool IsNativeBreakAllowed(const FName& InFunctionName)
{
	for(const FName& FunctionName : NativeBreakFunctionNameAllowList)
	{
		if(InFunctionName == FunctionName)
		{
			return true;
		}
	}

	return false;
}

/** The functions that we can safely native-make */
static const FName NativeMakeFunctionNameAllowList[] =
{
	FName(TEXT("MakeVector")),
	FName(TEXT("MakeVector2D")),
	FName(TEXT("MakeRotator")),
};

/** Check whether a native break function can be safely used in the fast-path copy system (ie. source and dest data will be the same) */
static bool IsNativeMakeAllowed(const FName& InFunctionName)
{
	for(const FName& FunctionName : NativeMakeFunctionNameAllowList)
	{
		if(InFunctionName == FunctionName)
		{
			return true;
		}
	}

	return false;
}

bool UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForStructMemberAccess(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin)
{
	if(DestPin)
	{
		UEdGraphPin* SourcePin = nullptr;
		if(UK2Node_BreakStruct* BreakStructNode = Cast<UK2Node_BreakStruct>(FollowKnots(DestPin, SourcePin)))
		{
			if(UEdGraphPin* InputPin = FindFirstInputPin(BreakStructNode))
			{
				if(CheckForStructMemberAccess(Context, InputPin) || CheckForVariableGet(Context, InputPin) || CheckForArrayAccess(Context, InputPin))
				{
					check(Context.CopyRecord->SourcePropertyPath.Num() > 0);	// this should have been filled in by CheckForVariableGet() above
					Context.CopyRecord->SourcePropertyPath.Add(SourcePin->PinName.ToString());
					return true;
				}
			}
		}
		// could be a native break
		else if(UK2Node_CallFunction* NativeBreakNode = Cast<UK2Node_CallFunction>(FollowKnots(DestPin, SourcePin)))
		{
			UFunction* Function = NativeBreakNode->FunctionReference.ResolveMember<UFunction>(UKismetMathLibrary::StaticClass());
			if(Function && Function->HasMetaData(TEXT("NativeBreakFunc")) && IsNativeBreakAllowed(Function->GetFName()))
			{
				if(UEdGraphPin* InputPin = FindFirstInputPin(NativeBreakNode))
				{
					if(CheckForStructMemberAccess(Context, InputPin) || CheckForVariableGet(Context, InputPin) || CheckForArrayAccess(Context, InputPin))
					{
						check(Context.CopyRecord->SourcePropertyPath.Num() > 0);	// this should have been filled in by CheckForVariableGet() above
						Context.CopyRecord->SourcePropertyPath.Add(SourcePin->PinName.ToString());
						return true;
					}
				}
			}
		}
	}

	return false;
}

bool UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForSplitPinAccess(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin)
{
	if(DestPin)
	{
		FPropertyCopyRecord OriginalRecord = *Context.CopyRecord;

		UEdGraphPin* SourcePin = nullptr;
		if(UK2Node_MakeStruct* MakeStructNode = Cast<UK2Node_MakeStruct>(FollowKnots(DestPin, SourcePin)))
		{
			// Idea here is to account for split pins, so we want to narrow the scope to not also include user-placed makes
			UObject* SourceObject = Context.MessageLog.FindSourceObject(MakeStructNode);
			if(SourceObject && SourceObject->IsA<UAnimGraphNode_Base>())
			{
				return ForEachInputPin(MakeStructNode, [this, &Context, &OriginalRecord](UEdGraphPin* InputPin)
				{
					Context.CopyRecord->SourcePropertyPath = OriginalRecord.SourcePropertyPath;
					if(CheckForStructMemberAccess(Context, InputPin) || CheckForVariableGet(Context, InputPin) || CheckForArrayAccess(Context, InputPin))
					{
						check(Context.CopyRecord->DestPropertyPath.Num() > 0);
						FPropertyCopyRecord RecordCopy = *Context.CopyRecord;
						FPropertyCopyRecord& NewRecord = Context.AdditionalCopyRecords.Add_GetRef(MoveTemp(RecordCopy));

						NewRecord.DestPropertyPath = OriginalRecord.DestPropertyPath; 
						NewRecord.DestPropertyPath.Add(InputPin->PinName.ToString());
						return true;
					}

					return false;
				});
			}
		}
		else if(UK2Node_CallFunction* NativeMakeNode = Cast<UK2Node_CallFunction>(FollowKnots(DestPin, SourcePin)))
		{
			UFunction* Function = NativeMakeNode->FunctionReference.ResolveMember<UFunction>(UKismetMathLibrary::StaticClass());
			if(Function && Function->HasMetaData(TEXT("NativeMakeFunc")) && IsNativeMakeAllowed(Function->GetFName()))
			{
				// Idea here is to account for split pins, so we want to narrow the scope to not also include user-placed makes
				UObject* SourceObject = Context.MessageLog.FindSourceObject(NativeMakeNode);
				if(SourceObject && SourceObject->IsA<UAnimGraphNode_Base>())
				{
					return ForEachInputPin(NativeMakeNode, [this, &Context, &OriginalRecord](UEdGraphPin* InputPin)
					{
						Context.CopyRecord->SourcePropertyPath = OriginalRecord.SourcePropertyPath;
						if(CheckForStructMemberAccess(Context, InputPin) || CheckForVariableGet(Context, InputPin) || CheckForArrayAccess(Context, InputPin))
						{
							check(Context.CopyRecord->DestPropertyPath.Num() > 0);
							FPropertyCopyRecord RecordCopy = *Context.CopyRecord;
							FPropertyCopyRecord& NewRecord = Context.AdditionalCopyRecords.Add_GetRef(MoveTemp(RecordCopy));

							NewRecord.DestPropertyPath = OriginalRecord.DestPropertyPath;
							NewRecord.DestPropertyPath.Add(InputPin->PinName.ToString());
							return true;
						}

						return false;
					});
				}
			}
		}
	}

	return false;
}

bool UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForArrayAccess(FCopyRecordGraphCheckContext& Context, UEdGraphPin* DestPin)
{
	if(DestPin)
	{
		UEdGraphPin* SourcePin = nullptr;
		if(UK2Node_CallArrayFunction* CallArrayFunctionNode = Cast<UK2Node_CallArrayFunction>(FollowKnots(DestPin, SourcePin)))
		{
			if(CallArrayFunctionNode->GetTargetFunction() == UKismetArrayLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Get)))
			{
				// Check array index is constant
				int32 ArrayIndex = INDEX_NONE;
				if(UEdGraphPin* IndexPin = CallArrayFunctionNode->FindPin(TEXT("Index")))
				{
					if(IndexPin->LinkedTo.Num() > 0)
					{
						return false;
					}

					ArrayIndex = FCString::Atoi(*IndexPin->DefaultValue);
				}

				if(UEdGraphPin* TargetArrayPin = CallArrayFunctionNode->FindPin(TEXT("TargetArray")))
				{
					if(CheckForVariableGet(Context, TargetArrayPin) || CheckForStructMemberAccess(Context, TargetArrayPin))
					{
						check(Context.CopyRecord->SourcePropertyPath.Num() > 0);	// this should have been filled in by CheckForVariableGet() or CheckForStructMemberAccess() above
						Context.CopyRecord->SourcePropertyPath.Last().Append(FString::Printf(TEXT("[%d]"), ArrayIndex));
						return true;
					}
				}
			}

		
		}
	}

	return false;
}

bool UAnimBlueprintExtension_Base::FEvaluationHandlerRecord::CheckForMemberOnlyAccess(FPropertyCopyRecord& CopyRecord, UEdGraphPin* DestPin)
{
	const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();

	if(DestPin)
	{
		// traverse pins to leaf nodes and check for member access/pure only
		TArray<UEdGraphPin*> PinStack;
		PinStack.Add(DestPin);
		while(PinStack.Num() > 0)
		{
			UEdGraphPin* CurrentPin = PinStack.Pop(EAllowShrinking::No);
			for(auto& LinkedPin : CurrentPin->LinkedTo)
			{
				UEdGraphNode* LinkedNode = LinkedPin->GetOwningNode();
				if(LinkedNode)
				{
					bool bLeafNode = true;
					for(auto& Pin : LinkedNode->Pins)
					{
						if(Pin != LinkedPin && Pin->Direction == EGPD_Input && !AnimGraphDefaultSchema->IsPosePin(Pin->PinType))
						{
							bLeafNode = false;
							PinStack.Add(Pin);
						}
					}

					if(bLeafNode)
					{
						if(UK2Node_VariableGet* LinkedVariableGetNode = Cast<UK2Node_VariableGet>(LinkedNode))
						{
							if(!LinkedVariableGetNode->IsNodePure() || !LinkedVariableGetNode->VariableReference.IsSelfContext())
							{
								// only local variable access is allowed for leaf nodes 
								CopyRecord.InvalidateFastPath();
							}
						}
						else if(UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(LinkedNode))
						{
							if(!CallFunctionNode->IsNodePure())
							{
								// only allow pure function calls
								CopyRecord.InvalidateFastPath();
							}
						}
						else if(!LinkedNode->IsA<UK2Node_TransitionRuleGetter>())
						{
							CopyRecord.InvalidateFastPath();
						}
					}
				}
			}
		}
	}

	return CopyRecord.IsFastPath();
}

#undef LOCTEXT_NAMESPACE
