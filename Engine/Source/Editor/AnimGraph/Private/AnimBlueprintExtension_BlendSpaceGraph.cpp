// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_BlendSpaceGraph.h"
#include "Animation/BlendSpace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_BlendSpaceGraph)

void UAnimBlueprintExtension_BlendSpaceGraph::HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	Subsystem.BlendSpaces.Empty();
	Class = const_cast<UClass*>(InClass);
}

UBlendSpace* UAnimBlueprintExtension_BlendSpaceGraph::AddBlendSpace(UBlendSpace* InSourceBlendSpace)
{
	UBlendSpace* CopiedBlendSpace = DuplicateObject(InSourceBlendSpace, Class);
	CopiedBlendSpace->ClearFlags(RF_Transient);

	// RF_Public is required because this blendspace may need to be referenced in child classes
	CopiedBlendSpace->SetFlags(RF_Public);
	
	Subsystem.BlendSpaces.Add(CopiedBlendSpace);
	return CopiedBlendSpace;
}
