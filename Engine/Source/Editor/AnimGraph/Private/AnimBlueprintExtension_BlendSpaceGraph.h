// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_Base.h"
#include "AnimBlueprintExtension.h"
#include "Animation/AnimSubsystem_BlendSpaceGraph.h"
#include "AnimBlueprintExtension_BlendSpaceGraph.generated.h"

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_BlendSpaceGraph : public UAnimBlueprintExtension
{
	GENERATED_BODY()

	friend class UAnimGraphNode_BlendSpaceGraphBase;
	
	// UAnimBlueprintExtension interface
	virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

	// Add a class-internal blendspace
	// @param	InSourceBlendSpace	The blendspace to duplicate
	// @return a duplicate of the blendspace outered to the class
	UBlendSpace* AddBlendSpace(UBlendSpace* InSourceBlendSpace);

private:
	// The class that is being compiled
	UPROPERTY(Transient)
	TObjectPtr<UClass> Class;

private:
	UPROPERTY()
	FAnimSubsystem_BlendSpaceGraph Subsystem;
};