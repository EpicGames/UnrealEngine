// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_CachedPose.h"
#include "AnimGraphNode_SaveCachedPose.h"
#include "AnimGraphNode_Root.h"
#include "AnimGraphNode_UseCachedPose.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimGraphNode_StateMachine.h"
#include "AnimGraphNode_StateResult.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "IAnimBlueprintCompilationContext.h"
#include "AnimationGraphSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_CachedPose)

void UAnimBlueprintExtension_CachedPose::HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	SaveCachedPoseNodes.Empty();
}

void UAnimBlueprintExtension_CachedPose::HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	for(UAnimGraphNode_Base* Node : InAnimNodes)
	{
		if(UAnimGraphNode_SaveCachedPose* SavePoseRoot = Cast<UAnimGraphNode_SaveCachedPose>(Node))
		{
			//@TODO: Ideally we only add these if there is a UseCachedPose node referencing them, but those can be anywhere and are hard to grab
			SaveCachedPoseNodes.Add(SavePoseRoot->CacheName, SavePoseRoot);
		}
	}
}

void UAnimBlueprintExtension_CachedPose::HandlePostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	// Build cached pose map
	BuildCachedPoseNodeUpdateOrder(InCompilationContext, OutCompiledData);
}

TAutoConsoleVariable<int32> CVarAnimDebugCachePoseNodeUpdateOrder(TEXT("a.Compiler.CachePoseNodeUpdateOrderDebug.Enable"), 0, TEXT("Toggle debugging for CacheNodeUpdateOrder debug during AnimBP compilation"));

void UAnimBlueprintExtension_CachedPose::BuildCachedPoseNodeUpdateOrder(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	TArray<UAnimGraphNode_Root*> RootNodes;
	InCompilationContext.GetConsolidatedEventGraph()->GetNodesOfClass<UAnimGraphNode_Root>(RootNodes);

	// State results are also "root" nodes, need to find the true roots
	RootNodes.RemoveAll([](UAnimGraphNode_Root* InPossibleRootNode)
	{
		return InPossibleRootNode->GetClass() != UAnimGraphNode_Root::StaticClass();
	});

	const bool bEnableDebug = (CVarAnimDebugCachePoseNodeUpdateOrder.GetValueOnAnyThread() == 1);

	for(UAnimGraphNode_Root* RootNode : RootNodes)
	{
		TArray<UAnimGraphNode_SaveCachedPose*> OrderedSavePoseNodes;

		TArray<UAnimGraphNode_Base*> VisitedRootNodes;
	
		UE_CLOGF(bEnableDebug, LogAnimation, Display, "CachePoseNodeOrdering BEGIN"); 

		CachePoseNodeOrdering_StartNewTraversal(InCompilationContext, RootNode, OrderedSavePoseNodes, VisitedRootNodes);

		UE_CLOGF(bEnableDebug, LogAnimation, Display, "CachePoseNodeOrdering END");

		if (bEnableDebug)
		{
			UE_LOGF(LogAnimation, Display, "Ordered Save Pose Node List:");
			for (UAnimGraphNode_SaveCachedPose* SavedPoseNode : OrderedSavePoseNodes)
			{
				UE_LOGF(LogAnimation, Display, "\t%ls", *SavedPoseNode->Node.CachePoseName.ToString())
			}
			UE_LOGF(LogAnimation, Display, "End List");
		}

		FCachedPoseIndices& OrderedSavedPoseIndices = OutCompiledData.GetOrderedSavedPoseIndicesMap().FindOrAdd(RootNode->Node.GetName());

		for(UAnimGraphNode_SaveCachedPose* PoseNode : OrderedSavePoseNodes)
		{
			if(const int32* NodeIndex = InCompilationContext.GetAllocatedAnimNodeIndices().Find(PoseNode))
			{
				OrderedSavedPoseIndices.OrderedSavedPoseNodeIndices.Add(*NodeIndex);
			}
			else
			{
				InCompilationContext.GetMessageLog().Error(TEXT("Failed to find index for a saved pose node while building ordered pose list."));
			}
		}
	}
}

void UAnimBlueprintExtension_CachedPose::CachePoseNodeOrdering_StartNewTraversal(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InRootNode, TArray<UAnimGraphNode_SaveCachedPose*> &OrderedSavePoseNodes, TArray<UAnimGraphNode_Base*> VisitedRootNodes)
{
	check(InRootNode);
	UAnimGraphNode_SaveCachedPose* RootCacheNode = Cast<UAnimGraphNode_SaveCachedPose>(InRootNode);
	FString RootName = RootCacheNode ? RootCacheNode->CacheName : InRootNode->GetName();

	const bool bEnableDebug = (CVarAnimDebugCachePoseNodeUpdateOrder.GetValueOnAnyThread() == 1);

	UE_CLOGF(bEnableDebug, LogAnimation, Display, "StartNewTraversal %ls", *RootName);

	// Track which root nodes we've visited to prevent infinite recursion.
	VisitedRootNodes.Add(InRootNode);

	// Need a list of only what we find here to recurse, we can't do that with the total list
	TArray<UAnimGraphNode_SaveCachedPose*> InternalOrderedNodes;

	// Traverse whole graph from root collecting SaveCachePose nodes we've touched.
	CachePoseNodeOrdering_TraverseInternal(InCompilationContext, InRootNode, InternalOrderedNodes);

	// Process nodes that we've touched 
	UE_CLOGF(bEnableDebug, LogAnimation, Display, "Process Queue for %ls", *RootName);
	for (UAnimGraphNode_SaveCachedPose* QueuedCacheNode : InternalOrderedNodes)
	{
		if (VisitedRootNodes.Contains(QueuedCacheNode))
		{
			UE_CLOGF(bEnableDebug, LogAnimation, Display, "Process Queue SaveCachePose %ls. ALREADY VISITED, INFINITE RECURSION DETECTED! SKIPPING", *QueuedCacheNode->CacheName);
			InCompilationContext.GetMessageLog().Error(*FString::Printf(TEXT("Infinite recursion detected with SaveCachePose %s and %s"), *RootName, *QueuedCacheNode->CacheName));
			continue;
		}
		else
		{
			OrderedSavePoseNodes.Remove(QueuedCacheNode);
			OrderedSavePoseNodes.Add(QueuedCacheNode);

			CachePoseNodeOrdering_StartNewTraversal(InCompilationContext, QueuedCacheNode, OrderedSavePoseNodes, VisitedRootNodes);
		}
	}

	UE_CLOGF(bEnableDebug, LogAnimation, Display, "EndNewTraversal %ls", *RootName);
}

void UAnimBlueprintExtension_CachedPose::CachePoseNodeOrdering_TraverseInternal_SubGraph(IAnimBlueprintCompilationContext& InCompilationContext, UEdGraph* InGraph, TArray<UAnimGraphNode_SaveCachedPose*> &OrderedSavePoseNodes)
{
	const bool bEnableDebug = (CVarAnimDebugCachePoseNodeUpdateOrder.GetValueOnAnyThread() == 1);
	UE_CLOGF(bEnableDebug, LogAnimation, Display, "ProcessGraph %ls", *InGraph->GetName());	

	if(InGraph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()))
	{
		TArray<UAnimGraphNode_Base*> PotentialRootNodes;
		InGraph->GetNodesOfClass(PotentialRootNodes);

		for (UAnimGraphNode_Base* PotentialRootNode : PotentialRootNodes)
		{
			if(PotentialRootNode->IsNodeRootSet())
			{
				CachePoseNodeOrdering_TraverseInternal(InCompilationContext, PotentialRootNode, OrderedSavePoseNodes);
			}
		}
	}
	else
	{
		for(UEdGraph* SubGraph : InGraph->SubGraphs)
		{
			CachePoseNodeOrdering_TraverseInternal_SubGraph(InCompilationContext, SubGraph, OrderedSavePoseNodes);
		}
	}
}

void UAnimBlueprintExtension_CachedPose::CachePoseNodeOrdering_TraverseInternal(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InAnimGraphNode, TArray<UAnimGraphNode_SaveCachedPose*> &OrderedSavePoseNodes)
{
	TArray<UAnimGraphNode_Base*> LinkedAnimNodes;
	InCompilationContext.GetLinkedAnimNodes(InAnimGraphNode, LinkedAnimNodes);

	const bool bEnableDebug = (CVarAnimDebugCachePoseNodeUpdateOrder.GetValueOnAnyThread() == 1);

	for (UAnimGraphNode_Base* LinkedNode : LinkedAnimNodes)
	{
		UE_CLOGF(bEnableDebug, LogAnimation, Display, "\t Processing %ls", *LinkedNode->GetName());
		if (UAnimGraphNode_UseCachedPose* UsePoseNode = Cast<UAnimGraphNode_UseCachedPose>(LinkedNode))
		{
			if (UAnimGraphNode_SaveCachedPose* SaveNode = UsePoseNode->SaveCachedPoseNode.Get())
			{
				UE_CLOGF(bEnableDebug, LogAnimation, Display, "\t Queueing SaveCachePose %ls", *SaveNode->CacheName);

				// Requeue the node we found
				OrderedSavePoseNodes.Remove(SaveNode);
				OrderedSavePoseNodes.Add(SaveNode);
			}
		}
		else
		{
			TArray<UEdGraph*> SubGraphs = LinkedNode->GetSubGraphs();
			if(SubGraphs.Num() > 0)
			{
				for (UEdGraph* SubGraph : SubGraphs)
				{
					CachePoseNodeOrdering_TraverseInternal_SubGraph(InCompilationContext, SubGraph, OrderedSavePoseNodes);
				}
			}
			else
			{
				CachePoseNodeOrdering_TraverseInternal(InCompilationContext, LinkedNode, OrderedSavePoseNodes);
			}
		}
	}
}

const TMap<FString, UAnimGraphNode_SaveCachedPose*>& UAnimBlueprintExtension_CachedPose::GetSaveCachedPoseNodes() const
{
	return SaveCachedPoseNodes;
}
