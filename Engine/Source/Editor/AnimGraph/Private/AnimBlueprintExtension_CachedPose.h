// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_Base.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_CachedPose.generated.h"

class UAnimGraphNode_SaveCachedPose;
class IAnimBlueprintCompilerCreationContext;
class IAnimBlueprintCompilationContext;
class IAnimBlueprintGeneratedClassCompiledData;
class IAnimBlueprintCompilationBracketContext;

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_CachedPose : public UAnimBlueprintExtension
{
	GENERATED_BODY()
	
public:
	// Get the map of cache name to encountered save cached pose nodes
	const TMap<FString, UAnimGraphNode_SaveCachedPose*>& GetSaveCachedPoseNodes() const;

private:
	virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandlePostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

	// Builds the update order list for saved pose nodes in this blueprint
	void BuildCachedPoseNodeUpdateOrder(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	// Traverses a graph to collect save pose nodes starting at InRootNode, then processes each node
	void CachePoseNodeOrdering_StartNewTraversal(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InRootNode, TArray<UAnimGraphNode_SaveCachedPose*> &OrderedSavePoseNodes, TArray<UAnimGraphNode_Base*> VisitedRootNodes);

	// Traverse a node's subgraph to collect save pose nodes
	void CachePoseNodeOrdering_TraverseInternal_SubGraph(IAnimBlueprintCompilationContext& InCompilationContext, UEdGraph* InGraph, TArray<UAnimGraphNode_SaveCachedPose*> &OrderedSavePoseNodes);
	
	// Traverses a graph to collect save pose nodes starting at InAnimGraphNode, does NOT process saved pose nodes afterwards
	void CachePoseNodeOrdering_TraverseInternal(IAnimBlueprintCompilationContext& InCompilationContext, UAnimGraphNode_Base* InAnimGraphNode, TArray<UAnimGraphNode_SaveCachedPose*> &OrderedSavePoseNodes);

private:
	// Map of cache name to encountered save cached pose nodes
	TMap<FString, UAnimGraphNode_SaveCachedPose*> SaveCachedPoseNodes;
};