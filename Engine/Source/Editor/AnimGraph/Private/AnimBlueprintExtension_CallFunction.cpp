// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_CallFunction.h"
#include "AnimGraphNode_Base.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_CallFunction)

void UAnimBlueprintExtension_CallFunction::HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	Counter = 0;
	CustomEventNames.Empty();
}

FName UAnimBlueprintExtension_CallFunction::AddCustomEventName(UAnimGraphNode_Base* InNode)
{
	static FName BaseName(TEXT("AnimNode_CallFunction_EventStub"));
	FName Name(BaseName, Counter++);
	CustomEventNames.Add(InNode, Name);

	return Name;
}

FName UAnimBlueprintExtension_CallFunction::FindCustomEventName(UAnimGraphNode_Base* InNode) const
{
	if(const FName* ExistingName = CustomEventNames.Find(InNode))
	{
		return *ExistingName;
	}
	return NAME_None;
}
