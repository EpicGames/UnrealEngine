// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_CallFunction.generated.h"

class UAnimGraphNode_Base;

UCLASS()
class UAnimBlueprintExtension_CallFunction : public UAnimBlueprintExtension
{
	GENERATED_BODY()

public:
	virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	
	// Add a custom event name for the specified node
	FName AddCustomEventName(UAnimGraphNode_Base* InNode);

	// Find the custom event name for the specified node. @return NAME_None if the node was not found
	FName FindCustomEventName(UAnimGraphNode_Base* InNode) const;

private:
	// Counter to allow us to create unique stub function names
	int32 Counter = 0;
	
	// Set of used custom event names per-node
	TMap<UAnimGraphNode_Base*, FName> CustomEventNames;
};