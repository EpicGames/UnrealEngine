// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_LinkedAnimGraph.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "IAnimBlueprintCompilationContext.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_LinkedAnimGraph)

void UAnimBlueprintExtension_LinkedAnimGraph::HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	for(UAnimGraphNode_Base* AnimNode : InAnimNodes)
	{
		if(UAnimGraphNode_LinkedAnimGraphBase* LinkedAnimGraphBase = Cast<UAnimGraphNode_LinkedAnimGraphBase>(AnimNode))
		{
			LinkedAnimGraphBase->AllocatePoseLinks();
		}
	}
}
