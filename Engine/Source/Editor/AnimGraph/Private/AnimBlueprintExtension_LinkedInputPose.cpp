// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_LinkedInputPose.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_LinkedInputPose.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "IAnimBlueprintCompilationContext.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_LinkedInputPose)

void UAnimBlueprintExtension_LinkedInputPose::HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	for(UAnimGraphNode_Base* AnimNode : InAnimNodes)
	{
		if(UAnimGraphNode_LinkedInputPose* LinkedInputPose = Cast<UAnimGraphNode_LinkedInputPose>(AnimNode))
		{
			LinkedInputPose->AnalyzeLinks(InAnimNodes);
		}
	}
}
