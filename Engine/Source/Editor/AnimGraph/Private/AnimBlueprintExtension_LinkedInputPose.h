// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_LinkedInputPose.generated.h"

class UAnimGraphNode_Base;
class IAnimBlueprintCompilerCreationContext;
class IAnimBlueprintCompilationContext;
class IAnimBlueprintGeneratedClassCompiledData;

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_LinkedInputPose : public UAnimBlueprintExtension
{
	GENERATED_BODY()

private:
	// UAnimBlueprintExtension interface
	virtual void HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
};