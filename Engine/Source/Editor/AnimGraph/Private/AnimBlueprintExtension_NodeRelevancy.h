// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintExtension.h"
#include "Animation/AnimSubsystem_NodeRelevancy.h"
#include "AnimBlueprintExtension_NodeRelevancy.generated.h"

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_NodeRelevancy : public UAnimBlueprintExtension
{
	GENERATED_BODY()

private:
	UPROPERTY()
	FAnimSubsystemInstance_NodeRelevancy Subsystem;
};