// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_PropertyAccess.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "Features/IModularFeatures.h"
#include "HAL/Platform.h"
#include "IAnimBlueprintCompilationBracketContext.h"
#include "IPropertyAccessCompiler.h"
#include "IPropertyAccessEditor.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_CallFunction.h"
#include "K2Node_GetArrayItem.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_PropertyAccess)

#define LOCTEXT_NAMESPACE "UAnimBlueprintExtension_PropertyAccess"

const FName UAnimBlueprintExtension_PropertyAccess::ContextId_Automatic(NAME_None);
const FName UAnimBlueprintExtension_PropertyAccess::ContextId_UnBatched_ThreadSafe(TEXT("UnBatched_ThreadSafe"));
const FName UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_WorkerThreadPreEventGraph(TEXT("Batched_WorkerThreadPreEventGraph"));
const FName UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_WorkerThreadPostEventGraph(TEXT("Batched_WorkerThreadPostEventGraph"));
const FName UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_GameThreadPreEventGraph(TEXT("Batched_GameThreadPreEventGraph"));
const FName UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_GameThreadPostEventGraph(TEXT("Batched_GameThreadPostEventGraph"));

FPropertyAccessHandle UAnimBlueprintExtension_PropertyAccess::AddCopy(TArrayView<FString> InSourcePath, TArrayView<FString> InDestPath, const FName& InContextId, UObject* InObject)
{
	if(PropertyAccessLibraryCompiler.IsValid())
	{
		return PropertyAccessLibraryCompiler->AddCopy(InSourcePath, InDestPath, InContextId, InObject);
	}
	return FPropertyAccessHandle();
}

FPropertyAccessHandle UAnimBlueprintExtension_PropertyAccess::AddAccess(TArrayView<FString> InPath, UObject* InObject)
{
	if (PropertyAccessLibraryCompiler.IsValid())
	{
		return PropertyAccessLibraryCompiler->AddAccess(InPath, InObject);
	}
	return FPropertyAccessHandle();
}

void UAnimBlueprintExtension_PropertyAccess::HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	FCompilerResultsLog& MessageLog = InCompilationContext.GetMessageLog();

	FPropertyAccessLibraryCompilerArgs Args(Subsystem.Library, InClass);
	Args.OnDetermineBatchId = FOnPropertyAccessDetermineBatchId::CreateLambda([&MessageLog](const FPropertyAccessCopyContext& InContext) -> int32
	{
		if(InContext.ContextId == ContextId_Automatic)
		{
			if(InContext.bSourceThreadSafe && InContext.bDestThreadSafe)
			{
				// Can only be in the worker thread batch if both endpoints are thread-safe
				return (int32)EAnimPropertyAccessCallSite::WorkerThread_Unbatched;
			}
			else
			{
				return (int32)EAnimPropertyAccessCallSite::GameThread_Batched_PreEventGraph;
			}
		}
		else if(InContext.ContextId == ContextId_UnBatched_ThreadSafe)
		{
			if(InContext.bSourceThreadSafe && InContext.bDestThreadSafe)
			{
				// Can only be in the worker thread batch if both endpoints are thread-safe
				return (int32)EAnimPropertyAccessCallSite::WorkerThread_Unbatched;
			}
			else
			{
				if(!InContext.bSourceThreadSafe)
				{
					MessageLog.Warning(*FText::Format(LOCTEXT("ThreadSafetyIncompatible", "@@ '{0}' is not thread-safe, access will be performed on the game thread (pre-event graph) and cached"), InContext.SourcePathAsText).ToString(), InContext.Object);
				}
				if(!InContext.bDestThreadSafe)
				{
					MessageLog.Warning(*FText::Format(LOCTEXT("ThreadSafetyIncompatible", "@@ '{0}' is not thread-safe, access will be performed on the game thread (pre-event graph) and cached"), InContext.DestPathAsText).ToString(), InContext.Object);
				}
				return (int32)EAnimPropertyAccessCallSite::GameThread_Batched_PreEventGraph;
			}
		}
		else if(InContext.ContextId == ContextId_Batched_GameThreadPreEventGraph)
		{
			return (int32)EAnimPropertyAccessCallSite::GameThread_Batched_PreEventGraph;
		}
		else if(InContext.ContextId == ContextId_Batched_GameThreadPostEventGraph)
		{
			return (int32)EAnimPropertyAccessCallSite::GameThread_Batched_PostEventGraph;
		}
		else if(InContext.ContextId == ContextId_Batched_WorkerThreadPreEventGraph)
		{
			if(InContext.bSourceThreadSafe && InContext.bDestThreadSafe)
			{
				return (int32)EAnimPropertyAccessCallSite::WorkerThread_Batched_PreEventGraph;
			}
			else
			{
				if(!InContext.bSourceThreadSafe)
				{
					MessageLog.Warning(*FText::Format(LOCTEXT("ThreadSafetyIncompatible", "@@ '{0}' is not thread-safe, access will be performed on the game thread (pre-event graph) and cached"), InContext.SourcePathAsText).ToString(), InContext.Object);
				}
				if(!InContext.bDestThreadSafe)
				{
					MessageLog.Warning(*FText::Format(LOCTEXT("ThreadSafetyIncompatible", "@@ '{0}' is not thread-safe, access will be performed on the game thread (pre-event graph) and cached"), InContext.DestPathAsText).ToString(), InContext.Object);
				}
				return (int32)EAnimPropertyAccessCallSite::GameThread_Batched_PreEventGraph;
			}
		}
		else if(InContext.ContextId == ContextId_Batched_WorkerThreadPostEventGraph)
		{
			if(InContext.bSourceThreadSafe && InContext.bDestThreadSafe)
			{
				return (int32)EAnimPropertyAccessCallSite::WorkerThread_Batched_PostEventGraph;
			}
			else
			{
				if(!InContext.bSourceThreadSafe)
				{
					MessageLog.Warning(*FText::Format(LOCTEXT("ThreadSafetyIncompatible", "@@ '{0}' is not thread-safe, access will be performed on the game thread (pre-event graph) and cached"), InContext.SourcePathAsText).ToString(), InContext.Object);
				}
				if(!InContext.bDestThreadSafe)
				{
					MessageLog.Warning(*FText::Format(LOCTEXT("ThreadSafetyIncompatible", "@@ '{0}' is not thread-safe, access will be performed on the game thread (pre-event graph) and cached"), InContext.DestPathAsText).ToString(), InContext.Object);
				}
				return (int32)EAnimPropertyAccessCallSite::GameThread_Batched_PreEventGraph;
			}
		}	
		else
		{
			MessageLog.Warning(*FText::Format(LOCTEXT("UnknownContext", "@@ has unknown context '{0}', access will be performed on the game thread (pre-event graph) and cached"), FText::FromName(InContext.ContextId)).ToString(), InContext.Object);
			return (int32)EAnimPropertyAccessCallSite::GameThread_Batched_PreEventGraph;
		}
	});
	
	PropertyAccessLibraryCompiler = PropertyAccessEditor.MakePropertyAccessCompiler(Args);
	PropertyAccessLibraryCompiler->BeginCompilation();
}

void UAnimBlueprintExtension_PropertyAccess::HandleFinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	OnPreLibraryCompiledDelegate.Broadcast();

	if(!PropertyAccessLibraryCompiler->FinishCompilation())
	{
		PropertyAccessLibraryCompiler->IterateErrors([&InCompilationContext](const FText& InErrorText, UObject* InObject)
        {
            // Output any property access errors as warnings
            if(InObject)
            {
                InCompilationContext.GetMessageLog().Warning(*InErrorText.ToString(), InObject);
            }
            else
            {
                InCompilationContext.GetMessageLog().Warning(*InErrorText.ToString());
            }
        });
	}

	OnPostLibraryCompiledDelegate.Broadcast(InCompilationContext, OutCompiledData);
	
	PropertyAccessLibraryCompiler.Reset();
}

FCompiledPropertyAccessHandle UAnimBlueprintExtension_PropertyAccess::GetCompiledHandle(FPropertyAccessHandle InHandle) const
{
	if(PropertyAccessLibraryCompiler.IsValid())
	{
		return PropertyAccessLibraryCompiler->GetCompiledHandle(InHandle);
	}
	return FCompiledPropertyAccessHandle();
}

EPropertyAccessCopyType UAnimBlueprintExtension_PropertyAccess::GetCompiledHandleAccessType(FPropertyAccessHandle InHandle) const
{
	if (PropertyAccessLibraryCompiler.IsValid())
	{
		return PropertyAccessLibraryCompiler->GetCompiledHandleAccessType(InHandle);
	}
	return EPropertyAccessCopyType::None;
}

FText UAnimBlueprintExtension_PropertyAccess::GetCompiledHandleContext(FCompiledPropertyAccessHandle InHandle)
{
	UEnum* EnumClass = StaticEnum<EAnimPropertyAccessCallSite>();
	check(EnumClass != nullptr);
	return EnumClass->GetDisplayNameTextByValue(InHandle.GetBatchId());
}

FText UAnimBlueprintExtension_PropertyAccess::GetCompiledHandleContextDesc(FCompiledPropertyAccessHandle InHandle)
{
	UEnum* EnumClass = StaticEnum<EAnimPropertyAccessCallSite>();
	check(EnumClass != nullptr);
	return EnumClass->GetToolTipTextByIndex(InHandle.GetBatchId());
}

bool UAnimBlueprintExtension_PropertyAccess::ContextRequiresCachedVariable(FName InName)
{
	if(InName == ContextId_Automatic)
	{
		// Indeterminate, so assume false - calling code will need to opt-in
		return false;
	}
	else if(InName == ContextId_UnBatched_ThreadSafe)
	{
		return false;
	}
	else if(InName == ContextId_Batched_WorkerThreadPreEventGraph)
	{
		return true;
	}
	else if(InName == ContextId_Batched_WorkerThreadPostEventGraph)
	{
		return true;
	}	
	else if(InName == ContextId_Batched_GameThreadPreEventGraph)
	{
		return true;
	}
	else if(InName == ContextId_Batched_GameThreadPostEventGraph)
	{
		return true;
	}

	return false;
}

void UAnimBlueprintExtension_PropertyAccess::ExpandPropertyAccess(FKismetCompilerContext& InCompilerContext, TArrayView<FString> InSourcePath, UEdGraph* InParentGraph, UEdGraphPin* InTargetPin) const
{
	check(InParentGraph);
	check(InTargetPin);
	
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	
	UEdGraphNode* SourceNode = InTargetPin->GetOwningNode();
	check(SourceNode);
	
	// Track the current pin in the pure chain
	UEdGraphPin* CurrentPin = nullptr;

	auto SpawnVariableGetNode = [&CurrentPin, &InCompilerContext, &SourceNode, &InParentGraph](FName InPropertyName)
	{
		const UEdGraphSchema* GraphSchema = InParentGraph->GetSchema();

		if(CurrentPin == nullptr)
		{
			// No current pin means we must be at the start of a chain with a 'self' member
			UK2Node_VariableGet* VariableGetNode = InCompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(SourceNode, InParentGraph);
			VariableGetNode->VariableReference.SetSelfMember(InPropertyName);
			VariableGetNode->AllocateDefaultPins();

			// Find pin that we just created - variable out pin is now CurrentPin
			CurrentPin = VariableGetNode->FindPinChecked(VariableGetNode->GetVarName());
		}
		else
		{
			// Current pin means we must be a class or a struct context
			if(UClass* Class = Cast<UClass>(CurrentPin->PinType.PinSubCategoryObject.Get()))
			{
				UK2Node_VariableGet* VariableGetNode = InCompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(SourceNode, InParentGraph);
				VariableGetNode->VariableReference.SetExternalMember(InPropertyName, Class);
				VariableGetNode->AllocateDefaultPins();

				// Link current to target, connection must succeed
				UEdGraphPin* TargetPin = VariableGetNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
				bool bSucceeded = GraphSchema->TryCreateConnection(CurrentPin, TargetPin);
				if(!bSucceeded)
				{
					InCompilerContext.MessageLog.Error(*LOCTEXT("VariableConnectionFailed", "@@ ICE: could not connect variable when expanding node").ToString(), SourceNode);
				}

				// Find pin that we just created - variable out pin is now CurrentPin
				CurrentPin = VariableGetNode->FindPinChecked(VariableGetNode->GetVarName());
			}
			else if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CurrentPin->PinType.PinSubCategoryObject.Get()))
			{
				// Create a break struct/split pin node
				const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(InParentGraph->GetSchema());
				UK2Node* SplitPinNode = K2Schema->CreateSplitPinNode(CurrentPin, UEdGraphSchema_K2::FCreateSplitPinNodeParams(&InCompilerContext, InParentGraph));
				if(UK2Node_BreakStruct* BreakStructNode = Cast<UK2Node_BreakStruct>(SplitPinNode))
				{
					// If we made a 'break struct' node, then we need to take into account that all the pins may not be visible
					for(FOptionalPinFromProperty& OptionalPin : BreakStructNode->ShowPinForProperties)
					{
						OptionalPin.bShowPin = true;
					}

					BreakStructNode->ReconstructNode();
				}
				UEdGraphPin* InputPin = SplitPinNode->FindPinByPredicate([ScriptStruct](UEdGraphPin* InPin)
				{
					if(InPin && InPin->Direction == EGPD_Input && InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
					{
						if(UScriptStruct* PinStruct = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()))
						{
							return PinStruct == ScriptStruct;
						}
					}

					return false;
				});
				check(InputPin);
				CurrentPin->MakeLinkTo(InputPin);

				// Current pin is the property of the struct
				// Note that this can fail if the struct failed to spawn a valid pin
				// Some custom struct pin logic (e.g. bOverride_* pins) can cause this to occur
				if(UEdGraphPin* FoundPin = SplitPinNode->FindPin(InPropertyName))
				{
					CurrentPin = FoundPin;
				}
			}
		}
	};
	
	IPropertyAccessEditor::FResolvePropertyAccessArgs Args;
	Args.PropertyFunction = [&CurrentPin, &InCompilerContext, &SourceNode, &InParentGraph, &SpawnVariableGetNode](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
	{
		SpawnVariableGetNode(InProperty->GetFName());
	};

	Args.ArrayFunction = [&CurrentPin, &InCompilerContext, &SourceNode, &InParentGraph, &SpawnVariableGetNode](int32 InSegmentIndex, FArrayProperty* InProperty, int32 InArrayIndex)
	{
		// First spawn a variable get to get the array into CurrentPin
		SpawnVariableGetNode(InProperty->GetFName());

		// Spawn an array item getter
		UK2Node_GetArrayItem* GetArrayItemNode = InCompilerContext.SpawnIntermediateNode<UK2Node_GetArrayItem>(SourceNode, InParentGraph);
		GetArrayItemNode->AllocateDefaultPins();

		UEdGraphPin* ArrayPin = GetArrayItemNode->GetTargetArrayPin();
		UEdGraphPin* ResultPin = GetArrayItemNode->GetResultPin();
		UEdGraphPin* IndexPin = GetArrayItemNode->GetIndexPin();
		check(ArrayPin && ResultPin && IndexPin);

		// Fill in the array index pin default value
		IndexPin->DefaultValue = FString::FromInt(InArrayIndex); 

		// Connect up the array variable output (CurrentPin) to the array pin
		const UEdGraphSchema* GraphSchema = InParentGraph->GetSchema();
		check(CurrentPin);
		bool bSucceeded = GraphSchema->TryCreateConnection(CurrentPin, ArrayPin);
		if(!bSucceeded)
		{
			InCompilerContext.MessageLog.Error(*LOCTEXT("ArrayConnectionFailed", "@@ ICE: could not connect array when expanding node").ToString(), SourceNode);
		}

		// Array element is now the current pin
		CurrentPin = ResultPin;
	};

	Args.FunctionFunction = [&CurrentPin, &InCompilerContext, &SourceNode, &InParentGraph](int32 InSegmentIndex, UFunction* InFunction, FProperty* InReturnProperty)
	{
		// Spawn a function call
		UK2Node_CallFunction* CallFunctionNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(SourceNode, InParentGraph);
		CallFunctionNode->SetFromFunction(InFunction);
		CallFunctionNode->AllocateDefaultPins();

		if(CurrentPin)
		{
			// If we have a current pin, hook it up to self
			UEdGraphPin* SelfPin = CallFunctionNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);

			const UEdGraphSchema* GraphSchema = InParentGraph->GetSchema();
			bool bSucceeded = GraphSchema->TryCreateConnection(CurrentPin, SelfPin);
			if(!bSucceeded)
			{
				InCompilerContext.MessageLog.Error(*LOCTEXT("FunctionConnectionFailed", "@@ ICE: could not connect function when expanding node").ToString(), SourceNode);
			}
		}

		// The new current pin is the return value
		UEdGraphPin* ReturnValuePin = CallFunctionNode->GetReturnValuePin();
		check(ReturnValuePin);
		CurrentPin = ReturnValuePin;
	};

	FPropertyAccessResolveResult Result = PropertyAccessEditor.ResolvePropertyAccess(InCompilerContext.Blueprint->SkeletonGeneratedClass, InSourcePath, Args);
	if(Result.Result == EPropertyAccessResolveResult::Succeeded)
	{
		const UEdGraphSchema* GraphSchema = InParentGraph->GetSchema();

		// Link current pin to target
		check(CurrentPin);
		if(InTargetPin->Direction == CurrentPin->Direction)
		{
			if(K2Schema->ArePinTypesCompatible(InTargetPin->PinType, CurrentPin->PinType, InCompilerContext.NewClass))
			{
				InCompilerContext.MovePinLinksToIntermediate(*InTargetPin, *CurrentPin);
			}
			else
			{
				// Need to create a conversion node. We will support basic conversion here for now as we are only looking for primitive types/casts
				if(TOptional<UEdGraphSchema_K2::FSearchForAutocastFunctionResults> AutoCastResults = K2Schema->SearchForAutocastFunction(CurrentPin->PinType, InTargetPin->PinType))
				{
					UK2Node_CallFunction* AutoCastNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(SourceNode, InParentGraph);
					AutoCastNode->FunctionReference.SetExternalMember(AutoCastResults->TargetFunction, AutoCastResults->FunctionOwner);
					AutoCastNode->AllocateDefaultPins();
					
					// Find output pin & connect
					UEdGraphPin* OutputPin = AutoCastNode->FindPinByPredicate([K2Schema, InTargetPin, &InCompilerContext](UEdGraphPin* InPin)
					{
						return InPin->Direction == EGPD_Output && K2Schema->ArePinTypesCompatible(InPin->PinType, InTargetPin->PinType, InCompilerContext.NewClass);
					});
					
					UEdGraphPin* InputPin = AutoCastNode->FindPinByPredicate([K2Schema, CurrentPin, &InCompilerContext](UEdGraphPin* InPin)
					{
						return InPin->Direction == EGPD_Input && K2Schema->ArePinTypesCompatible(InPin->PinType, CurrentPin->PinType, InCompilerContext.NewClass);
					});
					
					if(InputPin && OutputPin)
					{
						bool bSucceeded = GraphSchema->TryCreateConnection(CurrentPin, InputPin);
						if(!bSucceeded)
						{
							InCompilerContext.MessageLog.Error(*LOCTEXT("AutocastConnectionFailed", "@@ ICE: could not connect autocast when expanding node").ToString(), SourceNode);
						}
						else
						{
							CurrentPin = OutputPin;
							InCompilerContext.MovePinLinksToIntermediate(*InTargetPin, *CurrentPin);
						}
					}
					else
					{
						InCompilerContext.MessageLog.Error(*LOCTEXT("AutocastPinsConnectionFailed", "@@ ICE: could not find pins on autocast when expanding node").ToString(), SourceNode);
					}
				}
				else
				{
					InCompilerContext.MessageLog.Error(*LOCTEXT("AutocastFunctionExpansionFailed", "@@ could not make auto-cast function when expanding node").ToString(), SourceNode);
				}
			}
		}
		else
		{
			bool bSucceeded = GraphSchema->TryCreateConnection(CurrentPin, InTargetPin);
			if(!bSucceeded)
			{
				InCompilerContext.MessageLog.Error(*LOCTEXT("TargetConnectionFailed", "@@ ICE: could not connect target when expanding node").ToString(), SourceNode);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
