// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintExtension.h"
#include "Animation/AnimSubsystem_SharedLinkedAnimLayers.h"
#include "AnimBlueprintExtension_SharedLinkedAnimLayers.generated.h"

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_SharedLinkedAnimLayers : public UAnimBlueprintExtension
{
	GENERATED_BODY()

private:
	UPROPERTY()
	FAnimSubsystem_SharedLinkedAnimLayers Subsystem;
};