// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_StateMachine.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "K2Node_AnimGetter.h"
#include "K2Node_TransitionRuleGetter.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimGraphNode_Base.h"
#include "K2Node_StructMemberGet.h"
#include "AnimStateNode.h"
#include "AnimStateTransitionNode.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "EdGraphUtilities.h"
#include "AnimationStateMachineSchema.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"
#include "IAnimBlueprintCompilerCreationContext.h"
#include "IAnimBlueprintCompilationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_StateMachine)

#define LOCTEXT_NAMESPACE "AnimBlueprintExtension_StateMachine"

void UAnimBlueprintExtension_StateMachine::HandleBeginCompilation(IAnimBlueprintCompilerCreationContext& InCreationContext)
{
	InCreationContext.RegisterKnownGraphSchema(UAnimationStateMachineSchema::StaticClass());
}

void UAnimBlueprintExtension_StateMachine::HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	FoundGetterNodes.Empty();
	RootTransitionGetters.Empty();
	RootGraphAnimGetters.Empty();
}

void UAnimBlueprintExtension_StateMachine::HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	InCompilationContext.GetConsolidatedEventGraph()->GetNodesOfClass<UK2Node_TransitionRuleGetter>(RootTransitionGetters);

	// Get anim getters from the root anim graph (processing the nodes below will collect them in nested graphs)
	InCompilationContext.GetConsolidatedEventGraph()->GetNodesOfClass<UK2Node_AnimGetter>(RootGraphAnimGetters);
}

void UAnimBlueprintExtension_StateMachine::HandlePostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	// Process the getter nodes in the graph if there were any
	for (auto GetterIt = RootTransitionGetters.CreateIterator(); GetterIt; ++GetterIt)
	{
		ProcessTransitionGetter(*GetterIt, nullptr, InCompilationContext, OutCompiledData); // transition nodes should not appear at top-level
	}

	// Wire root getters
	for(UK2Node_AnimGetter* RootGraphGetter : RootGraphAnimGetters)
	{
		AutoWireAnimGetter(RootGraphGetter, nullptr, InCompilationContext, OutCompiledData);
	}

	// Wire nested getters
	for(UK2Node_AnimGetter* Getter : FoundGetterNodes)
	{
		AutoWireAnimGetter(Getter, nullptr, InCompilationContext, OutCompiledData);
	}
}

UK2Node_CallFunction* UAnimBlueprintExtension_StateMachine::SpawnCallAnimInstanceFunction(IAnimBlueprintCompilationContext& InCompilationContext, UEdGraphNode* SourceNode, FName FunctionName)
{
	//@TODO: SKELETON: This is a call on a parent function (UAnimInstance::StaticClass() specifically), should we treat it as self or not?
	UK2Node_CallFunction* FunctionCall = InCompilationContext.SpawnIntermediateNode<UK2Node_CallFunction>(SourceNode);
	FunctionCall->FunctionReference.SetSelfMember(FunctionName);
	FunctionCall->AllocateDefaultPins();

	return FunctionCall;
}

void UAnimBlueprintExtension_StateMachine::ProcessTransitionGetter(UK2Node_TransitionRuleGetter* Getter, UAnimStateTransitionNode* TransitionNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	// Get common elements for multiple getters
	UEdGraphPin* OutputPin = Getter->GetOutputPin();

	UEdGraphPin* SourceTimePin = NULL;
	UAnimationAsset* AnimAsset= NULL;
	int32 PlayerNodeIndex = INDEX_NONE;

	if (UAnimGraphNode_Base* SourcePlayerNode = Getter->AssociatedAnimAssetPlayerNode)
	{
		// This check should never fail as the source state is always processed first before handling it's rules
		UAnimGraphNode_Base* TrueSourceNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_Base>(SourcePlayerNode);
		UAnimGraphNode_Base* UndertypedPlayerNode = InCompilationContext.GetSourceNodeToProcessedNodeMap().FindRef(TrueSourceNode);

		if (UndertypedPlayerNode == NULL)
		{
			InCompilationContext.GetMessageLog().Error(TEXT("ICE: Player node @@ was not processed prior to handling a transition getter @@ that used it"), SourcePlayerNode, Getter);
			return;
		}

		// Make sure the node is still relevant
		UEdGraph* PlayerGraph = UndertypedPlayerNode->GetGraph();
		if (!PlayerGraph->Nodes.Contains(UndertypedPlayerNode))
		{
			InCompilationContext.GetMessageLog().Error(TEXT("@@ is not associated with a node in @@; please delete and recreate it"), Getter, PlayerGraph);
		}

		// Make sure the referenced AnimAsset player has been allocated
		PlayerNodeIndex = InCompilationContext.GetAllocationIndexOfNode(UndertypedPlayerNode);
		if (PlayerNodeIndex == INDEX_NONE)
		{
			InCompilationContext.GetMessageLog().Error(*LOCTEXT("BadAnimAssetNodeUsedInGetter", "@@ doesn't have a valid associated AnimAsset node.  Delete and recreate it").ToString(), Getter);
		}

		// Grab the AnimAsset, and time pin if needed
		UScriptStruct* TimePropertyInStructType = NULL;
		const TCHAR* TimePropertyName = NULL;
		if (UndertypedPlayerNode->DoesSupportTimeForTransitionGetter())
		{
			AnimAsset = UndertypedPlayerNode->GetAnimationAsset();
			TimePropertyInStructType = UndertypedPlayerNode->GetTimePropertyStruct();
			TimePropertyName = UndertypedPlayerNode->GetTimePropertyName();
		}
		else
		{
			InCompilationContext.GetMessageLog().Error(TEXT("@@ is associated with @@, which is an unexpected type"), Getter, UndertypedPlayerNode);
		}

		bool bNeedTimePin = false;

		// Determine if we need to read the current time variable from the specified sequence player
		switch (Getter->GetterType)
		{
		case ETransitionGetter::AnimationAsset_GetCurrentTime:
		case ETransitionGetter::AnimationAsset_GetCurrentTimeFraction:
		case ETransitionGetter::AnimationAsset_GetTimeFromEnd:
		case ETransitionGetter::AnimationAsset_GetTimeFromEndFraction:
			bNeedTimePin = true;
			break;
		default:
			bNeedTimePin = false;
			break;
		}

		if (bNeedTimePin && (PlayerNodeIndex != INDEX_NONE) && (TimePropertyName != NULL) && (TimePropertyInStructType != NULL))
		{
			const FProperty* NodeProperty = InCompilationContext.GetAllocatedPropertiesByIndex().FindChecked(PlayerNodeIndex);

			// Create a struct member read node to grab the current position of the sequence player node
			UK2Node_StructMemberGet* TimeReadNode = InCompilationContext.SpawnIntermediateNode<UK2Node_StructMemberGet>(Getter, InCompilationContext.GetConsolidatedEventGraph());
			TimeReadNode->VariableReference.SetSelfMember(NodeProperty->GetFName());
			TimeReadNode->StructType = TimePropertyInStructType;

			TimeReadNode->AllocatePinsForSingleMemberGet(TimePropertyName);
			SourceTimePin = TimeReadNode->FindPinChecked(TimePropertyName);
		}
	}

	// Expand it out
	UK2Node_CallFunction* GetterHelper = NULL;
	switch (Getter->GetterType)
	{
	case ETransitionGetter::AnimationAsset_GetCurrentTime:
		if ((AnimAsset != NULL) && (SourceTimePin != NULL))
		{
			GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceAssetPlayerTime"));
			GetterHelper->FindPinChecked(TEXT("AssetPlayerIndex"))->DefaultValue = FString::FromInt(PlayerNodeIndex);
		}
		else
		{
			if (Getter->AssociatedAnimAssetPlayerNode)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("Please replace @@ with Get Relevant Anim Time. @@ has no animation asset"), Getter, Getter->AssociatedAnimAssetPlayerNode);
			}
			else
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not asscociated with an asset player"), Getter);
			}
		}
		break;
	case ETransitionGetter::AnimationAsset_GetLength:
		if (AnimAsset != NULL)
		{
			GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceAssetPlayerLength"));
			GetterHelper->FindPinChecked(TEXT("AssetPlayerIndex"))->DefaultValue = FString::FromInt(PlayerNodeIndex);
		}
		else
		{
			if (Getter->AssociatedAnimAssetPlayerNode)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("Please replace @@ with Get Relevant Anim Length. @@ has no animation asset"), Getter, Getter->AssociatedAnimAssetPlayerNode);
			}
			else
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not asscociated with an asset player"), Getter);
			}
		}
		break;
	case ETransitionGetter::AnimationAsset_GetCurrentTimeFraction:
		if ((AnimAsset != NULL) && (SourceTimePin != NULL))
		{
			GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceAssetPlayerTimeFraction"));
			GetterHelper->FindPinChecked(TEXT("AssetPlayerIndex"))->DefaultValue = FString::FromInt(PlayerNodeIndex);
		}
		else
		{
			if (Getter->AssociatedAnimAssetPlayerNode)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("Please replace @@ with Get Relevant Anim Time Fraction. @@ has no animation asset"), Getter, Getter->AssociatedAnimAssetPlayerNode);
			}
			else
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not asscociated with an asset player"), Getter);
			}
		}
		break;
	case ETransitionGetter::AnimationAsset_GetTimeFromEnd:
		if ((AnimAsset != NULL) && (SourceTimePin != NULL))
		{
			GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceAssetPlayerTimeFromEnd"));
			GetterHelper->FindPinChecked(TEXT("AssetPlayerIndex"))->DefaultValue = FString::FromInt(PlayerNodeIndex);
		}
		else
		{
			if (Getter->AssociatedAnimAssetPlayerNode)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("Please replace @@ with Get Relevant Anim Time Remaining. @@ has no animation asset"), Getter, Getter->AssociatedAnimAssetPlayerNode);
			}
			else
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not asscociated with an asset player"), Getter);
			}
		}
		break;
	case ETransitionGetter::AnimationAsset_GetTimeFromEndFraction:
		if ((AnimAsset != NULL) && (SourceTimePin != NULL))
		{
			GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceAssetPlayerTimeFromEndFraction"));
			GetterHelper->FindPinChecked(TEXT("AssetPlayerIndex"))->DefaultValue = FString::FromInt(PlayerNodeIndex);
		}
		else
		{
			if (Getter->AssociatedAnimAssetPlayerNode)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("Please replace @@ with Get Relevant Anim Time Remaining Fraction. @@ has no animation asset"), Getter, Getter->AssociatedAnimAssetPlayerNode);
			}
			else
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not asscociated with an asset player"), Getter);
			}
		}
		break;

	case ETransitionGetter::CurrentTransitionDuration:
		{
			check(TransitionNode);
			if(UAnimStateNode* SourceStateNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimStateNode>(TransitionNode->GetPreviousState()))
			{
				if(UObject* SourceTransitionNode = InCompilationContext.GetMessageLog().FindSourceObject(TransitionNode))
				{
					if(FStateMachineDebugData* DebugData = OutCompiledData.GetAnimBlueprintDebugData().StateMachineDebugData.Find(SourceStateNode->GetGraph()))
					{
						if(int32* pStateIndex = DebugData->NodeToStateIndex.Find(SourceStateNode))
						{
							const int32 StateIndex = *pStateIndex;
							
							// This check should never fail as all animation nodes should be processed before getters are
							UAnimGraphNode_Base* CompiledMachineInstanceNode = InCompilationContext.GetSourceNodeToProcessedNodeMap().FindChecked(DebugData->MachineInstanceNode.Get());
							const int32 MachinePropertyIndex = InCompilationContext.GetAllocatedAnimNodeIndices().FindChecked(CompiledMachineInstanceNode);
							int32 TransitionPropertyIndex = INDEX_NONE;

							for(auto TransIt = DebugData->NodeToTransitionIndex.CreateConstIterator(); TransIt; ++TransIt)
							{
								const UEdGraphNode* CurrTransNode = TransIt.Key().Get();
								
								if(CurrTransNode == SourceTransitionNode)
								{
									TransitionPropertyIndex = TransIt.Value();
									break;
								}
							}

							if(TransitionPropertyIndex != INDEX_NONE)
							{
								GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceTransitionCrossfadeDuration"));
								GetterHelper->FindPinChecked(TEXT("MachineIndex"))->DefaultValue = FString::FromInt(MachinePropertyIndex);
								GetterHelper->FindPinChecked(TEXT("TransitionIndex"))->DefaultValue = FString::FromInt(TransitionPropertyIndex);
							}
						}
					}
				}
			}
		}
		break;

	case ETransitionGetter::ArbitraryState_GetBlendWeight:
		{
			if (Getter->AssociatedStateNode)
			{
				if (UAnimStateNode* SourceStateNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimStateNode>(Getter->AssociatedStateNode))
				{
					if (FStateMachineDebugData* DebugData = OutCompiledData.GetAnimBlueprintDebugData().StateMachineDebugData.Find(SourceStateNode->GetGraph()))
					{
						if (int32* pStateIndex = DebugData->NodeToStateIndex.Find(SourceStateNode))
						{
							const int32 StateIndex = *pStateIndex;
							//const int32 MachineIndex = DebugData->MachineIndex;

							// This check should never fail as all animation nodes should be processed before getters are
							UAnimGraphNode_Base* CompiledMachineInstanceNode = InCompilationContext.GetSourceNodeToProcessedNodeMap().FindChecked(DebugData->MachineInstanceNode.Get());
							const int32 MachinePropertyIndex = InCompilationContext.GetAllocatedAnimNodeIndices().FindChecked(CompiledMachineInstanceNode);

							GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceStateWeight"));
							GetterHelper->FindPinChecked(TEXT("MachineIndex"))->DefaultValue = FString::FromInt(MachinePropertyIndex);
							GetterHelper->FindPinChecked(TEXT("StateIndex"))->DefaultValue = FString::FromInt(StateIndex);
						}
					}
				}
			}

			if (GetterHelper == NULL)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not associated with a valid state"), Getter);
			}
		}
		break;

	case ETransitionGetter::CurrentState_ElapsedTime:
		{
			check(TransitionNode);
			if (UAnimStateNode* SourceStateNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimStateNode>(TransitionNode->GetPreviousState()))
			{
				if (FStateMachineDebugData* DebugData = OutCompiledData.GetAnimBlueprintDebugData().StateMachineDebugData.Find(SourceStateNode->GetGraph()))
				{
					// This check should never fail as all animation nodes should be processed before getters are
					UAnimGraphNode_Base* CompiledMachineInstanceNode = InCompilationContext.GetSourceNodeToProcessedNodeMap().FindChecked(DebugData->MachineInstanceNode.Get());
					const int32 MachinePropertyIndex = InCompilationContext.GetAllocatedAnimNodeIndices().FindChecked(CompiledMachineInstanceNode);

					GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceCurrentStateElapsedTime"));
					GetterHelper->FindPinChecked(TEXT("MachineIndex"))->DefaultValue = FString::FromInt(MachinePropertyIndex);
				}
			}
			if (GetterHelper == NULL)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not associated with a valid state"), Getter);
			}
		}
		break;

	case ETransitionGetter::CurrentState_GetBlendWeight:
		{
			check(TransitionNode);
			if (UAnimStateNode* SourceStateNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimStateNode>(TransitionNode->GetPreviousState()))
			{
				{
					if (FStateMachineDebugData* DebugData = OutCompiledData.GetAnimBlueprintDebugData().StateMachineDebugData.Find(SourceStateNode->GetGraph()))
					{
						if (int32* pStateIndex = DebugData->NodeToStateIndex.Find(SourceStateNode))
						{
							const int32 StateIndex = *pStateIndex;
							//const int32 MachineIndex = DebugData->MachineIndex;

							// This check should never fail as all animation nodes should be processed before getters are
							UAnimGraphNode_Base* CompiledMachineInstanceNode = InCompilationContext.GetSourceNodeToProcessedNodeMap().FindChecked(DebugData->MachineInstanceNode.Get());
							const int32 MachinePropertyIndex = InCompilationContext.GetAllocatedAnimNodeIndices().FindChecked(CompiledMachineInstanceNode);

							GetterHelper = SpawnCallAnimInstanceFunction(InCompilationContext, Getter, TEXT("GetInstanceStateWeight"));
							GetterHelper->FindPinChecked(TEXT("MachineIndex"))->DefaultValue = FString::FromInt(MachinePropertyIndex);
							GetterHelper->FindPinChecked(TEXT("StateIndex"))->DefaultValue = FString::FromInt(StateIndex);
						}
					}
				}
			}
			if (GetterHelper == NULL)
			{
				InCompilationContext.GetMessageLog().Error(TEXT("@@ is not associated with a valid state"), Getter);
			}
		}
		break;

	default:
		InCompilationContext.GetMessageLog().Error(TEXT("Unrecognized getter type on @@"), Getter);
		break;
	}

	// Finish wiring up a call function if needed
	if (GetterHelper != NULL)
	{
		check(GetterHelper->IsNodePure());

		UEdGraphPin* NewReturnPin = GetterHelper->FindPinChecked(TEXT("ReturnValue"));
		InCompilationContext.GetMessageLog().NotifyIntermediatePinCreation(NewReturnPin, OutputPin);

		NewReturnPin->CopyPersistentDataFromOldPin(*OutputPin);
	}

	// Remove the getter from the equation
	Getter->BreakAllNodeLinks();
}

void UAnimBlueprintExtension_StateMachine::AutoWireAnimGetter(class UK2Node_AnimGetter* Getter, UAnimStateTransitionNode* InTransitionNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UEdGraphPin* ReferencedNodeTimePin = nullptr;
	int32 ReferencedNodeIndex = INDEX_NONE;
	int32 SubNodeIndex = INDEX_NONE;
	
	UAnimGraphNode_Base* ProcessedNodeCheck = NULL;

	if(UAnimGraphNode_Base* SourceNode = Getter->SourceNode)
	{
		UAnimGraphNode_Base* ActualSourceNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_Base>(SourceNode);
		
		if(UAnimGraphNode_Base* ProcessedSourceNode = InCompilationContext.GetSourceNodeToProcessedNodeMap().FindRef(ActualSourceNode))
		{
			ProcessedNodeCheck = ProcessedSourceNode;

			ReferencedNodeIndex = InCompilationContext.GetAllocationIndexOfNode(ProcessedSourceNode);

			if(ProcessedSourceNode->DoesSupportTimeForTransitionGetter())
			{
				UScriptStruct* TimePropertyInStructType = ProcessedSourceNode->GetTimePropertyStruct();
				const TCHAR* TimePropertyName = ProcessedSourceNode->GetTimePropertyName();

				if(ReferencedNodeIndex != INDEX_NONE && TimePropertyName && TimePropertyInStructType)
				{
					FProperty* NodeProperty = InCompilationContext.GetAllocatedPropertiesByIndex().FindChecked(ReferencedNodeIndex);

					UK2Node_StructMemberGet* ReaderNode = InCompilationContext.SpawnIntermediateNode<UK2Node_StructMemberGet>(Getter, InCompilationContext.GetConsolidatedEventGraph());
					ReaderNode->VariableReference.SetSelfMember(NodeProperty->GetFName());
					ReaderNode->StructType = TimePropertyInStructType;
					ReaderNode->AllocatePinsForSingleMemberGet(TimePropertyName);

					ReferencedNodeTimePin = ReaderNode->FindPinChecked(TimePropertyName);
				}
			}
		}
	}
	
	if(Getter->SourceStateNode)
	{
		UObject* SourceObject = InCompilationContext.GetMessageLog().FindSourceObject(Getter->SourceStateNode);
		if(UAnimStateNode* SourceStateNode = Cast<UAnimStateNode>(SourceObject))
		{
			if(FStateMachineDebugData* DebugData = OutCompiledData.GetAnimBlueprintDebugData().StateMachineDebugData.Find(SourceStateNode->GetGraph()))
			{
				if(int32* StateIndexPtr = DebugData->NodeToStateIndex.Find(SourceStateNode))
				{
					SubNodeIndex = *StateIndexPtr;
				}
			}
		}
		else if(UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(SourceObject))
		{
			if(FStateMachineDebugData* DebugData = OutCompiledData.GetAnimBlueprintDebugData().StateMachineDebugData.Find(TransitionNode->GetGraph()))
			{
				TArray<int32> TransitionIndices;
				DebugData->NodeToTransitionIndex.MultiFind(TransitionNode, TransitionIndices);
				const int32 TransNum = TransitionIndices.Num();
				
				if (TransNum > 0)
				{
					SubNodeIndex = TransitionIndices[0];
				}
			}
		}
	}

	check(Getter->IsNodePure());

	for(UEdGraphPin* Pin : Getter->Pins)
	{
		// Hook up autowired parameters / pins
		if(Pin->PinName == TEXT("CurrentTime"))
		{
			Pin->MakeLinkTo(ReferencedNodeTimePin);
		}
		else if(Pin->PinName == TEXT("AssetPlayerIndex") || Pin->PinName == TEXT("MachineIndex"))
		{
			Pin->DefaultValue = FString::FromInt(ReferencedNodeIndex);
		}
		else if(Pin->PinName == TEXT("StateIndex") || Pin->PinName == TEXT("TransitionIndex"))
		{
			// Don't write an invalid state index which can occur for aliases.
			// Alias nodes set their default index earlier in UAnimGraphNode_StateMachineBase::OnProcessDuringCompilation
			if (SubNodeIndex != INDEX_NONE)
			{
				Pin->DefaultValue = FString::FromInt(SubNodeIndex);
			}
		}
	}
}

int32 UAnimBlueprintExtension_StateMachine::ExpandGraphAndProcessNodes(UEdGraph* SourceGraph, UAnimGraphNode_Base* SourceRootNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData, UAnimStateTransitionNode* TransitionNode, TArray<UEdGraphNode*>* ClonedNodes)
{
	// Clone the nodes from the source graph
	// Note that we outer this graph to the ConsolidatedEventGraph to allow ExpansionStep to 
	// correctly retrieve the context for any expanded function calls (custom make/break structs etc.)
	UEdGraph* ClonedGraph = FEdGraphUtilities::CloneGraph(SourceGraph, InCompilationContext.GetConsolidatedEventGraph(), &InCompilationContext.GetMessageLog(), true);

	// Grab all the animation nodes and find the corresponding root node in the cloned set
	UAnimGraphNode_Base* TargetRootNode = nullptr;
	TArray<UAnimGraphNode_Base*> AnimNodeList;
	TArray<UK2Node_TransitionRuleGetter*> Getters;
	TArray<UK2Node_AnimGetter*> AnimGetterNodes;

	for (auto NodeIt = ClonedGraph->Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = *NodeIt;

		if (UK2Node_TransitionRuleGetter* GetterNode = Cast<UK2Node_TransitionRuleGetter>(Node))
		{
			Getters.Add(GetterNode);
		}
		else if(UK2Node_AnimGetter* NewGetterNode = Cast<UK2Node_AnimGetter>(Node))
		{
			AnimGetterNodes.Add(NewGetterNode);
		}
		else if (UAnimGraphNode_Base* TestNode = Cast<UAnimGraphNode_Base>(Node))
		{
			AnimNodeList.Add(TestNode);

			//@TODO: There ought to be a better way to determine this
			if (InCompilationContext.GetMessageLog().FindSourceObject(TestNode) == InCompilationContext.GetMessageLog().FindSourceObject(SourceRootNode))
			{
				TargetRootNode = TestNode;
			}
		}

		if (ClonedNodes != NULL)
		{
			ClonedNodes->Add(Node);
		}
	}
	check(TargetRootNode);

	// Run another expansion pass to catch the graph we just added (this is slightly wasteful 
	InCompilationContext.ExpansionStep(ClonedGraph, false);

	// Validate graph now we have expanded/pruned
	InCompilationContext.ValidateGraphIsWellFormed(ClonedGraph);

	// Move the cloned nodes into the consolidated event graph
	const bool bIsLoading = InCompilationContext.GetBlueprint()->bIsRegeneratingOnLoad || IsAsyncLoading();
	const bool bIsCompiling = InCompilationContext.GetBlueprint()->bBeingCompiled;
	ClonedGraph->MoveNodesToAnotherGraph(InCompilationContext.GetConsolidatedEventGraph(), bIsLoading, bIsCompiling);

	// Process any animation nodes
	{
		TArray<UAnimGraphNode_Base*> RootSet;
		RootSet.Add(TargetRootNode);

		InCompilationContext.PruneIsolatedAnimationNodes(RootSet, AnimNodeList);

		InCompilationContext.ProcessAnimationNodes(AnimNodeList);
	}

	// Process the getter nodes in the graph if there were any
	for (auto GetterIt = Getters.CreateIterator(); GetterIt; ++GetterIt)
	{
		ProcessTransitionGetter(*GetterIt, TransitionNode, InCompilationContext, OutCompiledData);
	}

	// Wire anim getter nodes
	for(UK2Node_AnimGetter* GetterNode : AnimGetterNodes)
	{
		FoundGetterNodes.Add(GetterNode);
	}

	// Returns the index of the processed cloned version of SourceRootNode
	return InCompilationContext.GetAllocationIndexOfNode(TargetRootNode);	
}

#undef LOCTEXT_NAMESPACE
