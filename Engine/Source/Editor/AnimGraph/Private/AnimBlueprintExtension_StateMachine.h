// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_StateMachine.generated.h"

class UK2Node_CallFunction;
class UK2Node_AnimGetter;
class UK2Node_TransitionRuleGetter;
class UAnimGraphNode_Base;
class UAnimStateTransitionNode;
class UEdGraph;
class UEdGraphNode;
struct FAnimNotifyEvent;
class IAnimBlueprintCompilerCreationContext;
class IAnimBlueprintGeneratedClassCompiledData;
class IAnimBlueprintCompilationContext;
class IAnimBlueprintCompilationBracketContext;

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_StateMachine : public UAnimBlueprintExtension
{
	GENERATED_BODY()

public:
	// This function does the following steps:
	//   Clones the nodes in the specified source graph
	//   Merges them into the ConsolidatedEventGraph
	//   Processes any animation nodes
	//   Returns the index of the processed cloned version of SourceRootNode
	//	 If supplied, will also return an array of all cloned nodes
	int32 ExpandGraphAndProcessNodes(UEdGraph* SourceGraph, UAnimGraphNode_Base* SourceRootNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData, UAnimStateTransitionNode* TransitionNode = nullptr, TArray<UEdGraphNode*>* ClonedNodes = nullptr);

private:
	// UAnimBlueprintExtension interface
	virtual void HandleBeginCompilation(IAnimBlueprintCompilerCreationContext& InCreationContext) override;
	virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandlePostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

	// Spawns a function call node, calling a function on the anim instance
	UK2Node_CallFunction* SpawnCallAnimInstanceFunction(IAnimBlueprintCompilationContext& InCompilationContext, UEdGraphNode* SourceNode, FName FunctionName);

	// Convert transition getters into a function call/etc...
	void ProcessTransitionGetter(UK2Node_TransitionRuleGetter* Getter, UAnimStateTransitionNode* TransitionNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	// Automatically fill in parameters for the specified Getter node
	void AutoWireAnimGetter(UK2Node_AnimGetter* Getter, UAnimStateTransitionNode* InTransitionNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);
	
private:
	// List of getter node's we've found so the auto-wire can be deferred till after state machine compilation
	TArray<UK2Node_AnimGetter*> FoundGetterNodes;

	// Preprocessed lists of getters from the root of the ubergraph
	TArray<UK2Node_TransitionRuleGetter*> RootTransitionGetters;
	TArray<UK2Node_AnimGetter*> RootGraphAnimGetters;
};