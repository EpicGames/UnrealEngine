// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintExtension_Tag.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "IAnimBlueprintCompilationBracketContext.h"
#include "IAnimBlueprintCompilationContext.h"
#include "K2Node_CallFunction.h"
#include "IAnimBlueprintPostExpansionStepContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintExtension_Tag)

#define LOCTEXT_NAMESPACE "UAnimBlueprintExtension_Tag"

void UAnimBlueprintExtension_Tag::HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	TaggedNodes.Empty();
	Subsystem.NodeIndices.Empty();
	RequestedNodes.Empty();
}

void UAnimBlueprintExtension_Tag::HandlePostExpansionStep(const UEdGraph* InGraph, IAnimBlueprintPostExpansionStepContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	for (int32 Index = RequestedNodes.Num() - 1; Index >= 0; --Index)
	{
		const FNodeReferenceTag& RefTag = RequestedNodes[Index];
		
		if (UAnimGraphNode_Base* const* TaggedNode = TaggedNodes.Find(RefTag.Tag))
		{
			if (UEdGraphPin* IndexPin = RefTag.CallFunction->FindPin(TEXT("Index"), EGPD_Input))
			{
				if (const int32* NodeIndex = InCompilationContext.GetAllocatedAnimNodeIndices().Find(*TaggedNode))
				{
					IndexPin->DefaultValue = FString::FromInt(*NodeIndex);
					RequestedNodes.RemoveAt(Index, EAllowShrinking::No);
				}
			}
		}
	}
}

void UAnimBlueprintExtension_Tag::HandleFinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	for (const FNodeReferenceTag& RefTag : RequestedNodes)
	{
		InCompilationContext.GetMessageLog().Error(*FText::Format(LOCTEXT("MissingTaggedNodeError", "@@ cannot find referenced node with tag '{0}', ensure it is present and connected to the graph"), FText::FromName(RefTag.Tag)).ToString(), this);
	}

	for (const TPair<FName, UAnimGraphNode_Base*>& TaggedNodePair : TaggedNodes)
	{
		if (const int32* IndexPtr = InCompilationContext.GetAllocatedAnimNodeIndices().Find(TaggedNodePair.Value))
		{
			Subsystem.NodeIndices.Add(TaggedNodePair.Value->GetTag(), *IndexPtr);
		}
	}
}

void UAnimBlueprintExtension_Tag::AddTaggedNode(UAnimGraphNode_Base* InNode, IAnimBlueprintCompilationContext& InCompilationContext)
{
	if (InNode->GetTag() != NAME_None)
	{
		if (UAnimGraphNode_Base** ExistingNode = TaggedNodes.Find(InNode->GetTag()))
		{
			InCompilationContext.GetMessageLog().Error(*FText::Format(LOCTEXT("DuplicateLabelError", "Nodes @@ and @@ have the same reference tag '{0}'"), FText::FromName(InNode->GetTag())).ToString(), InNode, *ExistingNode);
		}
		else
		{
			TaggedNodes.Add(InNode->GetTag(), InNode);
		}
	}
}

void UAnimBlueprintExtension_Tag::RequestTaggedNode(UK2Node_AnimNodeReference* InNode, const FName InTag, UK2Node_CallFunction* InCallFunction)
{
	check(InNode && InTag != NAME_None && InCallFunction);
	RequestedNodes.Add({ InNode, InCallFunction, InTag });
}

#undef LOCTEXT_NAMESPACE
