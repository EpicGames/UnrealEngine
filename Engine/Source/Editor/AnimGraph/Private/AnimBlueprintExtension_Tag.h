// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimBlueprintExtension.h"
#include "Animation/AnimSubsystem_Tag.h"
#include "K2Node_AnimNodeReference.h"
#include "AnimBlueprintExtension_Tag.generated.h"

class UK2Node_CallFunction;
class UAnimGraphNode_Base;
class IAnimBlueprintCompilationContext;

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_Tag : public UAnimBlueprintExtension
{
	GENERATED_BODY()

public:
	// Add a tagged node
	void AddTaggedNode(UAnimGraphNode_Base* InNode, IAnimBlueprintCompilationContext& InCompilationContext);

	// Register the provided node to be hooked-up to the given tag during HandlePostProcessAnimationNodes, if the tag exists, error otherwise
	void RequestTaggedNode(UK2Node_AnimNodeReference* InNode, const FName InTag, UK2Node_CallFunction* InCallFunction);

private:
	// UAnimBlueprintExtension interface
	virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandlePostExpansionStep(const UEdGraph* InGraph, IAnimBlueprintPostExpansionStepContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void HandleFinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

private:
	// Map of tag -> tagged nodes
	TMap<FName, UAnimGraphNode_Base*> TaggedNodes;

	struct FNodeReferenceTag
	{
		UK2Node_AnimNodeReference* Node = nullptr;
		UK2Node_CallFunction* CallFunction = nullptr;
		FName Tag;
	};

	// Array of nodes to process during HandlePostProcessAnimationNodes
	TArray<FNodeReferenceTag> RequestedNodes;

	UPROPERTY()
	FAnimSubsystem_Tag Subsystem;
};