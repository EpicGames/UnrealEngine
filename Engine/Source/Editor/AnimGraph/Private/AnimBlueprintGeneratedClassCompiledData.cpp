// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintGeneratedClassCompiledData.h"
#include "Animation/AnimBlueprintGeneratedClass.h"

TArray<FBakedAnimationStateMachine>& FAnimBlueprintGeneratedClassCompiledData::GetBakedStateMachines() const
{
	return Class->BakedStateMachines;
}

TMap<FName, FCachedPoseIndices>& FAnimBlueprintGeneratedClassCompiledData::GetOrderedSavedPoseIndicesMap() const
{
	return Class->OrderedSavedPoseIndicesMap;
}

FBlueprintDebugData& FAnimBlueprintGeneratedClassCompiledData::GetBlueprintDebugData() const
{
	return Class->DebugData;
}

int32 FAnimBlueprintGeneratedClassCompiledData::FindOrAddNotify(FAnimNotifyEvent& Notify) const
{
	if ((Notify.NotifyName == NAME_None) && (Notify.Notify == nullptr) && (Notify.NotifyStateClass == nullptr))
	{
		// Non event, don't add it
		return INDEX_NONE;
	}

	int32 NewIndex = INDEX_NONE;
	for (int32 NotifyIdx = 0; NotifyIdx < Class->AnimNotifies.Num(); NotifyIdx++)
	{
		if( (Class->AnimNotifies[NotifyIdx].NotifyName == Notify.NotifyName) 
			&& (Class->AnimNotifies[NotifyIdx].Notify == Notify.Notify) 
			&& (Class->AnimNotifies[NotifyIdx].NotifyStateClass == Notify.NotifyStateClass) 
			)
		{
			NewIndex = NotifyIdx;
			break;
		}
	}

	if (NewIndex == INDEX_NONE)
	{
		NewIndex = Class->AnimNotifies.Add(Notify);
	}
	return NewIndex;
}

TArray<FAnimNotifyEvent>& FAnimBlueprintGeneratedClassCompiledData::GetAnimNotifies() const
{
	return Class->AnimNotifies;
}

FAnimBlueprintDebugData& FAnimBlueprintGeneratedClassCompiledData::GetAnimBlueprintDebugData() const
{
	return Class->AnimBlueprintDebugData;
}

TMap<FName, FGraphAssetPlayerInformation>& FAnimBlueprintGeneratedClassCompiledData::GetGraphAssetPlayerInformation() const
{
	return Class->GraphAssetPlayerInformation;
}

UBlendSpace* FAnimBlueprintGeneratedClassCompiledData::AddBlendSpace(UBlendSpace* InSourceBlendSpace)
{
	return nullptr;
}
