// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAnimBlueprintGeneratedClassCompiledData.h"

class UAnimBlueprintGeneratedClass;

class FAnimBlueprintGeneratedClassCompiledData : public IAnimBlueprintGeneratedClassCompiledData
{
public:
	FAnimBlueprintGeneratedClassCompiledData(UAnimBlueprintGeneratedClass* InClass)
		: Class(InClass)
	{}

private:
	// IAnimBlueprintGeneratedClassCompiledData interface
	virtual TArray<FBakedAnimationStateMachine>& GetBakedStateMachines() const override;
	virtual TMap<FName, FCachedPoseIndices>& GetOrderedSavedPoseIndicesMap() const override;
	virtual FBlueprintDebugData& GetBlueprintDebugData() const override;
	virtual TArray<FAnimNotifyEvent>& GetAnimNotifies() const override;
	virtual int32 FindOrAddNotify(FAnimNotifyEvent& Notify) const override;
	virtual FAnimBlueprintDebugData& GetAnimBlueprintDebugData() const override;
	virtual TMap<FName, FGraphAssetPlayerInformation>& GetGraphAssetPlayerInformation() const override;
	virtual UBlendSpace* AddBlendSpace(UBlendSpace* InSourceBlendSpace) override;

private:
	// The class we are wrapping
	UAnimBlueprintGeneratedClass* Class;
};