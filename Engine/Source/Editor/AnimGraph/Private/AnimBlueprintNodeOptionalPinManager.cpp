// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintNodeOptionalPinManager.h"
#include "UObject/UnrealType.h"
#include "Animation/AnimationAsset.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/AnimNodeBase.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimationGraphSchema.h"
#include "Algo/StableSort.h"

FAnimBlueprintNodeOptionalPinManager::FAnimBlueprintNodeOptionalPinManager(class UAnimGraphNode_Base* Node, TArray<UEdGraphPin*>* InOldPins)
	: BaseNode(Node)
	, OldPins(InOldPins)
{
	if (OldPins)
	{
		for (UEdGraphPin* Pin : *OldPins)
		{
			OldPinMap.Add(Pin->PinName, Pin);
		}
	}
}

void FAnimBlueprintNodeOptionalPinManager::GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const
{
	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();

	// Determine if this is a pose or array of poses
	FArrayProperty* ArrayProp = CastField<FArrayProperty>(TestProperty);
	FStructProperty* StructProp = CastField<FStructProperty>(ArrayProp ? ArrayProp->Inner : TestProperty);
	const bool bIsPoseInput = (StructProp  && StructProp->Struct->IsChildOf(FPoseLinkBase::StaticStruct()));

	//@TODO: Error if they specified two or more of these flags
	const bool bAlwaysShow = TestProperty->HasMetaData(Schema->NAME_AlwaysAsPin) || bIsPoseInput;
	const bool bOptional_ShowByDefault = TestProperty->HasMetaData(Schema->NAME_PinShownByDefault);
	const bool bOptional_HideByDefault = TestProperty->HasMetaData(Schema->NAME_PinHiddenByDefault);
	const bool bNeverShow = TestProperty->HasMetaData(Schema->NAME_NeverAsPin);
	const bool bPropertyIsCustomized = TestProperty->HasMetaData(Schema->NAME_CustomizeProperty);
	const bool bCanTreatPropertyAsOptional = CanTreatPropertyAsOptional(TestProperty);

	Record.bCanToggleVisibility = bCanTreatPropertyAsOptional && (bOptional_ShowByDefault || bOptional_HideByDefault);
	Record.bShowPin = bAlwaysShow || bOptional_ShowByDefault;
	Record.bPropertyIsCustomized = bPropertyIsCustomized;
}

void FAnimBlueprintNodeOptionalPinManager::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const
{
	if (BaseNode != nullptr)
	{
		BaseNode->CustomizePinData(Pin, SourcePropertyName, ArrayIndex);
	}
}

void FAnimBlueprintNodeOptionalPinManager::PostInitNewPin(UEdGraphPin* Pin, FOptionalPinFromProperty& Record, int32 ArrayIndex, FProperty* Property, uint8* PropertyAddress, uint8* DefaultPropertyAddress) const
{
	check(PropertyAddress != nullptr);
	check(Record.bShowPin);
	
	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();

	// In all cases set autogenerated default value from node defaults
	if (DefaultPropertyAddress)
	{
		FString LiteralValue;
		FBlueprintEditorUtils::PropertyValueToString_Direct(Property, DefaultPropertyAddress, LiteralValue);

		Schema->SetPinAutogeneratedDefaultValue(Pin, LiteralValue);
	}
	else
	{
		Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
	}

	if (OldPins == nullptr)
	{
		// Initial construction of a visible pin; copy values from the struct
		FString LiteralValue;
		FBlueprintEditorUtils::PropertyValueToString_Direct(Property, PropertyAddress, LiteralValue);

		Schema->SetPinDefaultValueAtConstruction(Pin, LiteralValue);
	}
	else if (Record.bCanToggleVisibility)
	{
		if (UEdGraphPin* OldPin = OldPinMap.FindRef(Pin->PinName))
		{
			// Was already visible, values will get copied over in pin reconstruction code
		}
		else
		{
			// Convert the struct property into DefaultValue/DefaultValueObject
			FString LiteralValue;
			FBlueprintEditorUtils::PropertyValueToString_Direct(Property, PropertyAddress, LiteralValue);

			Schema->SetPinDefaultValueAtConstruction(Pin, LiteralValue);
		}

		// Clear the asset reference on the node if the pin exists
		// In theory this is only needed for pins that are newly created but there are a lot of existing nodes that have dead asset references
		FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);
		if (ObjectProperty)
		{
			ObjectProperty->SetObjectPropertyValue(PropertyAddress, nullptr);
		}
	}
}

void FAnimBlueprintNodeOptionalPinManager::PostRemovedOldPin(FOptionalPinFromProperty& Record, int32 ArrayIndex, FProperty* Property, uint8* PropertyAddress, uint8* DefaultPropertyAddress) const
{
	check(PropertyAddress != nullptr);
	check(!Record.bShowPin);

	if (Record.bCanToggleVisibility && (OldPins != nullptr))
	{
		const FName OldPinName = (ArrayIndex != INDEX_NONE) ? *FString::Printf(TEXT("%s_%d"), *(Record.PropertyName.ToString()), ArrayIndex) : Record.PropertyName;
		// Pin was visible but it's now hidden
		if (UEdGraphPin* OldPin = OldPinMap.FindRef(OldPinName))
		{
			// Convert DefaultValue/DefaultValueObject and push back into the struct
			FBlueprintEditorUtils::PropertyValueFromString_Direct(Property, OldPin->GetDefaultAsString(), PropertyAddress, OldPin->GetOwningNodeUnchecked());
		}
	}
}

void FAnimBlueprintNodeOptionalPinManager::AllocateDefaultPins(UStruct* SourceStruct, uint8* StructBasePtr, uint8* DefaultsPtr)
{
	RebuildPropertyList(BaseNode->ShowPinForProperties, SourceStruct);

	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();
	static const FName NAME_DisplayPriority("DisplayPriority");

	constexpr auto GetDisplayPriority = [](const FProperty* Prop)
	{
		const FString& DisplayPriorityStr = Prop->GetMetaData(NAME_DisplayPriority);
		int32 DisplayPriority = (DisplayPriorityStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityStr));
		if (DisplayPriority == 0 && !FCString::IsNumeric(*DisplayPriorityStr))
		{
			// If there was a malformed display priority str Atoi will say it is 0, but we want to treat it as unset
			DisplayPriority = MAX_int32;
		}

		return DisplayPriority;
	};

	// Sort pins by display priority and property offset (makes pose pins consistent across derived nodes amongst other things)
	Algo::StableSort(BaseNode->ShowPinForProperties, [SourceStruct, Schema, GetDisplayPriority](const FOptionalPinFromProperty& InPin0, const FOptionalPinFromProperty& InPin1)
	{
		FProperty* Property0 = FindFieldChecked<FProperty>(SourceStruct, InPin0.PropertyName);
		FProperty* Property1 = FindFieldChecked<FProperty>(SourceStruct, InPin1.PropertyName);

		const int32 Priority0 = GetDisplayPriority(Property0);
		const int32 Priority1 = GetDisplayPriority(Property1);
		if (Priority0 != Priority1)
		{
			return Priority0 < Priority1;
		}

		return Property0->GetOffset_ForInternal() < Property1->GetOffset_ForInternal();
	});
	
	CreateVisiblePins(BaseNode->ShowPinForProperties, SourceStruct, EGPD_Input, BaseNode, StructBasePtr, DefaultsPtr);
}
