// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"

class UEdGraphPin;

struct FAnimBlueprintNodeOptionalPinManager : public FOptionalPinManager
{
protected:
	class UAnimGraphNode_Base* BaseNode;
	TArray<UEdGraphPin*>* OldPins;

	TMap<FName, UEdGraphPin*> OldPinMap;

public:
	FAnimBlueprintNodeOptionalPinManager(class UAnimGraphNode_Base* Node, TArray<UEdGraphPin*>* InOldPins);

	/** FOptionalPinManager interface */
	virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override;
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const override;
	virtual void PostInitNewPin(UEdGraphPin* Pin, FOptionalPinFromProperty& Record, int32 ArrayIndex, FProperty* Property, uint8* PropertyAddress, uint8* DefaultPropertyAddress) const override;
	virtual void PostRemovedOldPin(FOptionalPinFromProperty& Record, int32 ArrayIndex, FProperty* Property, uint8* PropertyAddress, uint8* DefaultPropertyAddress) const override;

	void AllocateDefaultPins(UStruct* SourceStruct, uint8* StructBasePtr, uint8* DefaultsPtr);
};
