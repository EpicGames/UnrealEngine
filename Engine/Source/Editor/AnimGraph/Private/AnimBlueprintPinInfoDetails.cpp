// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintPinInfoDetails.h"
#include "SPinTypeSelector.h"
#include "EdGraphSchema_K2.h"
#include "PropertyHandle.h"
#include "AnimGraphNode_LinkedInputPose.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

#include "Animation/AnimBlueprint.h"
#include "IAnimationBlueprintEditor.h"
#include "AnimationGraphSchema.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "AnimBlueprintPinInfoDetails"

void FAnimBlueprintFunctionPinInfoDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	TArray<UObject*> OuterObjects;
	InStructPropertyHandle->GetOuterObjects(OuterObjects);
	if(OuterObjects.Num() != 1)
	{
		return;
	}

	// skip if we cant edit this node as it is an interface graph
	UAnimGraphNode_LinkedInputPose* Node = CastChecked<UAnimGraphNode_LinkedInputPose>(OuterObjects[0]);
	if(!Node->CanUserDeleteNode())
	{
		return;
	}

	WeakOuterNode = Node;

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	StructPropertyHandle = InStructPropertyHandle;
	NamePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimBlueprintFunctionPinInfo, Name));
	TypePropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimBlueprintFunctionPinInfo, Type));

	StructPropertyHandle->MarkResetToDefaultCustomized(true);

	InHeaderRow
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.MinDesiredWidth(125.0f)
			.MaxDesiredWidth(250.0f)
			[
				SAssignNew(NameTextBox, SEditableTextBox)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this]()
				{ 
					FName Value;
					NamePropertyHandle->GetValue(Value);
					return FText::FromName(Value);
				})
				.OnTextChanged(this, &FAnimBlueprintFunctionPinInfoDetails::HandleTextChanged)
				.OnTextCommitted(this, &FAnimBlueprintFunctionPinInfoDetails::HandleTextCommitted)
			]
		]
	]
	.ValueContent()
	.MaxDesiredWidth(980.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		.AutoWidth()
		[
			SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(K2Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
			.TargetPinType(this, &FAnimBlueprintFunctionPinInfoDetails::GetTargetPinType)
			.OnPinTypeChanged(this, &FAnimBlueprintFunctionPinInfoDetails::HandlePinTypeChanged)
			.Schema(K2Schema)
			.TypeTreeFilter(ETypeTreeFilter::None)
			.bAllowArrays(true)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

FEdGraphPinType FAnimBlueprintFunctionPinInfoDetails::GetTargetPinType() const
{
	void* ValuePtr = nullptr;
	if(TypePropertyHandle->GetValueData(ValuePtr) != FPropertyAccess::Fail)
	{
		return *((FEdGraphPinType*)ValuePtr);
	}
	
	return FEdGraphPinType();
}

void FAnimBlueprintFunctionPinInfoDetails::HandlePinTypeChanged(const FEdGraphPinType& InPinType)
{
	void* ValuePtr = nullptr;
	if(TypePropertyHandle->GetValueData(ValuePtr) != FPropertyAccess::Fail)
	{
		TypePropertyHandle->NotifyPreChange();

		*((FEdGraphPinType*)ValuePtr) = InPinType;

		TypePropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

		UAnimBlueprint* AnimBlueprint = WeakOuterNode->GetAnimBlueprint();
		IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, false);
		check(AssetEditor->GetEditorName() == "AnimationBlueprintEditor");

		static_cast<IAnimationBlueprintEditor*>(AssetEditor)->SetLastGraphPinTypeUsed(InPinType);
	}
}

bool FAnimBlueprintFunctionPinInfoDetails::IsNameValid(const FString& InNewName)
{
	if(WeakOuterNode.IsValid())
	{
		UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(WeakOuterNode->GetGraph()->GetOuter());
		for(UEdGraph* Graph : AnimBlueprint->FunctionGraphs)
		{
			if(Graph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()))
			{
				TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
				Graph->GetNodesOfClass(LinkedInputPoseNodes);

				for(UAnimGraphNode_LinkedInputPose* LinkedInputPose : LinkedInputPoseNodes)
				{
					const TArray<FAnimBlueprintFunctionPinInfo>& Inputs = LinkedInputPose->Inputs;
					if(LinkedInputPose == WeakOuterNode.Get())
					{
						const int32 CurrentIndex = StructPropertyHandle->GetIndexInArray();
						for(int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
						{
							if(InputIndex != CurrentIndex)
							{
								if(Inputs[InputIndex].Name.ToString().Equals(InNewName, ESearchCase::IgnoreCase))
								{
									return false;
								}
							}
						}
					}
					else
					{
						for(int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
						{
							if(Inputs[InputIndex].Name.ToString().Equals(InNewName, ESearchCase::IgnoreCase))
							{
								return false;
							}
						}
					}
				}
			}
		}
	}

	return true;
}

void FAnimBlueprintFunctionPinInfoDetails::HandleTextChanged(const FText& InNewText)
{
	const FString NewTextAsString = InNewText.ToString();
	
	if(!IsNameValid(NewTextAsString))
	{
		NameTextBox->SetError(LOCTEXT("DuplicateInputError", "This input name already exists in this blueprint."));
	}
	else
	{
		NameTextBox->SetError(FText::GetEmpty());
	}
}

void FAnimBlueprintFunctionPinInfoDetails::HandleTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	const FString NewTextAsString = InNewText.ToString();
	if(IsNameValid(NewTextAsString))
	{
		FName NewName = *InNewText.ToString();
		NamePropertyHandle->SetValue(NewName);
	}

	NameTextBox->SetError(FText::GetEmpty());
}

#undef LOCTEXT_NAMESPACE