// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Types/SlateEnums.h"

struct FEdGraphPinType;
class UAnimGraphNode_LinkedInputPose;
class SEditableTextBox;

class FAnimBlueprintFunctionPinInfoDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FAnimBlueprintFunctionPinInfoDetails());
	}

	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) {};

private:
	/** Check to see if this name is valid */
	bool IsNameValid(const FString& InNewName);

	/** UI handlers */
	FEdGraphPinType GetTargetPinType() const;
	void HandlePinTypeChanged(const FEdGraphPinType& InPinType);
	void HandleTextChanged(const FText& InNewText);
	void HandleTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType);

private:
	/** Property handles to edit */
	TSharedPtr<IPropertyHandle> NamePropertyHandle;
	TSharedPtr<IPropertyHandle> TypePropertyHandle;
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Text box for editing names */
	TSharedPtr<SEditableTextBox> NameTextBox;

	/** The node we are editing on */
	TWeakObjectPtr<UAnimGraphNode_LinkedInputPose> WeakOuterNode;
};
