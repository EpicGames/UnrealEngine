// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintPostCompileValidation.h" 
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimInstance.h"
#include "AnimGraphNode_Base.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintPostCompileValidation)

UAnimBlueprintPostCompileValidation::UAnimBlueprintPostCompileValidation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimBlueprintPostCompileValidation::DoPostCompileValidation(FAnimBPCompileValidationParams& InParams) const
{

}

// Ensures the specified object is preloaded.  ReferencedObject can be NULL.
void UAnimBlueprintPostCompileValidation::PCV_PreloadObject(const UObject* const ReferencedObject)
{
	if (ReferencedObject)
	{
		const_cast<UObject*>(ReferencedObject)->ConditionalPreload();
	}
}

void UAnimBlueprintPostCompileValidation::PCV_GatherAllReferencedAnimSequences(TArray<FPCV_ReferencedAnimSequence>& OutRefAnimSequences, FAnimBPCompileValidationParams& PCV_Params)
{
	PCV_PreloadObject(PCV_Params.DefaultAnimInstance);
	TArray<FPCV_PropertyAndValue> PropertyCallChain;
	PCV_GatherAnimSequencesFromStruct(OutRefAnimSequences, PCV_Params, PCV_Params.NewAnimBlueprintClass, PCV_Params.DefaultAnimInstance, PropertyCallChain);
}

void UAnimBlueprintPostCompileValidation::PCV_GatherAnimSequencesFromStruct(TArray<FPCV_ReferencedAnimSequence>& OutRefAnimSequences, FAnimBPCompileValidationParams& PCV_Params, const UStruct* InStruct, const void* InData, TArray<FPCV_PropertyAndValue> InPropertyCallChain)
{
	for (TFieldIterator<FProperty> PropIt(InStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		if (const FProperty* Property = *PropIt)
		{
			const void* PropertyData = Property->ContainerPtrToValuePtr<void>(InData);
			PCV_GatherAnimSequencesFromProperty(OutRefAnimSequences, PCV_Params, Property, PropertyData, InPropertyCallChain);
		}
	}
}

void UAnimBlueprintPostCompileValidation::PCV_GatherAnimSequencesFromProperty(TArray<FPCV_ReferencedAnimSequence>& OutRefAnimSequences, FAnimBPCompileValidationParams& PCV_Params, const FProperty* InProperty, const void* InData, TArray<FPCV_PropertyAndValue> InPropertyCallChain)
{
	InPropertyCallChain.Add(FPCV_PropertyAndValue(InProperty, InData));

	// Always recurse into arrays and structs
	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, InData);
		const int32 NumElements = ArrayHelper.Num();
		for (int32 Index = 0; Index < NumElements; Index++)
		{
			const void* const ArrayData = ArrayHelper.GetRawPtr(Index);
			PCV_GatherAnimSequencesFromProperty(OutRefAnimSequences, PCV_Params, ArrayProperty->Inner, ArrayData, InPropertyCallChain);
		}
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		PCV_GatherAnimSequencesFromStruct(OutRefAnimSequences, PCV_Params, StructProperty->Struct, InData, InPropertyCallChain);
	}

	// Leaf Properties
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
	{
		const UObject* const ObjectPropertyValue = ObjectProperty->GetObjectPropertyValue(InData);
		if (const UAnimSequence* const AnimSequence = Cast<UAnimSequence>(ObjectPropertyValue))
		{
			// make sure we don't have duplicates
			for (const FPCV_ReferencedAnimSequence& AnimSequenceRef : OutRefAnimSequences)
			{
				if (AnimSequenceRef.AnimSequence == AnimSequence)
				{
					return;
				}
			}

			// Find Parent Referencer.
			const UObject* Referencer = PCV_Params.DefaultAnimInstance;
			for (int32 Index = InPropertyCallChain.Num() - 2; Index >= 0; Index--)
			{
				const FPCV_PropertyAndValue& Parent = InPropertyCallChain[Index];
				if (const FStructProperty* ParentStructProperty = CastField<FStructProperty>(Parent.Property))
				{
					if (const UAnimGraphNode_Base* AnimGraphVisualNode = PCV_Params.AllocatedNodePropertiesToNodes.FindRef(ParentStructProperty))
					{
						Referencer = AnimGraphVisualNode;
						break;
					}
				}
				else if (const FObjectProperty* ParentObjectProperty = CastField<FObjectProperty>(Parent.Property))
				{
					if (ParentObjectProperty->PropertyClass && ParentObjectProperty->PropertyClass->IsChildOf(UBlendSpace::StaticClass()))
					{
						Referencer = ParentObjectProperty->GetObjectPropertyValue(Parent.Value);
						break;
					}
				}
			}

			OutRefAnimSequences.Add(FPCV_ReferencedAnimSequence(AnimSequence, Referencer));
		}
		else if (const UBlendSpace* const BlendSpace = Cast<UBlendSpace>(ObjectPropertyValue))
		{
			PCV_PreloadObject(BlendSpace);

			// recurse into BlendSpaces to grab referenced animations.
			PCV_GatherAnimSequencesFromStruct(OutRefAnimSequences, PCV_Params, BlendSpace->GetClass(), BlendSpace, InPropertyCallChain);
		}
	}
}

void UAnimBlueprintPostCompileValidation::PCV_GatherAnimSequences(TArray<const UAnimSequence*>& OutAnimSequences, const UAnimSequenceBase* const InAnimSequenceBase)
{
	if (const UAnimSequence* const AnimSeq = Cast<UAnimSequence>(InAnimSequenceBase))
	{
		OutAnimSequences.AddUnique(AnimSeq);
	}
}

void UAnimBlueprintPostCompileValidation::PCV_GatherAnimSequences(TArray<const UAnimSequence*>& OutAnimSequences, const class UBlendSpace* const InBlendSpace)
{
	// Make sure BlendSpace is loaded, so we can access referenced AnimSequences.
	PCV_PreloadObject(InBlendSpace);

	if (InBlendSpace)
	{
		for (const FBlendSample& BlendSample : InBlendSpace->GetBlendSamples())
		{
			PCV_GatherAnimSequences(OutAnimSequences, BlendSample.Animation);
		}
	}
}

void UAnimBlueprintPostCompileValidation::PCV_GatherAnimSequencesFromGraph(TArray<const UAnimSequence*>& OutAnimSequences, FAnimBPCompileValidationParams& PCV_Params, const FPCV_GatherParams& GatherParams)
{
	for (FStructProperty* Property : TFieldRange<FStructProperty>(PCV_Params.NewAnimBlueprintClass, EFieldIteratorFlags::IncludeSuper))
	{
		if (Property->Struct->IsChildOf(FAnimNode_BlendSpacePlayer::StaticStruct()))
		{
			if (const FAnimNode_BlendSpacePlayer* const BlendSpacePlayer = Property->ContainerPtrToValuePtr<FAnimNode_BlendSpacePlayer>(PCV_Params.DefaultAnimInstance))
			{
				const bool bPassSyncGroupFilter = !GatherParams.bFilterBySyncGroup || (BlendSpacePlayer->GetGroupName() == GatherParams.SyncGroupName);
				const bool bPassLoopingFilter = !GatherParams.bFilterByLoopingCondition || (BlendSpacePlayer->IsLooping() == GatherParams.bLoopingCondition);
				if (bPassSyncGroupFilter && bPassLoopingFilter)
				{
					PCV_GatherAnimSequences(OutAnimSequences, BlendSpacePlayer->GetBlendSpace());
				}
			}
		}
		else if (Property->Struct->IsChildOf(FAnimNode_SequencePlayer::StaticStruct()))
		{
			if (const FAnimNode_SequencePlayer* const SequencePlayer = Property->ContainerPtrToValuePtr<FAnimNode_SequencePlayer>(PCV_Params.DefaultAnimInstance))
			{
				const bool bPassSyncGroupFilter = !GatherParams.bFilterBySyncGroup || (SequencePlayer->GetGroupName() == GatherParams.SyncGroupName);
				const bool bPassLoopingFilter = !GatherParams.bFilterByLoopingCondition || (SequencePlayer->IsLooping() == GatherParams.bLoopingCondition);
				if (bPassSyncGroupFilter && bPassLoopingFilter)
				{
					PCV_GatherAnimSequences(OutAnimSequences, SequencePlayer->GetSequence());
				}
			}
		}
	}
}

void UAnimBlueprintPostCompileValidation::PCV_GatherBlendSpacesFromGraph(TArray<const class UBlendSpace*>& OutBlendSpaces, FAnimBPCompileValidationParams& PCV_Params)
{
	for (FStructProperty* Property : TFieldRange<FStructProperty>(PCV_Params.NewAnimBlueprintClass, EFieldIteratorFlags::IncludeSuper))
	{
		if (Property->Struct->IsChildOf(FAnimNode_BlendSpacePlayer::StaticStruct()))
		{
			if (const FAnimNode_BlendSpacePlayer* const BlendSpacePlayer = Property->ContainerPtrToValuePtr<FAnimNode_BlendSpacePlayer>(PCV_Params.DefaultAnimInstance))
			{
				OutBlendSpaces.AddUnique(BlendSpacePlayer->GetBlendSpace());
			}
		}
	}
}
