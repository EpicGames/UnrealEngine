// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBlueprintVariableCreationContext.h"
#include "AnimBlueprintCompiler.h"

FProperty* FAnimBlueprintVariableCreationContext::CreateVariable(const FName Name, const FEdGraphPinType& Type)
{
	return CompilerContext->CreateVariable(Name, Type);
}

FProperty* FAnimBlueprintVariableCreationContext::CreateUniqueVariable(UObject* InForObject, const FEdGraphPinType& Type)
{
	return CompilerContext->CreateUniqueVariable(InForObject, Type);
}
