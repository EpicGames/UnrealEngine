// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClassVariableCreator.h"

class FAnimBlueprintCompilerContext;

class FAnimBlueprintVariableCreationContext : public IAnimBlueprintVariableCreationContext
{
private:
	friend class FAnimBlueprintCompilerContext;

	FAnimBlueprintVariableCreationContext(FAnimBlueprintCompilerContext* InCompilerContext)
		: CompilerContext(InCompilerContext)
	{}
	
	// IAnimBlueprintVariableCreationContext interface
	virtual FProperty* CreateVariable(const FName Name, const FEdGraphPinType& Type) override;
	virtual FProperty* CreateUniqueVariable(UObject* InForObject, const FEdGraphPinType& Type) override;

	FAnimBlueprintCompilerContext* CompilerContext;
};