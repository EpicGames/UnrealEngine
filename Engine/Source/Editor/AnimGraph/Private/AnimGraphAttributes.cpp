// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphAttributes.h"

#include "Templates/Function.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphAttributes)

struct FPropertyChangedEvent;

void UAnimGraphAttributes::Register(const FAnimGraphAttributeDesc& InDesc)
{
	if(AttributesMap.Find(InDesc.Name) == nullptr)
	{
		FAnimGraphAttributeDesc& NewDesc = Attributes.Add_GetRef(InDesc);
		if(NewDesc.SortOrder == INDEX_NONE)
		{
			NewDesc.SortOrder = Attributes.Num() - 1;
		}
		AttributesMap.Add(InDesc.Name, Attributes.Num() - 1);
	}
}

const FAnimGraphAttributeDesc* UAnimGraphAttributes::FindAttributeDesc(FName InName) const
{
	if(const int32* IndexPtr = AttributesMap.Find(InName))
	{
		return &Attributes[*IndexPtr];
	}
	
	return nullptr;
}

void UAnimGraphAttributes::ForEachAttribute(TFunctionRef<void(const FAnimGraphAttributeDesc&)> InFunction) const
{
	for(const FAnimGraphAttributeDesc& Desc : Attributes)
	{
		InFunction(Desc);
	}
}

void UAnimGraphAttributes::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Rebuild map
	AttributesMap.Reset();

	for(int32 DescIndex = 0; DescIndex < Attributes.Num(); ++DescIndex)
	{
		const FAnimGraphAttributeDesc& Desc = Attributes[DescIndex];
		AttributesMap.Add(Desc.Name, DescIndex);
	}
}
