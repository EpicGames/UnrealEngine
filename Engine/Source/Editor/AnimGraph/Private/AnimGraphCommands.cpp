// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AnimGraphCommands"

UE_DEFINE_TCOMMANDS(FAnimGraphCommands)

void FAnimGraphCommands::RegisterCommands()
{
	UI_COMMAND(TogglePoseWatch, "Toggle Pose Watch", "Toggle pose watching on this node", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(HideUnboundPropertyPins, "Hide Unbound/Unset Property Pins", "Unexpose all unbound/unset property pins from this node", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( SelectBone, "Select Bone", "Assign or change the bone for skeletal controls", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( AddBlendListPin, "Add Blend Pin", "Add blend pin to blend list", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( RemoveBlendListPin, "Remove Blend Pin", "Remove blend pin", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( ConvertToSeqEvaluator, "Convert to Single Frame Animation", "Convert to one frame animation that requires position", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ConvertToSeqPlayer, "Convert to Sequence Player", "Convert back to sequence player without manual position set up", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( ConvertToBSEvaluator, "Convert to Single Frame Blend Space Player", "Convert to one frame blend space player that requires position", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ConvertToBSPlayer, "Convert to Blend Space Player", "Convert back to blend space player without manual position set up", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ConvertToBSGraph, "Convert to Blend Space Graph", "Convert to blend space graph", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( ConvertToAimOffsetLookAt, "Convert to LookAt AimOffset", "Convert to one AimOffset that automatically tracks a Target", EUserInterfaceActionType::Button, FInputChord())
	UI_COMMAND( ConvertToAimOffsetSimple, "Convert to simple AimOffset", "Convert to a manual AimOffset", EUserInterfaceActionType::Button, FInputChord())
	UI_COMMAND( ConvertToAimOffsetGraph, "Convert to AimOffset Graph", "Convert to an AimOffset graph", EUserInterfaceActionType::Button, FInputChord())

	UI_COMMAND(ConvertToPoseBlender, "Convert to Pose Blender", "Convert to pose blender that can blend by source curves", EUserInterfaceActionType::Button, FInputChord())
	UI_COMMAND(ConvertToPoseByName, "Convert to Pose By Name", "Convert to pose node that returns by name", EUserInterfaceActionType::Button, FInputChord())

	UI_COMMAND( OpenRelatedAsset, "Open Asset", "Opens the asset related to this node", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( CreateSelfTransition, "Create Self Transition", "Creates an alias for this state and adds a 'self' transition from the alias to this state", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( SetActiveState, "Set Active State", "Sets the currently-selected state as active", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ReverseTransition, "Reverse Transition", "Reverses this transition", EUserInterfaceActionType::Button, FInputChord() )
}

#undef LOCTEXT_NAMESPACE
