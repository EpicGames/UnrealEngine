// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphDetails.h"
#include "IAnimationBlueprintEditor.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "EdGraph/EdGraph.h"
#include "AnimGraphNode_LinkedInputPose.h"
#include "Algo/Transform.h"
#include "DetailWidgetRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "AnimGraphNode_Root.h"
#include "ScopedTransaction.h"
#include "ObjectEditorUtils.h"
#include "SKismetInspector.h"
#include "AnimationGraph.h"
#include "AnimationGraphSchema.h"
#include "AnimGraphNode_LinkedAnimLayer.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "FAnimGraphDetails"

TSharedPtr<IDetailCustomization> FAnimGraphDetails::MakeInstance(TSharedPtr<IBlueprintEditor> InBlueprintEditor)
{
	const TArray<UObject*>* Objects = (InBlueprintEditor.IsValid() ? InBlueprintEditor->GetObjectsCurrentlyBeingEdited() : nullptr);
	if (Objects && Objects->Num() == 1)
	{
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>((*Objects)[0]))
		{
			return MakeShareable(new FAnimGraphDetails(StaticCastSharedPtr<IAnimationBlueprintEditor>(InBlueprintEditor), AnimBlueprint));
		}
	}

	return nullptr;
}

void FAnimGraphDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);

	Graph = CastChecked<UEdGraph>(Objects[0].Get());

	if(IsInterface())
	{
		DetailLayout.HideCategory("GraphBlending");
	}

	bool const bIsStateMachine = !Graph->GetOuter()->IsA(UAnimBlueprint::StaticClass());

	if(Objects.Num() > 1 || bIsStateMachine)
	{
		IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("Inputs", LOCTEXT("LinkedInputPoseInputsCategory", "Inputs"));
		InputsCategory.SetCategoryVisibility(false);
		return;
	}

	const bool bIsDefaultGraph = Graph->GetFName() == UEdGraphSchema_K2::GN_AnimGraph;

	if(!Graph->bAllowDeletion && !bIsDefaultGraph)
	{
		FText ReadOnlyWarning = LOCTEXT("ReadOnlyWarning", "This graph's inputs are read-only and cannot be edited");

		IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("Inputs", LOCTEXT("LinkedInputPoseInputsCategory", "Inputs"));
		InputsCategory.SetCategoryVisibility(false);

		IDetailCategoryBuilder& WarningCategoryBuilder = DetailLayout.EditCategory("GraphInputs", LOCTEXT("GraphInputsCategory", "Graph Inputs"));
		WarningCategoryBuilder.AddCustomRow(ReadOnlyWarning)
			.WholeRowContent()
			[
				SNew(STextBlock)
				.Text(ReadOnlyWarning)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];

		return;
	}

	if(!bIsDefaultGraph)
	{
		IDetailCategoryBuilder& LayerCategory = DetailLayout.EditCategory("Layer", LOCTEXT("LayerCategory", "Layer"));
		{
			FText GroupLabel(LOCTEXT("LayerGroup", "Group"));
			FText GroupToolTip(LOCTEXT("LayerGroupToolTip", "The group of this layer. Grouped layers will run using the same underlying instance, so can share state."));

			RefreshGroupSource();

			LayerCategory.AddCustomRow(GroupLabel)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(GroupLabel)
				.ToolTipText(GroupToolTip)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(GroupComboButton, SComboButton)
				.ContentPadding(FMargin(0,0,5,0))
				.ToolTipText(GroupToolTip)
				.ButtonContent()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.Padding(FMargin(0, 0, 5, 0))
					[
						SNew(SEditableTextBox)
						.Text(this, &FAnimGraphDetails::OnGetGroupText)
						.OnTextCommitted(this, &FAnimGraphDetails::OnGroupTextCommitted)
						.ToolTipText(GroupToolTip)
						.SelectAllTextWhenFocused(true)
						.RevertTextOnEscape(true)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				.MenuContent()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.MaxHeight(400.0f)
					[
						SAssignNew(GroupListView, SListView<TSharedPtr<FText>>)
						.ListItemsSource(&GroupSource)
						.OnGenerateRow(this, &FAnimGraphDetails::MakeGroupViewWidget)
						.OnSelectionChanged(this, &FAnimGraphDetails::OnGroupSelectionChanged)
					]
				]
			];
		}
	}

	IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("Inputs", LOCTEXT("LinkedInputPoseInputsCategory", "Inputs"));
	InputsCategory.RestoreExpansionState(true);

	DetailLayoutBuilder = &DetailLayout;

	// Gather inputs, if any
	TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseInputs;
	Graph->GetNodesOfClass<UAnimGraphNode_LinkedInputPose>(LinkedInputPoseInputs);

	TSharedRef<SHorizontalBox> InputsHeaderContentWidget = SNew(SHorizontalBox);
	TWeakPtr<SWidget> WeakInputsHeaderWidget = InputsHeaderContentWidget;
	InputsHeaderContentWidget->AddSlot()
	[
		SNew(SHorizontalBox)
	];
	InputsHeaderContentWidget->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "RoundButton")
		.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
		.ContentPadding(FMargin(2, 0))
		.OnClicked(this, &FAnimGraphDetails::OnAddNewInputPoseClicked)
		.HAlign(HAlign_Right)
		.ToolTipText(LOCTEXT("NewInputPoseTooltip", "Create a new input pose"))
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0, 1))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Plus"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(2, 0, 0, 0))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
				.Text(LOCTEXT("NewInputPoseButtonText", "New Input Pose"))
				.Visibility(this, &FAnimGraphDetails::OnGetNewInputPoseTextVisibility, WeakInputsHeaderWidget)
				.ShadowOffset(FVector2D(1, 1))
			]
		]
	];
	InputsCategory.HeaderContent(InputsHeaderContentWidget);

	if(LinkedInputPoseInputs.Num())
	{
		for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode : LinkedInputPoseInputs)
		{
			auto GetLinkedInputPoseLabel = [WeakLinkedInputPoseNode = TWeakObjectPtr<UAnimGraphNode_LinkedInputPose>(LinkedInputPoseNode)]()
			{
				if(WeakLinkedInputPoseNode.IsValid())
				{
					return FText::FromName(WeakLinkedInputPoseNode->Node.Name);
				}

				return FText::GetEmpty();
			};

			TArray<UObject*> ExternalObjects;
			ExternalObjects.Add(LinkedInputPoseNode);
			FAddPropertyParams AddPropertyParams;
			AddPropertyParams.UniqueId(LinkedInputPoseNode->GetFName());

			if(IDetailPropertyRow* LinkedInputPoseRow = InputsCategory.AddExternalObjects(ExternalObjects, EPropertyLocation::Default, AddPropertyParams))
			{
				LinkedInputPoseRow->CustomWidget()
				.NameContent()
				[
					SNew(SBox)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("InputPose", "Input Pose"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				.ValueContent()
				.MaxDesiredWidth(250.0f)
				[
					SNew(SBox)
					.Padding(2.0f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.0f)
						[
							SNew(STextBlock)
							.Text_Lambda(GetLinkedInputPoseLabel)
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
						+SHorizontalBox::Slot()
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Right)
						.AutoWidth()
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
							.ContentPadding(FMargin(2, 2))
							.OnClicked(this, &FAnimGraphDetails::OnRemoveInputPoseClicked, LinkedInputPoseNode)
							.ToolTipText(LOCTEXT("RemoveInputPoseTooltip", "Remove this input pose"))
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Cross"))
							]
						]
					]
				];
			}
		}
	}
	else
	{
		// Add a text widget to let the user know to hit the + icon to add parameters.
		InputsCategory.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.MaxDesiredWidth(980.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoInputPosesAddedForAnimGraph", "Please press the + icon above to add input poses"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}

	if(IsInterface())
	{
		UAnimationGraphSchema::AutoArrangeInterfaceGraph(*Graph);
	}
}

FReply FAnimGraphDetails::OnAddNewInputPoseClicked()
{
	EK2NewNodeFlags NewNodeOperation = EK2NewNodeFlags::None;
	FVector2D NewNodePosition(0.0f, 0.0f);

	if(!IsInterface())
	{
		NewNodePosition = UAnimationGraphSchema::GetPositionForNewLinkedInputPoseNode(*Graph);
	}

	FEdGraphSchemaAction_K2NewNode::SpawnNode<UAnimGraphNode_LinkedInputPose>(Graph, NewNodePosition, EK2NewNodeFlags::None);

	TSharedPtr<IAnimationBlueprintEditor> AnimBlueprintEditor = AnimBlueprintEditorPtr.Pin();
	UBlueprint* Blueprint = AnimBlueprintEditor->GetBlueprintObj();
	UAnimGraphNode_LinkedInputPose::ReconstructLayerNodes(Blueprint);

	DetailLayoutBuilder->ForceRefreshDetails();
	
	return FReply::Handled();
}

EVisibility FAnimGraphDetails::OnGetNewInputPoseTextVisibility(TWeakPtr<SWidget> WeakInputsHeaderWidget) const
{
	return WeakInputsHeaderWidget.Pin()->IsHovered() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FAnimGraphDetails::OnRemoveInputPoseClicked(UAnimGraphNode_LinkedInputPose* InLinkedInputPose)
{
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveInputPose", "Remove Linked Input Pose"));
		Graph->RemoveNode(InLinkedInputPose);
	}

	TSharedPtr<IAnimationBlueprintEditor> AnimBlueprintEditor = AnimBlueprintEditorPtr.Pin();
	UBlueprint* Blueprint = AnimBlueprintEditor->GetBlueprintObj();
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	UAnimGraphNode_LinkedInputPose::ReconstructLayerNodes(Blueprint);
	
	DetailLayoutBuilder->ForceRefreshDetails();

	return FReply::Handled();
}

FText FAnimGraphDetails::OnGetGroupText() const
{
	UAnimGraphNode_Root* Root = FBlueprintEditorUtils::GetAnimGraphRoot(Graph);
	if (Root->Node.GetGroup() == FAnimNode_Root::DefaultSharedGroup)
	{
		return LOCTEXT("DefaultGroupSharedGroup", "Default Shared Group");
	}
	else if(Root->Node.GetGroup() == NAME_None)
	{
		return LOCTEXT("DefaultGroupUngrouped", "Ungrouped");
	}
		
	return FText::FromName(Root->Node.GetGroup());
}

void FAnimGraphDetails::OnGroupTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		// Remove excess whitespace and prevent categories with just spaces
		FText GroupName = FText::TrimPrecedingAndTrailing(NewText);
		if(GroupName.ToString().Equals(TEXT("Ungrouped")))
		{
			GroupName = FText::GetEmpty();
		}

		FBlueprintEditorUtils::SetAnimationGraphLayerGroup(Graph, GroupName);

		AnimBlueprintEditorPtr.Pin()->RefreshMyBlueprint();

		RefreshGroupSource();
	}
}

void FAnimGraphDetails::OnGroupSelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type SelectInfo)
{
	if(ProposedSelection.IsValid())
	{
		FText GroupName = *ProposedSelection.Get();
		if(GroupName.ToString().Equals(TEXT("Ungrouped")))
		{
			GroupName = FText::GetEmpty();
		}

		FBlueprintEditorUtils::SetAnimationGraphLayerGroup(Graph, GroupName);
		AnimBlueprintEditorPtr.Pin()->RefreshMyBlueprint();

		GroupListView.Pin()->ClearSelection();
		GroupComboButton.Pin()->SetIsOpen(false);

		RefreshGroupSource();
	}
}

TSharedRef< ITableRow > FAnimGraphDetails::MakeGroupViewWidget( TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return 
		SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock)
			.Text(*Item.Get())
		];
}

bool FAnimGraphDetails::IsInterface() const
{
	TSharedPtr<IAnimationBlueprintEditor> AnimBlueprintEditor = AnimBlueprintEditorPtr.Pin();
	return AnimBlueprintEditor->GetBlueprintObj()->BlueprintType == BPTYPE_Interface;
}

void FAnimGraphDetails::RefreshGroupSource()
{
	TSharedPtr<IAnimationBlueprintEditor> AnimBlueprintEditor = AnimBlueprintEditorPtr.Pin();
	UClass* Class = AnimBlueprintEditor->GetBlueprintObj()->SkeletonGeneratedClass;

	GroupSource.Empty();

	UAnimGraphNode_Root* Root = FBlueprintEditorUtils::GetAnimGraphRoot(Graph);
	if(Root->Node.GetGroup() != NAME_None)
	{
		GroupSource.Add(MakeShared<FText>(LOCTEXT("DefaultGroupUngrouped", "Ungrouped")));
	}

	// Pull groups from implemented functions
	for (TFieldIterator<UFunction> FunctionIt(Class, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
	{
		const UFunction* Function = *FunctionIt;
				
		if(Function->HasMetaData(FBlueprintMetadata::MD_AnimBlueprintFunction))
		{
			FText Group = FObjectEditorUtils::GetCategoryText(Function);

			if (!Group.IsEmpty())
			{
				bool bNewCategory = true;
				for (int32 GroupIndex = 0; GroupIndex < GroupSource.Num() && bNewCategory; ++GroupIndex)
				{
					bNewCategory &= !GroupSource[GroupIndex].Get()->EqualTo(Group);
				}

				if (bNewCategory)
				{
					GroupSource.Add(MakeShared<FText>(Group));
				}
			}
		}
	}

	if(GroupListView.IsValid())
	{
		GroupListView.Pin()->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE