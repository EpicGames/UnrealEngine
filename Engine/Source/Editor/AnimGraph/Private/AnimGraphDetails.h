// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"
#include "Animation/AnimBlueprint.h"
#include "Types/SlateEnums.h"
#include "Widgets/Views/SListView.h"

class IBlueprintEditor;
class IAnimationBlueprintEditor;
class FReply;
struct EVisibility;
class SWidget;
class UAnimGraphNode_Base;
class UAnimGraphNode_Root;
class UAnimGraphNode_LinkedInputPose;
class ITableRow;
class SComboButton;

/** Customization for editing animation graphs */
class FAnimGraphDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedPtr<IDetailCustomization> MakeInstance(TSharedPtr<IBlueprintEditor> InBlueprintEditor);

	FAnimGraphDetails(TSharedPtr<IAnimationBlueprintEditor> InAnimBlueprintEditor, UAnimBlueprint* AnimBlueprint)
		: AnimBlueprintEditorPtr(InAnimBlueprintEditor)
		, AnimBlueprintPtr(AnimBlueprint)
	{}

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	/** UI handlers */
	FReply OnAddNewInputPoseClicked();
	EVisibility OnGetNewInputPoseTextVisibility(TWeakPtr<SWidget> WeakInputsHeaderWidget) const;
	FReply OnRemoveInputPoseClicked(UAnimGraphNode_LinkedInputPose* InLinkedInputPose);
	FText OnGetGroupText() const;
	void OnGroupTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	void OnGroupSelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type SelectInfo);
	TSharedRef<ITableRow> MakeGroupViewWidget(TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable);

	/** Helper function to check if this is the anim graph details for an anim bp interface */
	bool IsInterface() const;

	/** Refresh the displayed groups */
	void RefreshGroupSource();

private:
	/** The Blueprint editor we are embedded in */
	TWeakPtr<IAnimationBlueprintEditor> AnimBlueprintEditorPtr;

	/** The blueprint we are editing */
	TWeakObjectPtr<UAnimBlueprint> AnimBlueprintPtr;

	/** The graph we are editing */
	UEdGraph* Graph;

	/** Hold onto the builder so we can refresh the panel */
	IDetailLayoutBuilder* DetailLayoutBuilder;

	/** Cached combo button widget */
	TWeakPtr<SComboButton> GroupComboButton;

	/** Cached list view widget */
	TWeakPtr<SListView<TSharedPtr<FText>>> GroupListView;

	/** A list of all group names to choose from */
	TArray<TSharedPtr<FText>> GroupSource;
};