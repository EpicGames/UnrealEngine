// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#include "AnimGraphCommands.h"
#include "Modules/ModuleManager.h"
#include "AnimNodeEditModes.h"
#include "EditorModeRegistry.h"
#include "AnimNodeEditMode.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "AnimGraphNode_PoseDriver.h"
#include "PoseDriverDetails.h"
#include "EditModes/AnimDynamicsEditMode.h"
#include "EditModes/TwoBoneIKEditMode.h"
#include "EditModes/ObserveBoneEditMode.h"
#include "EditModes/ModifyBoneEditMode.h"
#include "EditModes/FabrikEditMode.h"
#include "EditModes/PoseDriverEditMode.h"
#include "EditModes/SplineIKEditMode.h"
#include "EditModes/LookAtEditMode.h"
#include "EditModes/CCDIKEditMode.h"
#include "AnimBlueprintPinInfoDetails.h"
#include "BlueprintEditorModule.h"
#include "AnimGraphDetails.h"
#include "AnimationGraphSchema.h"
#include "AnimBlueprintCompiler.h"
#include "Animation/AnimAttributes.h"
#include "AnimGraphAttributes.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSync.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Features/IModularFeatures.h"
#include "PropertyAccessAnimBlueprintBinding.h"
#include "AnimBlueprintDetails.h"
#include "AnimSequenceTransformProviderLayerCustomization.h"

class FAnimGraphModule : public IModuleInterface
{
public:
	/** IModuleInterface interface */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FPropertyAccessAnimBlueprintBinding PropertyAccessAnimBlueprintBinding;
};

IMPLEMENT_MODULE(FAnimGraphModule, AnimGraph);

#define LOCTEXT_NAMESPACE "AnimGraphModule"

void FAnimGraphModule::StartupModule()
{
	FAnimGraphCommands::Register();

	FKismetCompilerContext::RegisterCompilerForBP(UAnimBlueprint::StaticClass(), [](UBlueprint* InBlueprint, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
	{
		return MakeShared<FAnimBlueprintCompilerContext>(CastChecked<UAnimBlueprint>(InBlueprint), InMessageLog, InCompileOptions);
	});
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.OverrideBPTypeForClass(UAnimInstance::StaticClass(), UAnimBlueprint::StaticClass());

	IModularFeatures::Get().RegisterModularFeature("PropertyAccessBlueprintBinding", &PropertyAccessAnimBlueprintBinding);

	// Register the editor modes
	FEditorModeRegistry::Get().RegisterMode<FAnimDynamicsEditMode>(AnimNodeEditModes::AnimDynamics, LOCTEXT("AnimDynamicsEditMode", "Anim Dynamics"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FAnimNodeEditMode>(AnimNodeEditModes::AnimNode, LOCTEXT("AnimNodeEditMode", "Anim Node"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FTwoBoneIKEditMode>(AnimNodeEditModes::TwoBoneIK, LOCTEXT("TwoBoneIKEditMode", "2-Bone IK"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FObserveBoneEditMode>(AnimNodeEditModes::ObserveBone, LOCTEXT("ObserveBoneEditMode", "Observe Bone"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FModifyBoneEditMode>(AnimNodeEditModes::ModifyBone, LOCTEXT("ModifyBoneEditMode", "Modify Bone"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FFabrikEditMode>(AnimNodeEditModes::Fabrik, LOCTEXT("FabrikEditMode", "Fabrik"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FPoseDriverEditMode>(AnimNodeEditModes::PoseDriver, LOCTEXT("PoseDriverEditMode", "PoseDriver"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FSplineIKEditMode>(AnimNodeEditModes::SplineIK, LOCTEXT("SplineIKEditMode", "Spline IK"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FLookAtEditMode>(AnimNodeEditModes::LookAt, LOCTEXT("LookAtEditMode", "LookAt"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FCCDIKEditMode>(AnimNodeEditModes::CCDIK, LOCTEXT("CCDIKEditMode", "CCD IK"), FSlateIcon(), false);

	// Register details customization
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(UAnimBlueprint::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FAnimBlueprintDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(UAnimGraphNode_PoseDriver::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPoseDriverDetails::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout("AnimBlueprintFunctionPinInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimBlueprintFunctionPinInfoDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("AnimSequenceTransformProviderLayer", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimSequenceTransformProviderLayerCustomization::MakeInstance));
	if(GIsEditor)
	{
		// Register built-in graph attributes
		UAnimGraphAttributes* AnimGraphAttributes = GetMutableDefault<UAnimGraphAttributes>();
		AnimGraphAttributes->LoadConfig();

		AnimGraphAttributes->Register(
			FAnimGraphAttributeDesc(
				UE::Anim::FAttributes::Pose,
				EAnimGraphAttributeBlend::Blendable,
				FSlateBrush(),
				LOCTEXT("Pose", "Pose"),
				LOCTEXT("PoseToolTip", "Pose: Transforms for all bones in the current LOD, blendable"),
				FAppStyle::GetSlateColor("AnimGraph.Attribute.Pose.Color"),
				EAnimGraphAttributesDisplayMode::HideOnPins)
		);

		AnimGraphAttributes->Register(
			FAnimGraphAttributeDesc(
				UE::Anim::FAttributes::Curves,
				EAnimGraphAttributeBlend::Blendable,
				*FAppStyle::GetBrush("AnimGraph.Attribute.Curves.Icon"),
				LOCTEXT("Curves", "Curves"),
				LOCTEXT("CurvesToolTip", "Curves: Floating point curve attributes, blendable"),
				FAppStyle::GetSlateColor("AnimGraph.Attribute.Curves.Color"),
				EAnimGraphAttributesDisplayMode::Automatic)
		);

		AnimGraphAttributes->Register(
			FAnimGraphAttributeDesc(
				UE::Anim::FAttributes::Attributes,
				EAnimGraphAttributeBlend::Blendable,
				*FAppStyle::GetBrush("AnimGraph.Attribute.Attributes.Icon"),
				LOCTEXT("Attributes", "Attributes"),
				LOCTEXT("AttributesToolTip", "Attributes: Misc. custom attributes, blendable"),
				FAppStyle::GetSlateColor("AnimGraph.Attribute.Attributes.Color"),
				EAnimGraphAttributesDisplayMode::Automatic)
		);

		AnimGraphAttributes->Register(
			FAnimGraphAttributeDesc(
				UE::Anim::FAnimSync::Attribute,
				EAnimGraphAttributeBlend::NonBlendable,
				*FAppStyle::Get().GetBrush("AnimGraph.Attribute.Sync.Icon"),
				LOCTEXT("Sync", "Sync"),
				LOCTEXT("SyncToolTip", "Sync: Synchronization between the playback of different assets present in the graph, not blendable"),
				FAppStyle::Get().GetSlateColor("AnimGraph.Attribute.Sync.Color"),
				EAnimGraphAttributesDisplayMode::Automatic)
		);

		AnimGraphAttributes->Register(
			FAnimGraphAttributeDesc(
				UE::Anim::IAnimRootMotionProvider::AttributeName,
				EAnimGraphAttributeBlend::Blendable,
				*FAppStyle::Get().GetBrush("AnimGraph.Attribute.RootMotionDelta.Icon"),
				LOCTEXT("RootMotionDelta", "Root Motion Delta"),
				LOCTEXT("RootMotionDeltaToolTip", "Root Motion Delta: Per-frame root motion delta transform, blendable"),
				FAppStyle::Get().GetSlateColor("AnimGraph.Attribute.RootMotionDelta.Color"),
				EAnimGraphAttributesDisplayMode::Automatic)
		);

		AnimGraphAttributes->Register(
			FAnimGraphAttributeDesc(
				UE::Anim::IInertializationRequester::Attribute,
				EAnimGraphAttributeBlend::NonBlendable,
				*FAppStyle::Get().GetBrush("AnimGraph.Attribute.InertialBlending.Icon"),
				LOCTEXT("InertialBlending", "Inertialize"),
				LOCTEXT("InertialBlendingToolTip", "Inertial Blending: Used to blend between poses preserving bone velocities, not blendable"),
				FAppStyle::Get().GetSlateColor("AnimGraph.Attribute.InertialBlending.Color"),
				EAnimGraphAttributesDisplayMode::Automatic)
		);
	}
	
	// Register BP-editor function customization once the Kismet module is loaded.
	if (FModuleManager::Get().IsModuleLoaded("Kismet"))
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::GetModuleChecked<FBlueprintEditorModule>("Kismet");
		BlueprintEditorModule.RegisterGraphCustomization(GetDefault<UAnimationGraphSchema>(), FOnGetGraphCustomizationInstance::CreateStatic(&FAnimGraphDetails::MakeInstance));
	}
	else
	{
		FModuleManager::Get().OnModulesChanged().AddLambda([](FName InModuleName, EModuleChangeReason InReason)
		{
			if (InReason == EModuleChangeReason::ModuleLoaded && InModuleName == "Kismet")
			{
				FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
				BlueprintEditorModule.RegisterGraphCustomization(GetDefault<UAnimationGraphSchema>(), FOnGetGraphCustomizationInstance::CreateStatic(&FAnimGraphDetails::MakeInstance));
			}
		});
	}
}

void FAnimGraphModule::ShutdownModule()
{
	// Unregister the editor modes
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::CCDIK);
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::SplineIK);
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::PoseDriver);
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::Fabrik);
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::ModifyBone);
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::ObserveBone);
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::TwoBoneIK);
	FEditorModeRegistry::Get().UnregisterMode(AnimNodeEditModes::AnimNode);

	IModularFeatures::Get().UnregisterModularFeature("PropertyAccessBlueprintBinding", &PropertyAccessAnimBlueprintBinding);
	
	// Unregister details customization
	if (UObjectInitialized() && FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		// Unregister details customization
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyModule)
		{
			PropertyModule->UnregisterCustomClassLayout(UAnimGraphNode_PoseDriver::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomPropertyTypeLayout("AnimBlueprintFunctionPinInfo");
		}

		FBlueprintEditorModule* BlueprintEditorModule = FModuleManager::GetModulePtr<FBlueprintEditorModule>("Kismet");
		if(BlueprintEditorModule)
		{
			BlueprintEditorModule->UnregisterGraphCustomization(GetDefault<UAnimationGraphSchema>());
		}
	}
}

#undef LOCTEXT_NAMESPACE
