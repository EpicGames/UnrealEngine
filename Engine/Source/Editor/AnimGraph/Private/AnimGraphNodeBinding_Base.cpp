// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNodeBinding_Base.h"
#include "IPropertyAccessEditor.h"
#include "Features/IModularFeatures.h"
#include "AnimationGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "FindInBlueprintManager.h"
#include "IPropertyAccessBlueprintBinding.h"
#include "ScopedTransaction.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Fonts/FontMeasure.h"
#include "Widgets/Layout/SSpacer.h"
#include "AnimBlueprintExtension_Base.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNodeBinding_Base)

#define LOCTEXT_NAMESPACE "UAnimGraphNodeBinding_Base"

UScriptStruct* UAnimGraphNodeBinding_Base::GetAnimNodeHandlerStruct() const
{
	return FAnimNodeExposedValueHandler_PropertyAccess::StaticStruct();
}

void UAnimGraphNodeBinding_Base::OnInternalPinCreation(UAnimGraphNode_Base* InNode)
{
	if (InNode->HasValidBlueprint())
	{
		// Update any binding's display text
		IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
		for (TPair<FName, FAnimGraphNodePropertyBinding>& BindingPair : PropertyBindings)
		{
			BindingPair.Value.PathAsText = PropertyAccessEditor.MakeTextPath(BindingPair.Value.PropertyPath, InNode->GetBlueprint()->SkeletonGeneratedClass);
		}
	}
}

void UAnimGraphNodeBinding_Base::RecalculateBindingType(UAnimGraphNode_Base* InNode, FAnimGraphNodePropertyBinding& InBinding)
{
	if (FProperty* BindingProperty = InNode->GetPinProperty(InBinding.PropertyName))
	{
		// Use the inner for array properties
		if (BindingProperty->IsA<FArrayProperty>() && InBinding.ArrayIndex != INDEX_NONE)
		{
			BindingProperty = CastFieldChecked<FArrayProperty>(BindingProperty)->Inner;
		}

		UAnimBlueprint* AnimBlueprint = InNode->GetAnimBlueprint();

		IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
		const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();

		FProperty* LeafProperty = nullptr;
		int32 ArrayIndex = INDEX_NONE;
		FPropertyAccessResolveResult Result = PropertyAccessEditor.ResolvePropertyAccess(AnimBlueprint->SkeletonGeneratedClass, InBinding.PropertyPath, LeafProperty, ArrayIndex);
		if (Result.Result == EPropertyAccessResolveResult::Succeeded)
		{
			if (LeafProperty)
			{
				Schema->ConvertPropertyToPinType(LeafProperty, InBinding.PinType);

				if (PropertyAccessEditor.GetPropertyCompatibility(LeafProperty, BindingProperty) == EPropertyAccessCompatibility::Promotable)
				{
					InBinding.bIsPromotion = true;
					Schema->ConvertPropertyToPinType(LeafProperty, InBinding.PromotedPinType);
				}
				else
				{
					InBinding.bIsPromotion = false;
					InBinding.PromotedPinType = InBinding.PinType;
				}
			}
		}
	}
}

void UAnimGraphNodeBinding_Base::OnReconstructNode(UAnimGraphNode_Base* InNode)
{
	if (InNode->HasValidBlueprint())
	{
		// Refresh bindings
		for (auto& BindingPair : PropertyBindings)
		{
			RecalculateBindingType(InNode, BindingPair.Value);
		}
	}
}

bool UAnimGraphNodeBinding_Base::HasBinding(FName InBindingName, bool bCheckArrayIndexName) const
{
	if (bCheckArrayIndexName)
	{
		// This checks for the 'pin name' of an array element
		return PropertyBindings.Find(InBindingName) != nullptr;
	}
	else
	{
		// Comparison without name index to deal with arrays
		// This checks for any pin of an array element and the array itself
		const FName ComparisonName = FName(InBindingName, 0);

		for (const TPair<FName, FAnimGraphNodePropertyBinding>& BindingPair : PropertyBindings)
		{
			if (ComparisonName == FName(BindingPair.Key, 0))
			{
				return true;
			}
		}

		return false;
	}
}

void UAnimGraphNodeBinding_Base::RemoveBindings(FName InBindingName)
{
	const FName ComparisonName = FName(InBindingName, 0);

	for (auto Iter = PropertyBindings.CreateIterator(); Iter; ++Iter)
	{
		if (ComparisonName == FName(Iter.Key(), 0))
		{
			Iter.RemoveCurrent();
		}
	}
}

void UAnimGraphNodeBinding_Base::AddPinSearchMetaDataInfo(const UEdGraphPin* InPin, FName InBindingName, TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	if (const FAnimGraphNodePropertyBinding* BindingInfo = PropertyBindings.Find(InBindingName))
	{
		OutTaggedMetaData.Add(FSearchTagDataPair(FText::FromString(TEXT("Binding")), BindingInfo->PathAsText));
	}
}

void UAnimGraphNodeBinding_Base::HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName)
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	UClass* SkeletonVariableClass = FBlueprintEditorUtils::GetMostUpToDateClass(InVariableClass);
	UBlueprint* OuterBlueprint = GetTypedOuter<UBlueprint>();

	if(OuterBlueprint)
	{
		// See if any of bindings reference the variable
		for (auto& BindingPair : PropertyBindings)
		{
			TArray<int32> RenameIndices;
			IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
			ResolveArgs.bUseMostUpToDateClasses = true;
			ResolveArgs.PropertyFunction = [InOldVarName, SkeletonVariableClass, &RenameIndices](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
			{
				UClass* OwnerClass = InProperty->GetOwnerClass();
				if (OwnerClass && InProperty->GetFName() == InOldVarName && OwnerClass->IsChildOf(SkeletonVariableClass))
				{
					RenameIndices.Add(InSegmentIndex);
				}
			};

			PropertyAccessEditor.ResolvePropertyAccess(OuterBlueprint->SkeletonGeneratedClass, BindingPair.Value.PropertyPath, ResolveArgs);

			// Rename any references we found
			for (const int32& RenameIndex : RenameIndices)
			{
				BindingPair.Value.PropertyPath[RenameIndex] = InNewVarName.ToString();
				BindingPair.Value.PathAsText = PropertyAccessEditor.MakeTextPath(BindingPair.Value.PropertyPath, OuterBlueprint->SkeletonGeneratedClass);
			}
		}
	}
}

void UAnimGraphNodeBinding_Base::HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName)
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	UBlueprint* OuterBlueprint = GetTypedOuter<UBlueprint>();

	if(OuterBlueprint)
	{
		UClass* SkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(InFunctionClass);

		// See if any of bindings reference the variable
		for (auto& BindingPair : PropertyBindings)
		{
			TArray<int32> RenameIndices;
			IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
			ResolveArgs.bUseMostUpToDateClasses = true;
			ResolveArgs.FunctionFunction = [InOldFuncName, SkeletonFunctionClass, &RenameIndices](int32 InSegmentIndex, UFunction* InFunction, FProperty* InReturnProperty)
			{
				const UClass* OwnerSkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(InFunction->GetOuterUClass());
				if (OwnerSkeletonFunctionClass && InFunction->GetFName() == InOldFuncName && OwnerSkeletonFunctionClass->IsChildOf(SkeletonFunctionClass))
				{
					RenameIndices.Add(InSegmentIndex);
				}
			};

			PropertyAccessEditor.ResolvePropertyAccess(OuterBlueprint->SkeletonGeneratedClass, BindingPair.Value.PropertyPath, ResolveArgs);

			// Rename any references we found
			for (const int32& RenameIndex : RenameIndices)
			{
				BindingPair.Value.PropertyPath[RenameIndex] = InNewFuncName.ToString();
				BindingPair.Value.PathAsText = PropertyAccessEditor.MakeTextPath(BindingPair.Value.PropertyPath, OuterBlueprint->SkeletonGeneratedClass);
			}
		}
	}
}

void UAnimGraphNodeBinding_Base::ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement)
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	UBlueprint* OuterBlueprint = GetTypedOuter<UBlueprint>();

	if (OuterBlueprint)
	{
		UClass* SkeletonClass = InBlueprint->SkeletonGeneratedClass;

		FMemberReference Source = InSource;
		FProperty* SourceProperty = Source.ResolveMember<FProperty>(InBlueprint);
		FMemberReference Replacement = InReplacement;
		FProperty* ReplacementProperty = Replacement.ResolveMember<FProperty>(InReplacementBlueprint);

		// See if any of bindings reference the variable
		for (auto& BindingPair : PropertyBindings)
		{
			TArray<int32> ReplaceIndices;
			IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
			ResolveArgs.bUseMostUpToDateClasses = true;
			ResolveArgs.PropertyFunction = [SourceProperty, &ReplaceIndices](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
			{
				if (InProperty == SourceProperty)
				{
					ReplaceIndices.Add(InSegmentIndex);
				}
			};

			PropertyAccessEditor.ResolvePropertyAccess(GetTypedOuter<UBlueprint>()->SkeletonGeneratedClass, BindingPair.Value.PropertyPath, ResolveArgs);

			// Replace any references we found
			for (const int32& RenameIndex : ReplaceIndices)
			{
				BindingPair.Value.PropertyPath[RenameIndex] = ReplacementProperty->GetName();
				BindingPair.Value.PathAsText = PropertyAccessEditor.MakeTextPath(BindingPair.Value.PropertyPath, OuterBlueprint->SkeletonGeneratedClass);
			}
		}
	}
}

bool UAnimGraphNodeBinding_Base::ReferencesVariable(const FName& InVarName, const UStruct* InScope) const
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	UBlueprint* OuterBlueprint = GetTypedOuter<UBlueprint>();

	if(OuterBlueprint)
	{
		const UClass* SkeletonVariableClass = FBlueprintEditorUtils::GetMostUpToDateClass(Cast<UClass>(InScope));

		// See if any of bindings reference the variable
		for (const auto& BindingPair : PropertyBindings)
		{
			bool bReferencesVariable = false;

			IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
			ResolveArgs.bUseMostUpToDateClasses = true;
			ResolveArgs.PropertyFunction = [InVarName, SkeletonVariableClass, &bReferencesVariable](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
			{
				if (SkeletonVariableClass)
				{
					const UClass* OwnerSkeletonVariableClass = FBlueprintEditorUtils::GetMostUpToDateClass(Cast<UClass>(InProperty->GetOwnerStruct()));

					if (OwnerSkeletonVariableClass && InProperty->GetFName() == InVarName && OwnerSkeletonVariableClass->IsChildOf(SkeletonVariableClass))
					{
						bReferencesVariable = true;
					}
				}
				else if (InProperty->GetFName() == InVarName)
				{
					bReferencesVariable = true;
				}
			};

			PropertyAccessEditor.ResolvePropertyAccess(OuterBlueprint->SkeletonGeneratedClass, BindingPair.Value.PropertyPath, ResolveArgs);

			if (bReferencesVariable)
			{
				return true;
			}
		}
	}

	return false;
}

bool UAnimGraphNodeBinding_Base::ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	UBlueprint* OuterBlueprint = GetTypedOuter<UBlueprint>();

	if(OuterBlueprint)
	{
		const UClass* SkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(Cast<UClass>(InScope));

		// See if any of bindings reference the function
		for (const auto& BindingPair : PropertyBindings)
		{
			bool bReferencesFunction = false;

			IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
			ResolveArgs.bUseMostUpToDateClasses = true;
			ResolveArgs.FunctionFunction = [InFunctionName, SkeletonFunctionClass, &bReferencesFunction](int32 InSegmentIndex, UFunction* InFunction, FProperty* InProperty)
			{
				if (SkeletonFunctionClass)
				{
					const UClass* OwnerSkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(InFunction->GetOuterUClass());

					if (OwnerSkeletonFunctionClass && InFunction->GetFName() == InFunctionName && OwnerSkeletonFunctionClass->IsChildOf(SkeletonFunctionClass))
					{
						bReferencesFunction = true;
					}
				}
				else if (InFunction->GetFName() == InFunctionName)
				{
					bReferencesFunction = true;
				}
			};

			PropertyAccessEditor.ResolvePropertyAccess(OuterBlueprint->SkeletonGeneratedClass, BindingPair.Value.PropertyPath, ResolveArgs);

			if (bReferencesFunction)
			{
				return true;
			}
		}
	}

	// Check private member anim node function binding 
	return false;
}

void UAnimGraphNodeBinding_Base::UpdateBindingNames(TFunctionRef<FString(const FString& InOldName)> InModifierFunction)
{
	if (UAnimGraphNode_Base* Node = GetTypedOuter<UAnimGraphNode_Base>())
	{
		TMap<FName, FAnimGraphNodePropertyBinding> NewBindings;

		for (const TPair<FName, FAnimGraphNodePropertyBinding>& BindingPair : PropertyBindings)
		{
			FString BindingNameString = BindingPair.Key.ToString();

			FString NewNameString = InModifierFunction(BindingNameString);

			if (NewNameString != BindingNameString)
			{
				FName NewName = *NewNameString;
				FAnimGraphNodePropertyBinding NewBinding = BindingPair.Value;
				NewBinding.PropertyName = NewName;
				NewBindings.Add(NewName, NewBinding);
			}
		}

		if (NewBindings.Num() > 0)
		{
			PropertyBindings = NewBindings;
		}
	}
}

void UAnimGraphNodeBinding_Base::ProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	if (UAnimGraphNode_Base* Node = GetTypedOuter<UAnimGraphNode_Base>())
	{
		if(UAnimBlueprint* AnimBlueprint = Node->GetTypedOuter<UAnimBlueprint>())
		{
			UAnimBlueprintExtension_Base* Extension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_Base>(AnimBlueprint);

			// Gather pins that have an associated evaluation handler
			Extension->ProcessNonPosePins(Node, InCompilationContext, OutCompiledData, UAnimBlueprintExtension_Base::EPinProcessingFlags::All);
		}
	}
}

#if WITH_EDITOR

TSharedRef<SWidget> UAnimGraphNodeBinding_Base::MakePropertyBindingWidget(const UAnimGraphNode_Base::FAnimPropertyBindingWidgetArgs& InArgs)
{
	UAnimGraphNode_Base* FirstAnimGraphNode = InArgs.Nodes[0];
	const bool bMultiSelect = InArgs.Nodes.Num() > 1;
	
	if(FirstAnimGraphNode->HasValidBlueprint() && IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
	{
		UBlueprint* Blueprint = FirstAnimGraphNode->GetAnimBlueprint();
		
		int32 PinArrayIndex = InArgs.PinName.GetNumber() - 1;
		const bool bIsArrayOrArrayElement = InArgs.PinProperty->IsA<FArrayProperty>();
		const bool bIsArrayElement = bIsArrayOrArrayElement && PinArrayIndex != INDEX_NONE && InArgs.bPropertyIsOnFNode;
		const bool bIsArray = bIsArrayOrArrayElement && PinArrayIndex == INDEX_NONE && InArgs.bPropertyIsOnFNode;

		FProperty* PropertyToBindTo = bIsArrayElement ? CastField<FArrayProperty>(InArgs.PinProperty)->Inner : InArgs.PinProperty;
		
		// Properties could potentially be removed underneath this widget, so keep a TFieldPath reference to them
		TFieldPath<FProperty> BindingPropertyPath(PropertyToBindTo);
		TFieldPath<FProperty> PinPropertyPath(InArgs.PinProperty);
		
		auto OnCanBindProperty = [BindingPropertyPath](FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain)
		{
			// Note: We support type promotion here
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
			FProperty* BindingProperty = BindingPropertyPath.Get();
			return BindingProperty && PropertyAccessEditor.GetPropertyCompatibility(InProperty, BindingProperty) != EPropertyAccessCompatibility::Incompatible;
		};

		auto OnCanBindFunction = [BindingPropertyPath](UFunction* InFunction)
		{
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
			FProperty* BindingProperty = BindingPropertyPath.Get();
			
			// Note: We support type promotion here
			return InFunction->NumParms == 1 
				&& BindingProperty != nullptr && PropertyAccessEditor.GetPropertyCompatibility(InFunction->GetReturnProperty(), BindingProperty) != EPropertyAccessCompatibility::Incompatible
				&& InFunction->HasAnyFunctionFlags(FUNC_BlueprintPure);
		};

		auto OnAddBinding = [InArgs, Blueprint, BindingPropertyPath, PinPropertyPath, bIsArrayElement, bIsArray](FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
		{
			const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

			for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
			{
				if(UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetMutableBinding()))
				{
					Binding->Modify();

					// Reset to default so that references are not preserved
					FProperty* BindingProperty = BindingPropertyPath.Get();
					if(BindingProperty && BindingProperty->GetOwner<UStruct>() && AnimGraphNode->GetFNodeType()->IsChildOf(BindingProperty->GetOwner<UStruct>()) && BindingProperty->IsA<FObjectPropertyBase>())
					{
						void* PropertyAddress = BindingProperty->ContainerPtrToValuePtr<void>(AnimGraphNode->GetFNode());
						BindingProperty->InitializeValue(PropertyAddress);
					}
				
					// Pins are exposed if we have a binding or not - and after running this we do.
					InArgs.OnSetPinVisibility.ExecuteIfBound(AnimGraphNode, true, InArgs.OptionalPinIndex);

					// Need to break all pin links now we have a binding
					if(UEdGraphPin* Pin = AnimGraphNode->FindPin(InArgs.PinName))
					{
						Pin->BreakAllPinLinks();
					}

					if(bIsArray)
					{
						// Remove bindings for array elements if this is an array
						FName ComparisonName(InArgs.BindingName, 0);
						for(auto Iter = Binding->PropertyBindings.CreateIterator(); Iter; ++Iter)
						{
							if(ComparisonName == FName(Iter.Key(), 0))
							{
								Iter.RemoveCurrent();
							}
						}
					}
					else if(bIsArrayElement)
					{
						// If we are an array element, remove only whole-array bindings
						FName ComparisonName(InArgs.BindingName, 0);
						for(auto Iter = Binding->PropertyBindings.CreateIterator(); Iter; ++Iter)
						{
							if(ComparisonName == Iter.Key())
							{
								Iter.RemoveCurrent();
							}
						}
					}
				
					const FFieldVariant& LeafField = InBindingChain.Last().Field;

					FProperty* PinProperty = PinPropertyPath.Get();
					if(PinProperty && BindingProperty)
					{
						FAnimGraphNodePropertyBinding PropertyBinding;
						PropertyBinding.PropertyName = InArgs.BindingName;
						if(bIsArrayElement)
						{
							// Pull array index from the pin's FName if this is an array property
							PropertyBinding.ArrayIndex = InArgs.PinName.GetNumber() - 1;
						}
						PropertyAccessEditor.MakeStringPath(InBindingChain, PropertyBinding.PropertyPath);
						PropertyBinding.PathAsText = PropertyAccessEditor.MakeTextPath(PropertyBinding.PropertyPath, Blueprint->SkeletonGeneratedClass);
						PropertyBinding.Type = LeafField.IsA<UFunction>() ? EAnimGraphNodePropertyBindingType::Function : EAnimGraphNodePropertyBindingType::Property;
						PropertyBinding.bIsBound = true;
						RecalculateBindingType(AnimGraphNode, PropertyBinding);

						Binding->PropertyBindings.Add(InArgs.BindingName, PropertyBinding);
					}
				}

				AnimGraphNode->ReconstructNode();
			}

			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		};

		auto OnRemoveBinding = [InArgs, Blueprint](FName InPropertyName)
		{
			for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
			{
				if (UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetMutableBinding()))
				{
					Binding->Modify();
					Binding->PropertyBindings.Remove(InArgs.BindingName);
				}
			}

			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		};

		auto CanRemoveBinding = [InArgs](FName InPropertyName)
		{
			for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
			{
				if(const UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
				{
					if(Binding->PropertyBindings.Contains(InArgs.BindingName))
					{
						return true;
					}
				}
			}

			return false;
		}; 

		enum class ECurrentValueType : int32
		{
			None,
			Pin,
			Binding,
			Dynamic,
			MultipleValues,
		};

		auto CurrentBindingText = [InArgs]()
		{
			ECurrentValueType CurrentValueType = ECurrentValueType::None;

			const FText MultipleValues = LOCTEXT("MultipleValuesLabel", "Multiple Values");
			const FText Bind = LOCTEXT("BindLabel", "Bind");
			const FText ExposedAsPin = LOCTEXT("ExposedAsPinLabel", "Pin");
			const FText Dynamic = LOCTEXT("DynamicLabel", "Dynamic");
			FText CurrentValue = Bind;

			auto SetAssignValue = [&CurrentValueType, &CurrentValue, &MultipleValues](const FText& InValue, ECurrentValueType InType)
			{
				if(CurrentValueType != ECurrentValueType::MultipleValues)
				{
					if(CurrentValueType == ECurrentValueType::None)
					{
						CurrentValueType = InType;
						CurrentValue = InValue;
					}
					else if(CurrentValueType == InType)
					{
						if(!CurrentValue.EqualTo(InValue))
						{
							CurrentValueType = ECurrentValueType::MultipleValues;
							CurrentValue = MultipleValues;
						}
					}
					else
					{
						CurrentValueType = ECurrentValueType::MultipleValues;
						CurrentValue = MultipleValues;
					}
				}
			};

			const FName ComparisonName = FName(InArgs.PinName, 0);

			for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
			{
				if(const UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
				{
					if(const FAnimGraphNodePropertyBinding* BindingPtr = Binding->PropertyBindings.Find(InArgs.BindingName))
					{
						SetAssignValue(BindingPtr->PathAsText, ECurrentValueType::Binding);
					}
					else if(AnimGraphNode->AlwaysDynamicProperties.Find(ComparisonName))
					{
						SetAssignValue(Dynamic, ECurrentValueType::Dynamic);
					}
					else
					{
						TArrayView<FOptionalPinFromProperty> OptionalPins; 
						InArgs.OnGetOptionalPins.ExecuteIfBound(AnimGraphNode, OptionalPins);
						if(OptionalPins.IsValidIndex(InArgs.OptionalPinIndex) && OptionalPins[InArgs.OptionalPinIndex].bShowPin)
						{
							SetAssignValue(InArgs.bOnGraphNode ? Bind : ExposedAsPin, ECurrentValueType::Pin);
						}
						else
						{
							SetAssignValue(Bind, ECurrentValueType::None);
						}
					}
				}
			}

			return CurrentValue;
		};

		auto CurrentBindingToolTipText = [InArgs]()
		{
			ECurrentValueType CurrentValueType = ECurrentValueType::None;

			const FText MultipleValues = LOCTEXT("MultipleValuesToolTip", "Bindings Have Multiple Values");
			const FText ExposedAsPin = LOCTEXT("ExposedAsPinToolTip", "Exposed As a Pin on the Node");
			const FText BindValue = LOCTEXT("BindValueToolTip", "Bind This Value");
			const FText DynamicValue = LOCTEXT("DynamicValueToolTip", "Dynamic value that can be set externally");
			FText CurrentValue;
			
			auto SetAssignValue = [&CurrentValueType, &CurrentValue, &MultipleValues](const FText& InValue, ECurrentValueType InType)
			{
				if(CurrentValueType != ECurrentValueType::MultipleValues)
				{
					if(CurrentValueType == ECurrentValueType::None)
					{
						CurrentValueType = InType;
						CurrentValue = InValue;
					}
					else if(CurrentValueType == InType)
					{
						if(!CurrentValue.EqualTo(InValue))
						{
							CurrentValueType = ECurrentValueType::MultipleValues;
							CurrentValue = MultipleValues;
						}
					}
					else
					{
						CurrentValueType = ECurrentValueType::MultipleValues;
						CurrentValue = MultipleValues;
					}
				}
			};

			const FName ComparisonName = FName(InArgs.PinName, 0);

			for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
			{
				if(const UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
				{
					if(const FAnimGraphNodePropertyBinding* BindingPtr = Binding->PropertyBindings.Find(InArgs.BindingName))
					{
						if(BindingPtr->PathAsText.IsEmpty())
						{
							SetAssignValue(BindValue, ECurrentValueType::Binding);
						}
						else
						{
							IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
							const FText UnderlyingPath = PropertyAccessEditor.MakeTextPath(BindingPtr->PropertyPath);
							const FText& CompilationContext = BindingPtr->CompiledContext;
							const FText& CompilationContextDesc = BindingPtr->CompiledContextDesc;
							if(CompilationContext.IsEmpty() && CompilationContextDesc.IsEmpty())
							{
								SetAssignValue(FText::Format(LOCTEXT("BindingToolTipFormat", "Pin is bound to property '{0}'\nNative: {1}"), BindingPtr->PathAsText, UnderlyingPath), ECurrentValueType::Binding);
							}
							else
							{
								SetAssignValue(FText::Format(LOCTEXT("BindingToolTipFormatWithDesc", "Pin is bound to property '{0}'\nNative: {1}\n{2}\n{2}"), BindingPtr->PathAsText, UnderlyingPath, CompilationContext, CompilationContextDesc), ECurrentValueType::Binding);
							}
						}
					}
					else if(AnimGraphNode->AlwaysDynamicProperties.Find(ComparisonName))
					{
						SetAssignValue(DynamicValue, ECurrentValueType::Dynamic);
					}
					else
					{
						TArrayView<FOptionalPinFromProperty> OptionalPins; 
						InArgs.OnGetOptionalPins.ExecuteIfBound(AnimGraphNode, OptionalPins);
						if(OptionalPins.IsValidIndex(InArgs.OptionalPinIndex) && OptionalPins[InArgs.OptionalPinIndex].bShowPin)
						{
							SetAssignValue(InArgs.bOnGraphNode ? BindValue : ExposedAsPin, ECurrentValueType::Pin);
						}
						else
						{
							SetAssignValue(BindValue, ECurrentValueType::None);
						}
					}
				}
			}

			return CurrentValue;
		};

		auto CurrentBindingImage = [InArgs, BindingPropertyPath]() -> const FSlateBrush*
		{
			static FName PropertyIcon(TEXT("Kismet.VariableList.TypeIcon"));
			static FName FunctionIcon(TEXT("GraphEditor.Function_16x"));

			EAnimGraphNodePropertyBindingType BindingType = EAnimGraphNodePropertyBindingType::None;
			for(UObject* OuterObject : InArgs.Nodes)
			{
				if(UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(OuterObject))
				{
					if(const UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
					{
						TArrayView<FOptionalPinFromProperty> OptionalPins; 
						InArgs.OnGetOptionalPins.ExecuteIfBound(AnimGraphNode, OptionalPins);
						if(OptionalPins.IsValidIndex(InArgs.OptionalPinIndex) && OptionalPins[InArgs.OptionalPinIndex].bShowPin)
						{
							BindingType = EAnimGraphNodePropertyBindingType::None;
							break;
						}
						else if(const FAnimGraphNodePropertyBinding* BindingPtr = Binding->PropertyBindings.Find(InArgs.BindingName))
						{
							if(BindingType == EAnimGraphNodePropertyBindingType::None)
							{
								BindingType = BindingPtr->Type;
							}
							else if(BindingType != BindingPtr->Type)
							{
								BindingType = EAnimGraphNodePropertyBindingType::None;
								break;
							}
						}
						else if(BindingType != EAnimGraphNodePropertyBindingType::None)
						{
							BindingType = EAnimGraphNodePropertyBindingType::None;
							break;
						}
					}
				}
			}

			if (BindingType == EAnimGraphNodePropertyBindingType::Function)
			{
				return FAppStyle::GetBrush(FunctionIcon);
			}
			else
			{
				const UAnimationGraphSchema* AnimationGraphSchema = GetDefault<UAnimationGraphSchema>();
				FProperty* BindingProperty = BindingPropertyPath.Get();
				FEdGraphPinType PinType;
				if(BindingProperty != nullptr && AnimationGraphSchema->ConvertPropertyToPinType(BindingProperty, PinType))
				{
					return FBlueprintEditorUtils::GetIconFromPin(PinType, false);
				}
				else
				{
					return FAppStyle::GetBrush(PropertyIcon);
				}
			}
		};

		auto CurrentBindingColor = [InArgs, BindingPropertyPath]() -> FLinearColor
		{
			const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

			FLinearColor BindingColor = FLinearColor::Gray;

			FProperty* BindingProperty = BindingPropertyPath.Get();
			FEdGraphPinType PinType;
			if(BindingProperty != nullptr && Schema->ConvertPropertyToPinType(BindingProperty, PinType))
			{
				BindingColor = Schema->GetPinTypeColor(PinType);

				enum class EPromotionState
				{
					NotChecked,
					NotPromoted,
					Promoted,
				} Promotion = EPromotionState::NotChecked;

				for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
				{
					if(const UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
					{
						TArrayView<FOptionalPinFromProperty> OptionalPins; 
						InArgs.OnGetOptionalPins.ExecuteIfBound(AnimGraphNode, OptionalPins);
						if(const FAnimGraphNodePropertyBinding* BindingPtr = Binding->PropertyBindings.Find(InArgs.BindingName))
						{
							if(Promotion == EPromotionState::NotChecked)
							{
								if(BindingPtr->bIsPromotion)
								{
									Promotion = EPromotionState::Promoted;
									BindingColor = Schema->GetPinTypeColor(BindingPtr->PromotedPinType);
								}
								else
								{
									Promotion = EPromotionState::NotPromoted;
								}
							}
							else
							{
								EPromotionState NewPromotion = BindingPtr->bIsPromotion ? EPromotionState::Promoted : EPromotionState::NotPromoted;
								if(Promotion != NewPromotion)
								{
									BindingColor = FLinearColor::Gray;
									break;
								}
							}
						}
						else if(OptionalPins.IsValidIndex(InArgs.OptionalPinIndex) && OptionalPins[InArgs.OptionalPinIndex].bShowPin)
						{
							if(Promotion == EPromotionState::NotChecked)
							{
								Promotion = EPromotionState::NotPromoted;
							}
							else if(Promotion == EPromotionState::Promoted)
							{
								BindingColor = FLinearColor::Gray;
								break;
							}
						}
					}
				}
			}

			return BindingColor;
		};

		auto AddMenuExtension = [InArgs, Blueprint, BindingPropertyPath, PinPropertyPath](FMenuBuilder& InMenuBuilder)
		{
			InMenuBuilder.BeginSection("Pins", LOCTEXT("Pin", "Pin"));
			{
				auto ExposeAsPin = [InArgs, Blueprint]()
				{
					bool bHasBinding = false;

					for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
					{
						if(AnimGraphNode->HasBinding(InArgs.BindingName))
						{
							bHasBinding = true;
							break;
						}
					}

					{
						FScopedTransaction Transaction(LOCTEXT("PinExposure", "Pin Exposure"));

						// Switching from non-pin to pin, remove any bindings
						for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
						{
							if(UAnimGraphNodeBinding_Base* Binding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetMutableBinding()))
							{
								Binding->Modify();
								AnimGraphNode->Modify();

								TArrayView<FOptionalPinFromProperty> OptionalPins; 
								InArgs.OnGetOptionalPins.ExecuteIfBound(AnimGraphNode, OptionalPins);
								const bool bVisible = OptionalPins.IsValidIndex(InArgs.OptionalPinIndex) && OptionalPins[InArgs.OptionalPinIndex].bShowPin;
								InArgs.OnSetPinVisibility.ExecuteIfBound(AnimGraphNode, !bVisible || bHasBinding, InArgs.OptionalPinIndex);

								// Remove all bindings that match the property, array or array elements
								const FName ComparisonName = FName(InArgs.BindingName, 0);
								for(auto It = Binding->PropertyBindings.CreateIterator(); It; ++It)
								{
									if(ComparisonName == FName(It.Key(), 0))
									{
										It.RemoveCurrent();
									}
								}
							}
						}

						FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
					}
				}; 

				auto GetExposedAsPinCheckState = [InArgs]()
				{
					bool bPinShown = false;
					bool bHasBinding = false;

					for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
					{
						if(AnimGraphNode->HasBinding(InArgs.BindingName))
						{
							bHasBinding = true;
							break;
						}

						TArrayView<FOptionalPinFromProperty> OptionalPins; 
						InArgs.OnGetOptionalPins.ExecuteIfBound(AnimGraphNode, OptionalPins);
						bPinShown |= OptionalPins.IsValidIndex(InArgs.OptionalPinIndex) && OptionalPins[InArgs.OptionalPinIndex].bShowPin;
					}

					// Pins are exposed if we have a binding or not, so treat as unchecked only if we have
					// no binding
					return bPinShown && !bHasBinding ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				};
				
				InMenuBuilder.AddMenuEntry(
					LOCTEXT("ExposeAsPin", "Expose As Pin"),
					LOCTEXT("ExposeAsPinTooltip", "Show/hide this property as a pin on the node"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.PinIcon"),
					FUIAction(
						FExecuteAction::CreateLambda(ExposeAsPin),
						FCanExecuteAction(),
						FGetActionCheckState::CreateLambda(GetExposedAsPinCheckState)
					),
					NAME_None,
					EUserInterfaceActionType::Check
				);

				if(InArgs.bPropertyIsOnFNode)
				{
					auto MakeDynamic = [InArgs, Blueprint]()
					{
						// Comparison without name index to deal with arrays
						const FName ComparisonName = FName(InArgs.PinName, 0);

						bool bIsAlwaysDynamic = false;
						for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
						{
							if(AnimGraphNode->AlwaysDynamicProperties.Contains(ComparisonName))
							{
								bIsAlwaysDynamic = true;
								break;
							}
						}

						{
							FScopedTransaction Transaction(LOCTEXT("AlwaysDynamic", "Always Dynamic"));

							for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
							{
								AnimGraphNode->Modify();

								if(bIsAlwaysDynamic)
								{
									AnimGraphNode->AlwaysDynamicProperties.Remove(ComparisonName);
								}
								else
								{
									AnimGraphNode->AlwaysDynamicProperties.Add(ComparisonName);
								}
							}

							FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
						}
					};

					auto GetDynamicCheckState = [InArgs]()
					{
						// Comparison without name index to deal with arrays
						const FName ComparisonName = FName(InArgs.PinName, 0);

						bool bIsAlwaysDynamic = false;
						for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
						{
							for(const FName& AlwaysDynamicPropertyName : AnimGraphNode->AlwaysDynamicProperties)
							{
								if(ComparisonName == FName(AlwaysDynamicPropertyName, 0))
								{
									bIsAlwaysDynamic = true;
									break;
								}
							}
						}
						
						return bIsAlwaysDynamic ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					};
					
					InMenuBuilder.AddMenuEntry(
						LOCTEXT("DynamicValue", "Dynamic Value"),
						LOCTEXT("DynamicValueTooltip", "Flag this value as dynamic. This way it can be set from functions even when not exposed as a pin."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda(MakeDynamic),
							FCanExecuteAction(),
							FGetActionCheckState::CreateLambda(GetDynamicCheckState)
						),
						NAME_None,
						EUserInterfaceActionType::Check
					);

					// Only Update When Active flag.
					{
						auto ToggleOnlyUpdateWhenActive = [InArgs, Blueprint]()
						{
							// by default, the new desired value is true...
							bool bNewOnlyUpdateWhenActive = true;
							for(const UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
							{
								if (const UAnimGraphNodeBinding_Base* GraphNodeBinding = Cast<const UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
								{
									if (const FAnimGraphNodePropertyBinding* PropertyBinding = GraphNodeBinding->PropertyBindings.Find(InArgs.BindingName))
									{
										//... unless any currently selected nodes already have it enabled. Set all to false to respect multiple values behavior in blueprints.
										if (PropertyBinding->bOnlyUpdateWhenActive == true)
										{
											bNewOnlyUpdateWhenActive = false;
											break;
										}
									}
								}
							}

							FScopedTransaction Transaction(LOCTEXT("ToggleOnlyUpdateWhenActive", "Toggle Only Update When Active"));
							for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
							{
								AnimGraphNode->Modify();
								if (UAnimGraphNodeBinding_Base* GraphNodeBinding = Cast<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetMutableBinding()))
								{
									if (FAnimGraphNodePropertyBinding* PropertyBinding = GraphNodeBinding->PropertyBindings.Find(InArgs.BindingName))
									{
										PropertyBinding->bOnlyUpdateWhenActive = bNewOnlyUpdateWhenActive;
									}
								}
							}
							FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
						};

						auto GetOnlyUpdateWhenActiveState = [InArgs]()
						{
							ECheckBoxState OutCheckboxState = ECheckBoxState::Unchecked;
							for(const UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
							{
								if (const UAnimGraphNodeBinding_Base* GraphNodeBinding = Cast<const UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
								{
									if (const FAnimGraphNodePropertyBinding* PropertyBinding = GraphNodeBinding->PropertyBindings.Find(InArgs.BindingName))
									{
										if (PropertyBinding->bOnlyUpdateWhenActive == true)
										{
											// If any selected node has the option enabled, add a check mark.
											// @todo: Ideally for multiple values, we'd use a different icon, but that's not what the other settings do.
											OutCheckboxState = ECheckBoxState::Checked;
											break;
										}
									}
								}
							}

							return OutCheckboxState;
						};

						auto CanToggleUpdateWhenActiveState = [InArgs]()
						{
							for(const UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
							{
								if (const UAnimGraphNodeBinding_Base* GraphNodeBinding = Cast<const UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding()))
								{
									// If any selected node doesn't have a binding object, disable this option.
									// We only support this feature through property access bindings on the node.
									if (GraphNodeBinding->PropertyBindings.Find(InArgs.BindingName) == nullptr)
									{
										return false;
									}
								}
							}

							return true;
						};

						InMenuBuilder.AddMenuEntry(
							LOCTEXT("OnlyUpdateWhenActive", "Only Update When Active"),
							LOCTEXT("OnlyUpdateWhenActiveTooltip", "Only update this property when the node is in an active graph branch (not blending out). To enable this, the property must be bound through property access."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda(ToggleOnlyUpdateWhenActive),
								FCanExecuteAction::CreateLambda(CanToggleUpdateWhenActiveState),
								FGetActionCheckState::CreateLambda(GetOnlyUpdateWhenActiveState)
							),
							NAME_None,
							EUserInterfaceActionType::Check
						);
					}
				}
			}
			InMenuBuilder.EndSection();
		};

		FPropertyBindingWidgetArgs Args;
		Args.Property = PropertyToBindTo;
		Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda(OnCanBindProperty);
		Args.OnCanBindFunction = FOnCanBindFunction::CreateLambda(OnCanBindFunction);
		Args.OnCanBindToClass = FOnCanBindToClass::CreateLambda([](UClass* InClass){ return true; });
		Args.OnAddBinding = FOnAddBinding::CreateLambda(OnAddBinding);
		Args.OnRemoveBinding = FOnRemoveBinding::CreateLambda(OnRemoveBinding);
		Args.OnCanRemoveBinding = FOnCanRemoveBinding::CreateLambda(CanRemoveBinding);
		Args.CurrentBindingText = MakeAttributeLambda(CurrentBindingText);
		Args.CurrentBindingToolTipText = MakeAttributeLambda(CurrentBindingToolTipText);
		Args.CurrentBindingImage = MakeAttributeLambda(CurrentBindingImage);
		Args.CurrentBindingColor = MakeAttributeLambda(CurrentBindingColor);
		Args.MenuExtender = MakeShared<FExtender>();
		Args.MenuExtender->AddMenuExtension("BindingActions", EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateLambda(AddMenuExtension));
		if(InArgs.MenuExtender.IsValid())
		{
			Args.MenuExtender = FExtender::Combine( { Args.MenuExtender, InArgs.MenuExtender } );
		}

		IPropertyAccessBlueprintBinding::FContext BindingContext;
		BindingContext.Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(FirstAnimGraphNode);
		BindingContext.Graph = FirstAnimGraphNode->GetGraph();
		BindingContext.Node = FirstAnimGraphNode;
		BindingContext.Pin = FirstAnimGraphNode->FindPin(InArgs.PinName);

		auto OnSetPropertyAccessContextId = [InArgs, Blueprint](const FName& InContextId)
		{
			for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
			{
				UAnimGraphNodeBinding_Base* Binding = CastChecked<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetMutableBinding());
				if(FAnimGraphNodePropertyBinding* PropertyBinding = Binding->PropertyBindings.Find(InArgs.BindingName))
				{
					Binding->Modify();
					PropertyBinding->ContextId = InContextId;
				}
			}

			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		};

		auto OnCanSetPropertyAccessContextId = [InArgs, FirstAnimGraphNode](const FName& InContextId)
		{
			const UAnimGraphNodeBinding_Base* Binding = CastChecked<UAnimGraphNodeBinding_Base>(FirstAnimGraphNode->GetBinding());
			return Binding->PropertyBindings.Find(InArgs.BindingName) != nullptr;
		};
		
		auto OnGetPropertyAccessContextId = [InArgs]() -> FName
		{
			FName CurrentContext = NAME_None;
			for(UAnimGraphNode_Base* AnimGraphNode : InArgs.Nodes)
			{
				const UAnimGraphNodeBinding_Base* Binding = CastChecked<UAnimGraphNodeBinding_Base>(AnimGraphNode->GetBinding());
				if(const FAnimGraphNodePropertyBinding* PropertyBinding = Binding->PropertyBindings.Find(InArgs.BindingName))
				{
					if(CurrentContext != NAME_None && CurrentContext != PropertyBinding->ContextId)
					{
						return NAME_None;
					}
					else
					{
						CurrentContext = PropertyBinding->ContextId;
					}
				}
			}
		
			return CurrentContext;
		};
		
		IPropertyAccessBlueprintBinding::FBindingMenuArgs MenuArgs;
		MenuArgs.OnSetPropertyAccessContextId = FOnSetPropertyAccessContextId::CreateLambda(OnSetPropertyAccessContextId);
		MenuArgs.OnCanSetPropertyAccessContextId = FOnCanSetPropertyAccessContextId::CreateLambda(OnCanSetPropertyAccessContextId);
		MenuArgs.OnGetPropertyAccessContextId = FOnGetPropertyAccessContextId::CreateLambda(OnGetPropertyAccessContextId);
		
		// Add the binding menu extenders
		TArray<TSharedPtr<FExtender>> Extenders( { Args.MenuExtender } );
		for(IPropertyAccessBlueprintBinding* Binding : IModularFeatures::Get().GetModularFeatureImplementations<IPropertyAccessBlueprintBinding>("PropertyAccessBlueprintBinding"))
		{
			TSharedPtr<FExtender> BindingExtender = Binding->MakeBindingMenuExtender(BindingContext, MenuArgs);
			if(BindingExtender)
			{
				Extenders.Add(BindingExtender);
			}
		}
		
		if(Extenders.Num() > 0)
		{
			Args.MenuExtender = FExtender::Combine(Extenders);
		}

		Args.bAllowNewBindings = false;
		Args.bAllowArrayElementBindings = !bIsArray;
		Args.bAllowUObjectFunctions = !bIsArray;
		Args.bAllowStructFunctions = !bIsArray;

		IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

		const FTextBlockStyle& TextBlockStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PropertyAccess.CompiledContext.Text");
		
		return
			SNew(SBox)
			.MaxDesiredWidth(200.0f)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				[
					SNew(SBorder)
					.Padding(FMargin(1.0f, 3.0f, 1.0f, 1.0f))
					.Visibility(InArgs.bOnGraphNode ? EVisibility::Visible : EVisibility::Collapsed)
					.BorderImage(FAppStyle::GetBrush("PropertyAccess.CompiledContext.Border"))
					.RenderTransform_Lambda([InArgs, FirstAnimGraphNode, &TextBlockStyle]()
					{
						const UAnimGraphNodeBinding_Base* Binding = CastChecked<UAnimGraphNodeBinding_Base>(FirstAnimGraphNode->GetBinding());
						const FAnimGraphNodePropertyBinding* PropertyBindingPtr = Binding->PropertyBindings.Find(InArgs.BindingName);
						FVector2D TextSize(0.0f, 0.0f);
						if(PropertyBindingPtr)
						{
							const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
							TextSize = FontMeasureService->Measure(PropertyBindingPtr->CompiledContext, TextBlockStyle.Font);
						}
						return FSlateRenderTransform(FVector2D(0.0f, TextSize.Y - 1.0f));
					})	
					[
						SNew(STextBlock)
						.TextStyle(&TextBlockStyle)
						.Visibility_Lambda([InArgs, FirstAnimGraphNode, bMultiSelect]()
						{
							const UAnimGraphNodeBinding_Base* Binding = CastChecked<UAnimGraphNodeBinding_Base>(FirstAnimGraphNode->GetBinding());
							const FAnimGraphNodePropertyBinding* PropertyBindingPtr = Binding->PropertyBindings.Find(InArgs.BindingName);
							return bMultiSelect || PropertyBindingPtr == nullptr || PropertyBindingPtr->CompiledContext.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
						})
						.Text_Lambda([InArgs, FirstAnimGraphNode]()
						{
							const UAnimGraphNodeBinding_Base* Binding = CastChecked<UAnimGraphNodeBinding_Base>(FirstAnimGraphNode->GetBinding());
							const FAnimGraphNodePropertyBinding* PropertyBindingPtr = Binding->PropertyBindings.Find(InArgs.BindingName);
							return PropertyBindingPtr != nullptr ? PropertyBindingPtr->CompiledContext : FText::GetEmpty();
						})
					]
				]
				+SOverlay::Slot()
				[
					PropertyAccessEditor.MakePropertyBindingWidget(Blueprint, Args)
				]	
			];
	}
	else
	{
		return SNew(SSpacer);
	}
}

#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
