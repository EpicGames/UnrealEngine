// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNodeBinding.h"
#include "AnimGraphNodeBinding_Base.generated.h"

UCLASS(EditInlineNew, DisplayName = "Default Anim Graph Node Binding")
class UAnimGraphNodeBinding_Base : public UAnimGraphNodeBinding
{
	GENERATED_BODY()

private:
	friend class UAnimGraphNode_Base;
	friend class UAnimBlueprintExtension_Base;

	// UAnimGraphNodeBinding interface
	virtual UScriptStruct* GetAnimNodeHandlerStruct() const override;
	virtual void OnInternalPinCreation(UAnimGraphNode_Base* InNode) override;
	virtual void OnReconstructNode(UAnimGraphNode_Base* InNode) override;
	virtual bool HasBinding(FName InBindingName, bool bCheckArrayIndexName) const override;
	virtual void RemoveBindings(FName InBindingName) override;
	virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* InPin, FName InBindingName, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) override;
	virtual void HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName) override;
	virtual void ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement) override;
	virtual bool ReferencesVariable(const FName& InVarName, const UStruct* InScope) const override;
	virtual bool ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const override;
	virtual void UpdateBindingNames(TFunctionRef<FString(const FString& InOldName)> InModifierFunction) override;
	virtual void ProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
#if WITH_EDITOR
	virtual TSharedRef<SWidget> MakePropertyBindingWidget(const UAnimGraphNode_Base::FAnimPropertyBindingWidgetArgs& InArgs) override;
#endif

	static void RecalculateBindingType(UAnimGraphNode_Base* InNode, FAnimGraphNodePropertyBinding& InBinding);

	/** Map from property name->binding info */
	UPROPERTY()
	TMap<FName, FAnimGraphNodePropertyBinding> PropertyBindings;
};