// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_AimOffsetLookAt.h"
#include "GraphEditorActions.h"
#include "Kismet2/CompilerResultsLog.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "ToolMenus.h"
#include "AnimGraphCommands.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_RotationOffsetBlendSpace

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_AimOffsetLookAt)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_AimOffsetLookAt"

UAnimGraphNode_AimOffsetLookAt::UAnimGraphNode_AimOffsetLookAt(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_AimOffsetLookAt::GetTooltipText() const
{
	// FText::Format() is slow, so we utilize the cached list title
	return GetNodeTitle(ENodeTitleType::ListView);
}

FText UAnimGraphNode_AimOffsetLookAt::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UBlendSpace* BlendSpaceToCheck = Node.GetBlendSpace();
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_AimOffsetLookAt, BlendSpace));
	if (BlendSpacePin != nullptr && BlendSpaceToCheck == nullptr)
	{
		BlendSpaceToCheck = Cast<UBlendSpace>(BlendSpacePin->DefaultObject);
	}

	if (BlendSpaceToCheck == nullptr)
	{
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			return LOCTEXT("AimOffsetLookAt_NONE_ListTitle", "LookAt AimOffset '(None)'");
		}
		else
		{
			return LOCTEXT("AimOffsetLookAt_NONE_Title", "(None)\nLookAt AimOffset");
		}
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		const FText BlendSpaceName = FText::FromString(BlendSpaceToCheck->GetName());

		FFormatNamedArguments Args;
		Args.Add(TEXT("BlendSpaceName"), BlendSpaceName);

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AimOffsetLookAtListTitle", "LookAt AimOffset '{BlendSpaceName}'"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AimOffsetLookAtFullTitle", "{BlendSpaceName}\nLookAt AimOffset"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

void UAnimGraphNode_AimOffsetLookAt::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeBlendSpace(UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TWeakObjectPtr<UBlendSpace> BlendSpace)
		{
			UAnimGraphNode_AimOffsetLookAt* BlendSpaceNode = CastChecked<UAnimGraphNode_AimOffsetLookAt>(NewNode);
			BlendSpaceNode->Node.SetBlendSpace(BlendSpace.Get());
		}

		static UBlueprintNodeSpawner* MakeBlendSpaceAction(TSubclassOf<UEdGraphNode> const NodeClass, const UBlendSpace* BlendSpace)
		{
			UBlueprintNodeSpawner* NodeSpawner = nullptr;

			bool const bIsAimOffset = BlendSpace->IsA(UAimOffsetBlendSpace::StaticClass()) ||
				BlendSpace->IsA(UAimOffsetBlendSpace1D::StaticClass());
			if (bIsAimOffset)
			{
				NodeSpawner = UBlueprintNodeSpawner::Create(NodeClass);
				check(NodeSpawner != nullptr);

				TWeakObjectPtr<UBlendSpace> BlendSpacePtr = MakeWeakObjectPtr(const_cast<UBlendSpace*>(BlendSpace));
				NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(GetMenuActions_Utils::SetNodeBlendSpace, BlendSpacePtr);
			}
			return NodeSpawner;
		}
	};

	if (const UObject* RegistrarTarget = ActionRegistrar.GetActionKeyFilter())
	{
		if (const UBlendSpace* TargetBlendSpace = Cast<UBlendSpace>(RegistrarTarget))
		{
			if(TargetBlendSpace->IsAsset())
			{
				if (UBlueprintNodeSpawner* NodeSpawner = GetMenuActions_Utils::MakeBlendSpaceAction(GetClass(), TargetBlendSpace))
				{
					ActionRegistrar.AddBlueprintAction(TargetBlendSpace, NodeSpawner);
				}
			}
		}
		// else, the Blueprint database is specifically looking for actions pertaining to something different (not a BlendSpace asset)
	}
	else
	{
		UClass* NodeClass = GetClass();
		for (TObjectIterator<UBlendSpace> BlendSpaceIt; BlendSpaceIt; ++BlendSpaceIt)
		{
			UBlendSpace* BlendSpace = *BlendSpaceIt;
			if(BlendSpace->IsAsset())
			{
				if (UBlueprintNodeSpawner* NodeSpawner = GetMenuActions_Utils::MakeBlendSpaceAction(NodeClass, BlendSpace))
				{
					ActionRegistrar.AddBlueprintAction(BlendSpace, NodeSpawner);
				}
			}
		}
	}
}

FBlueprintNodeSignature UAnimGraphNode_AimOffsetLookAt::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddSubObject(Node.GetBlendSpace());

	return NodeSignature;
}

void UAnimGraphNode_AimOffsetLookAt::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Asset))
	{
		Node.SetBlendSpace(BlendSpace);
	}
}

void UAnimGraphNode_AimOffsetLookAt::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	ValidateAnimNodeDuringCompilationHelper(ForSkeleton, MessageLog, Node.GetBlendSpace(), UBlendSpace::StaticClass(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_AimOffsetLookAt, BlendSpace)), GET_MEMBER_NAME_CHECKED(FAnimNode_AimOffsetLookAt, BlendSpace));

	UBlendSpace* BlendSpaceToCheck = Node.GetBlendSpace();
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_AimOffsetLookAt, BlendSpace));
	if (BlendSpacePin != nullptr && BlendSpaceToCheck == nullptr)
	{
		BlendSpaceToCheck = Cast<UBlendSpace>(BlendSpacePin->DefaultObject);
	}

	if (BlendSpaceToCheck)
	{
		if (Cast<UAimOffsetBlendSpace>(BlendSpaceToCheck) == NULL &&
			Cast<UAimOffsetBlendSpace1D>(BlendSpaceToCheck) == NULL)
		{
			MessageLog.Error(TEXT("@@ references an invalid blend space (one that is not an aim offset)"), this);
		}

		const USkeleton* BlendSpaceSkeleton = BlendSpaceToCheck->GetSkeleton();
		if (BlendSpaceSkeleton) // if blend space doesn't have skeleton, it might be due to blend space not loaded yet, @todo: wait with anim blueprint compilation until all assets are loaded?
		{
			// Temporary fix where skeleton is not fully loaded during AnimBP compilation and thus the socket name check is invalid UE-39499 (NEED FIX) 
			if (!BlendSpaceSkeleton->HasAnyFlags(RF_NeedPostLoad))
			{
				{
					// Make sure that the source socket name is a valid one for the skeleton
					UEdGraphPin* SocketNamePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_AimOffsetLookAt, SourceSocketName));
					FName SocketNameToCheck = (SocketNamePin != nullptr) ? FName(*SocketNamePin->DefaultValue) : Node.SourceSocketName;

					const FReferenceSkeleton& RefSkel = BlendSpaceSkeleton->GetReferenceSkeleton();

					const bool bValidValue = SocketNamePin == nullptr && (
						BlendSpaceSkeleton->FindSocket(Node.SourceSocketName) ||
						RefSkel.FindBoneIndex(Node.SourceSocketName) != INDEX_NONE);
					const bool bValidPinValue = SocketNamePin != nullptr && (
						BlendSpaceSkeleton->FindSocket(FName(*SocketNamePin->DefaultValue)) ||
						RefSkel.FindBoneIndex(FName(*SocketNamePin->DefaultValue)) != INDEX_NONE);
					const bool bValidConnectedPin = SocketNamePin != nullptr && SocketNamePin->LinkedTo.Num();

					if (!bValidValue && !bValidPinValue && !bValidConnectedPin)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SocketName"), FText::FromName(SocketNameToCheck));

						const FText Msg = FText::Format(LOCTEXT("SocketNameNotFound", "@@ - Socket {SocketName} not found in Skeleton"), Args);
						MessageLog.Error(*Msg.ToString(), this);
					}
				}

				// And similarly the pivot socket, though here we allow it to be blank (but not invalid)
				{
					// Make sure that the source socket name is a valid one for the skeleton
					UEdGraphPin* SocketNamePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_AimOffsetLookAt, PivotSocketName));
					FName SocketNameToCheck = (SocketNamePin != nullptr) ? FName(*SocketNamePin->DefaultValue) : Node.PivotSocketName;
					const FReferenceSkeleton& RefSkel = BlendSpaceSkeleton->GetReferenceSkeleton();

					const bool bValidValue = SocketNamePin == nullptr && (
						Node.PivotSocketName.IsNone() ||
						BlendSpaceSkeleton->FindSocket(Node.PivotSocketName) ||
						RefSkel.FindBoneIndex(Node.PivotSocketName) != INDEX_NONE);
					const bool bValidPinValue = SocketNamePin != nullptr && (
						BlendSpaceSkeleton->FindSocket(FName(*SocketNamePin->DefaultValue)) ||
						RefSkel.FindBoneIndex(FName(*SocketNamePin->DefaultValue)) != INDEX_NONE);
					const bool bValidConnectedPin = SocketNamePin != nullptr && SocketNamePin->LinkedTo.Num();

					if (!bValidValue && !bValidPinValue && !bValidConnectedPin)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SocketName"), FText::FromName(SocketNameToCheck));

						const FText Msg = FText::Format(LOCTEXT("PivotSocketNameNotFound", "@@ - PivotSocket {SocketName} not found in Skeleton"), Args);
						MessageLog.Error(*Msg.ToString(), this);
					}
				}
			}
		}
	}

	if (UAnimationSettings::Get()->bEnablePerformanceLog)
	{
		if (Node.LODThreshold < 0)
		{
			MessageLog.Warning(TEXT("@@ contains no LOD Threshold."), this);
		}
	}

	if(FMath::IsNearlyZero(Node.SocketAxis.SizeSquared()))
	{
		MessageLog.Error(TEXT("Socket axis for node @@ is zero."), this);
	}
}

void UAnimGraphNode_AimOffsetLookAt::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to single frame
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeBlendSpacePlayer", NSLOCTEXT("A3Nodes", "BlendSpaceHeading", "Blend Space"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().OpenRelatedAsset);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToAimOffsetSimple);
		}
	}
}

void UAnimGraphNode_AimOffsetLookAt::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const
{
	if (Node.GetBlendSpace())
	{
		HandleAnimReferenceCollection(Node.BlendSpace, AnimationAssets);
	}
}

void UAnimGraphNode_AimOffsetLookAt::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(Node.BlendSpace, AnimAssetReplacementMap);
}

void UAnimGraphNode_AimOffsetLookAt::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	// Hide input pins that are not relevant for this child class.
	UBlendSpace * BlendSpace = GetBlendSpace();
	if (BlendSpace != NULL)
	{
		if ((SourcePropertyName == TEXT("X")) || (SourcePropertyName == FName(*BlendSpace->GetBlendParameter(0).DisplayName)))
		{
			Pin->bHidden = true;
		}
		if ((SourcePropertyName == TEXT("Y")) || (SourcePropertyName == FName(*BlendSpace->GetBlendParameter(1).DisplayName)))
		{
			Pin->bHidden = true;
		}
		if ((SourcePropertyName == TEXT("Z")) || (SourcePropertyName == FName(*BlendSpace->GetBlendParameter(2).DisplayName)))
		{
			Pin->bHidden = true;
		}
	}
}

#undef LOCTEXT_NAMESPACE
