// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_AnimDynamics.h"
#include "AnimNodeEditModes.h"
#include "Components/SkeletalMeshComponent.h"
#include "EngineGlobals.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PrimitiveDrawingUtils.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/ReleaseObjectVersion.h"
#include "Widgets/Input/SButton.h"

// Details includes
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "AnimationCustomVersion.h"
#include "Animation/AnimInstance.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_AnimDynamics)

#define LOCTEXT_NAMESPACE "AnimDynamicsNode"

////////////////////////////////////////
// class UAnimGraphNode_AnimDynamics

FText UAnimGraphNode_AnimDynamics::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Anim Dynamics");
}


FString UAnimGraphNode_AnimDynamics::GetNodeCategory() const
{
	return TEXT("Animation|Dynamics");
}

void UAnimGraphNode_AnimDynamics::GetOnScreenDebugInfo(TArray<FText>& DebugInfo, FAnimNode_Base* RuntimeAnimNode, USkeletalMeshComponent* PreviewSkelMeshComp) const
{
	if(RuntimeAnimNode)
	{
		const FAnimNode_AnimDynamics* PreviewNode = static_cast<FAnimNode_AnimDynamics*>(RuntimeAnimNode);

		for(const FAnimPhysBodyDefinition& PhysicsBodyDef : PreviewNode->PhysicsBodyDefinitions)
		{
			const FName BoneName = PhysicsBodyDef.BoundBone.BoneName;
			const int32 SkelBoneIndex = PreviewSkelMeshComp->GetBoneIndex(BoneName);
			if(SkelBoneIndex != INDEX_NONE)
			{
				FTransform BoneTransform = PreviewSkelMeshComp->GetBoneTransform(SkelBoneIndex);
				DebugInfo.Add(FText::Format(LOCTEXT("DebugOnScreenName", "Anim Dynamics (Bone:{0})"), FText::FromName(BoneName)));
				DebugInfo.Add(FText::Format(LOCTEXT("DebugOnScreenTranslation", "    Translation: {0}"), FText::FromString(BoneTransform.GetTranslation().ToString())));
				DebugInfo.Add(FText::Format(LOCTEXT("DebugOnScreenRotation", "    Rotation: {0}"), FText::FromString(BoneTransform.Rotator().ToString())));
			}
		}
	}
}

FText UAnimGraphNode_AnimDynamics::GetControllerDescription() const
{
	return LOCTEXT("Description", "Anim Dynamics");
}

void UAnimGraphNode_AnimDynamics::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> PreviewFlagHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_AnimDynamics, bPreviewLive));

	IDetailCategoryBuilder& PreviewCategory = DetailBuilder.EditCategory(TEXT("Preview"));
	PreviewCategory.AddProperty(PreviewFlagHandle);

	FDetailWidgetRow& WidgetRow = PreviewCategory.AddCustomRow(LOCTEXT("ResetButtonRow", "Reset"));

	WidgetRow
		[
			SNew(SButton)
			.Text(LOCTEXT("ResetButtonText", "Reset Simulation"))
			.ToolTipText(LOCTEXT("ResetButtonToolTip", "Resets the simulation for this node"))
			.OnClicked(FOnClicked::CreateStatic(&UAnimGraphNode_AnimDynamics::ResetButtonClicked, &DetailBuilder))
		];

	// Customise the physics body definition array rendered in the details panel.
	IDetailCategoryBuilder& PhysicsParametersCategory = DetailBuilder.EditCategory(TEXT("PhysicsParameters"));
	TSharedRef< IPropertyHandle > PhysicsBodyDefinitionsProperty = DetailBuilder.GetProperty("Node.PhysicsBodyDefinitions", GetClass());
	if (PhysicsBodyDefinitionsProperty->AsArray().IsValid())
	{
		TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShared<FDetailArrayBuilder>(PhysicsBodyDefinitionsProperty, /*InGenerateHeader*/ true, /*InDisplayResetToDefault*/ true, /*InDisplayElementNum*/ true);
		PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateUObject(this, &UAnimGraphNode_AnimDynamics::OnPhysicsBodyDefCustomizeDetails, &DetailBuilder));
		PhysicsParametersCategory.AddCustomBuilder(PropertyBuilder, false);
	}	

	// Force order of details panel catagories - Must set order for all of them as any that are edited automatically move to the top.
	{
		uint32 SortOrder = 0;
		DetailBuilder.EditCategory("Preview").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Setup").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("PhysicsParameters").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Settings").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("SphericalLimit").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("PlanarLimit").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Forces").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Wind").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Retargetting").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Performance").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Functions").SetSortOrder(SortOrder++);
		DetailBuilder.EditCategory("Alpha").SetSortOrder(SortOrder++);
	}
}

void UAnimGraphNode_AnimDynamics::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

void UAnimGraphNode_AnimDynamics::UpdatePreviewDynamicsNode(const int32 PhysicsBodyIndex)
{
	// Update the preview node from details panel and reset physics simulation.

	if(FAnimNode_AnimDynamics* const ActivePreviewNode = GetPreviewDynamicsNode())
	{
		if(ActivePreviewNode->PhysicsBodyDefinitions.IsValidIndex(PhysicsBodyIndex) && Node.PhysicsBodyDefinitions.IsValidIndex(PhysicsBodyIndex))
		{
			ActivePreviewNode->PhysicsBodyDefinitions[PhysicsBodyIndex] = Node.PhysicsBodyDefinitions[PhysicsBodyIndex];
			ResetSim();
		}
	}
}

FEditorModeID UAnimGraphNode_AnimDynamics::GetEditorMode() const
{
	return AnimNodeEditModes::AnimDynamics;
}

FText UAnimGraphNode_AnimDynamics::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("ControllerDescription"), GetControllerDescription());
	Arguments.Add(TEXT("BoundBoneName"), FText::FromName(Node.BoundBone.BoneName));
	if(Node.bChain)
	{
		Arguments.Add(TEXT("ChainEndBoneName"), FText::FromName(Node.ChainEnd.BoneName));
	}

	if(TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
	{
		if(Node.BoundBone.BoneName == NAME_None || (Node.bChain && Node.ChainEnd.BoneName == NAME_None))
		{
			return GetControllerDescription();
		}

		if(Node.bChain)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimDynamicsNodeTitleSmallChain", "{ControllerDescription} - Chain: {BoundBoneName} -> {ChainEndBoneName}"), Arguments), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimDynamicsNodeTitleSmall", "{ControllerDescription} - Bone: {BoundBoneName}"), Arguments), this);
		}
	}
	else
	{
		if(Node.bChain)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimDynamicsNodeTitleLargeChain", "{ControllerDescription}\nChain: {BoundBoneName} -> {ChainEndBoneName}"), Arguments), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimDynamicsNodeTitleLarge", "{ControllerDescription}\nBone: {BoundBoneName}"), Arguments), this);
		}
	}

	return CachedNodeTitles[TitleType];
}


USkeleton* UAnimGraphNode_AnimDynamics::GetSkeleton() const
{
	USkeleton* Skeleton = nullptr;

	if (UAnimBlueprint* const AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(this)))
	{
		Skeleton = AnimBlueprint->TargetSkeleton;
	}

	return Skeleton;
}

void UAnimGraphNode_AnimDynamics::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bShouldRebuildChain = false;

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNode_AnimDynamics, ChainEnd))
	{
		// bChain has been modified.
		if (Node.bChain)
		{
			Node.ChainEnd = Node.PhysicsBodyDefinitions[0].BoundBone;
		}
		else
		{
			Node.ChainEnd.BoneName = NAME_None;
		}

		bShouldRebuildChain = true;
	}

	bShouldRebuildChain |= (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FBoneReference, BoneName)); // Either BoundBone or ChainEnd have been modified. 
	
	if (USkeleton* const Skeleton = GetSkeleton())
	{
		if (bShouldRebuildChain)
		{
			// Rebuild Chain if begin or end bones have changed.
			Node.UpdateChainPhysicsBodyDefinitions(Skeleton->GetReferenceSkeleton());
		}
		else
		{
			// Write chain bones names to each body in chain and check chain length in case either have been changed by a copy and paste operation.
			Node.ValidateChainPhysicsBodyDefinitions(Skeleton->GetReferenceSkeleton());
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_AnimDynamics::PostLoad()
{
	Super::PostLoad();
}

void UAnimGraphNode_AnimDynamics::ResetSim()
{
	FAnimNode_AnimDynamics* PreviewNode = GetPreviewDynamicsNode();
	if(PreviewNode)
	{
		PreviewNode->RequestInitialise(ETeleportType::ResetPhysics);
	}
}

FAnimNode_AnimDynamics* UAnimGraphNode_AnimDynamics::GetPreviewDynamicsNode() const
{
	return GetDebuggedAnimNode<FAnimNode_AnimDynamics>();
}

FReply UAnimGraphNode_AnimDynamics::ResetButtonClicked(IDetailLayoutBuilder* DetailLayoutBuilder)
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList = DetailLayoutBuilder->GetSelectedObjects();
	for(TWeakObjectPtr<UObject> Object : SelectedObjectsList)
	{
		if(UAnimGraphNode_AnimDynamics* AnimDynamicsNode = Cast<UAnimGraphNode_AnimDynamics>(Object.Get()))
		{
			AnimDynamicsNode->ResetSim();
		}
	}
	
	return FReply::Handled();
}

void UAnimGraphNode_AnimDynamics::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAnimationCustomVersion::GUID);

	const int32 CustomAnimVersion = Ar.CustomVer(FAnimationCustomVersion::GUID);
	
	if(CustomAnimVersion < FAnimationCustomVersion::AnimDynamicsAddAngularOffsets)
	{
		FAnimPhysConstraintSetup& ConSetup = Node.ConstraintSetup_DEPRECATED;
		ConSetup.AngularLimitsMin = FVector(-ConSetup.AngularXAngle_DEPRECATED, -ConSetup.AngularYAngle_DEPRECATED, -ConSetup.AngularZAngle_DEPRECATED);
		ConSetup.AngularLimitsMax = FVector(ConSetup.AngularXAngle_DEPRECATED, ConSetup.AngularYAngle_DEPRECATED, ConSetup.AngularZAngle_DEPRECATED);
	}

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
		
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::GravityOverrideDefinedInWorldSpace)
	{
		Node.bGravityOverrideInSimSpace = true;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimDynamicsEditableChainParameters)
	{
		// Initialize first physics body using deprecated parameters then re-initialize chain.
		Node.PhysicsBodyDefinitions.Reset();		
		FAnimPhysBodyDefinition PhysBodyDef;
		PhysBodyDef.BoundBone = Node.BoundBone;
		PhysBodyDef.BoxExtents = Node.BoxExtents_DEPRECATED;
		PhysBodyDef.LocalJointOffset = -Node.LocalJointOffset_DEPRECATED; // Note: definition of joint offset has changed from 'Joint position relative to physics body' to 'physics body position relative to Joint'.
		PhysBodyDef.ConstraintSetup = Node.ConstraintSetup_DEPRECATED;
		PhysBodyDef.CollisionType = Node.CollisionType_DEPRECATED;
		PhysBodyDef.SphereCollisionRadius = Node.SphereCollisionRadius_DEPRECATED;
		Node.PhysicsBodyDefinitions.Add(PhysBodyDef);
	}
}

void UAnimGraphNode_AnimDynamics::OnPhysicsBodyDefCustomizeDetails(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
{
	TSharedPtr<IPropertyHandle> BoundBoneProperty = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimPhysBodyDefinition, BoundBone));

	// Get Bone Name
	FName BoneName;

	if (BoundBoneProperty)
	{
		TSharedPtr<IPropertyHandle> BoundBoneNameProperty = BoundBoneProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBoneReference, BoneName));
		if (BoundBoneNameProperty)
		{
			BoundBoneNameProperty->GetValue(BoneName);
		}

		DetailLayout->HideProperty(BoundBoneProperty);
		IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(ElementProperty);

		// Set a custom widget to show a more useful item name and remove the 'n items' text that would otherwise appear on the body def group header		
		Row.CustomWidget(true)
			.NameContent()
			[
				SNew(STextBlock)
				.Text_Lambda([this, ElementIndex]() -> FText { return this->BodyDefinitionUINameString(ElementIndex); })
			];
	}

	ElementProperty->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([this, ElementIndex]() { this->UpdatePreviewDynamicsNode(ElementIndex); }));
}

FText UAnimGraphNode_AnimDynamics::BodyDefinitionUINameString(const uint32 BodyIndex) const
{
	if (Node.PhysicsBodyDefinitions.IsValidIndex(BodyIndex))
	{
		const FName BoneName = Node.PhysicsBodyDefinitions[BodyIndex].BoundBone.BoneName;
		return FText::Format(INVTEXT("[{0}] {1}"), FText::AsNumber(BodyIndex), FText::FromName(BoneName));
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
