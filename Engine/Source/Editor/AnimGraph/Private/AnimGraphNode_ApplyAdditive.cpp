// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ApplyAdditive.h"

#include "Animation/AnimationSettings.h"
#include "Kismet2/CompilerResultsLog.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "DetailLayoutBuilder.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ApplyAdditive

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ApplyAdditive)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_ApplyAdditive::UAnimGraphNode_ApplyAdditive(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_ApplyAdditive::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.75f);
}

void UAnimGraphNode_ApplyAdditive::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.AlphaBoolBlend.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FText UAnimGraphNode_ApplyAdditive::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ApplyAdditive_Tooltip", "Apply additive animation to normal pose");
}

FText UAnimGraphNode_ApplyAdditive::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_ApplyAdditive_Title", "Apply Additive");
}

void UAnimGraphNode_ApplyAdditive::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, Alpha))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Float);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBias.GetFriendlyName(Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName));
		}
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, bAlphaBoolEnabled))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Bool);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, AlphaCurveName))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Curve);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_ApplyAdditive::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, AlphaScaleBias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
	{
		ReconstructNode();
	}

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, AlphaInputType))
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeAlphaInputType", "Change Alpha Input Type"));
		Modify();

		// Break links to pins going away
		for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Pins[PinIndex];
			if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, Alpha))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Float)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, bAlphaBoolEnabled))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyAdditive, AlphaCurveName))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
		}

		ReconstructNode();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_ApplyAdditive::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyAdditive, bAlphaBoolEnabled)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyAdditive, AlphaBoolBlend)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Float)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyAdditive, Alpha)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyAdditive, AlphaScaleBias)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyAdditive, AlphaCurveName)));
	}

	if ((Node.AlphaInputType != EAnimAlphaInputType::Float)
		&& (Node.AlphaInputType != EAnimAlphaInputType::Curve))
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyAdditive, AlphaScaleBiasClamp)));
	}
}

FString UAnimGraphNode_ApplyAdditive::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
	//@TODO: TEXT("Apply additive to normal pose"), TEXT("Apply additive pose"));
}

void UAnimGraphNode_ApplyAdditive::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if (UAnimationSettings::Get()->bEnablePerformanceLog)
	{
		if (Node.LODThreshold < 0)
		{
			MessageLog.Warning(TEXT("@@ contains no LOD Threshold."), this);
		}
	}
}
#undef LOCTEXT_NAMESPACE
