// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ApplyLimits.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PrimitiveDrawingUtils.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ApplyLimits

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ApplyLimits)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_ApplyLimits::UAnimGraphNode_ApplyLimits(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_ApplyLimits::GetControllerDescription() const
{
	return LOCTEXT("ApplyLimits", "Apply Limits");
}

FText UAnimGraphNode_ApplyLimits::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ApplyLimits_Tooltip", "Apply Limits.");
}

FText UAnimGraphNode_ApplyLimits::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

void UAnimGraphNode_ApplyLimits::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const
{
	DrawAngularLimits(PDI, PreviewSkelMeshComp, Node);
}

void UAnimGraphNode_ApplyLimits::DrawAngularLimits(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp, const FAnimNode_ApplyLimits& NodeToVisualize) const
{
	for(const FAngularRangeLimit& AngularRangeLimit : NodeToVisualize.AngularRangeLimits)
	{
		const int32 BoneIndex = SkelMeshComp->GetBoneIndex(AngularRangeLimit.Bone.BoneName);
		if(BoneIndex != INDEX_NONE)
		{
			FTransform JointTransform = SkelMeshComp->GetBoneTransform(BoneIndex);

			FVector XAxis = JointTransform.GetUnitAxis(EAxis::X);
			FVector YAxis = JointTransform.GetUnitAxis(EAxis::Y);
			FVector ZAxis = JointTransform.GetUnitAxis(EAxis::Z);

			const FVector& MinAngles = AngularRangeLimit.LimitMin;
			const FVector& MaxAngles = AngularRangeLimit.LimitMax;
			FVector AngleRange = MaxAngles - MinAngles;
			FVector Middle = MinAngles + AngleRange * 0.5f;

			FTransform XAxisConeTM(YAxis, XAxis ^ YAxis, XAxis, JointTransform.GetTranslation());
			XAxisConeTM.SetRotation(FQuat(XAxis, FMath::DegreesToRadians(-Middle.X)) * XAxisConeTM.GetRotation());
			DrawCone(PDI, FScaleMatrix(30.0f) * XAxisConeTM.ToMatrixWithScale(), static_cast<float>(FMath::DegreesToRadians(AngleRange.X / 2.0)), 0.0f, 24, false, FLinearColor::Red, GEngine->ConstraintLimitMaterialX->GetRenderProxy(), SDPG_World);

			FTransform YAxisConeTM(ZAxis, YAxis ^ ZAxis, YAxis, JointTransform.GetTranslation());
			YAxisConeTM.SetRotation(FQuat(YAxis, FMath::DegreesToRadians(Middle.Y)) * YAxisConeTM.GetRotation());
			DrawCone(PDI, FScaleMatrix(30.0f) * YAxisConeTM.ToMatrixWithScale(), static_cast<float>(FMath::DegreesToRadians(AngleRange.Y / 2.0)), 0.0f, 24, false, FLinearColor::Green, GEngine->ConstraintLimitMaterialY->GetRenderProxy(), SDPG_World);
	
			FTransform ZAxisConeTM(XAxis, ZAxis ^ XAxis, ZAxis, JointTransform.GetTranslation());
			ZAxisConeTM.SetRotation(FQuat(ZAxis, FMath::DegreesToRadians(Middle.Z)) * ZAxisConeTM.GetRotation());
			DrawCone(PDI, FScaleMatrix(30.0f) * ZAxisConeTM.ToMatrixWithScale(), static_cast<float>(FMath::DegreesToRadians(AngleRange.Z / 2.0)), 0.0f, 24, false, FLinearColor::Blue, GEngine->ConstraintLimitMaterialZ->GetRenderProxy(), SDPG_World);
		}
	}
}

void UAnimGraphNode_ApplyLimits::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyLimits, AngularRangeLimits))
	{
		Node.RecalcLimits();

		ReconstructNode();
	}
}


#undef LOCTEXT_NAMESPACE
