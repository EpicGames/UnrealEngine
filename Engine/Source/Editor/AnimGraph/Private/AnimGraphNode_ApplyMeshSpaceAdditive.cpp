// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ApplyMeshSpaceAdditive.h"

#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"

#include "Animation/AnimationSettings.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ApplyAdditive

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ApplyMeshSpaceAdditive)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_ApplyMeshSpaceAdditive::UAnimGraphNode_ApplyMeshSpaceAdditive(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_ApplyMeshSpaceAdditive::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.75f);
}

void UAnimGraphNode_ApplyMeshSpaceAdditive::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.AlphaBoolBlend.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FText UAnimGraphNode_ApplyMeshSpaceAdditive::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ApplyMeshSpaceAdditive_Tooltip", "Apply mesh space additive animation to normal pose");
}

FText UAnimGraphNode_ApplyMeshSpaceAdditive::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_ApplyMeshSpaceAdditive_Title", "Apply Mesh Space Additive");
}

void UAnimGraphNode_ApplyMeshSpaceAdditive::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, Alpha))
	{
		Pin->bHidden = Node.AlphaInputType != EAnimAlphaInputType::Float;

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBiasClamp.GetFriendlyName(Node.AlphaScaleBias.GetFriendlyName(Pin->PinFriendlyName));
		}
	}
	else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, bAlphaBoolEnabled))
	{
		Pin->bHidden = Node.AlphaInputType != EAnimAlphaInputType::Bool;
	}
	else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaCurveName))
	{
		Pin->bHidden = Node.AlphaInputType != EAnimAlphaInputType::Curve;

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_ApplyMeshSpaceAdditive::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, bAlphaBoolEnabled)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaBoolBlend)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Float)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, Alpha)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaScaleBias)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaCurveName)));
	}

	if (Node.AlphaInputType == EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaScaleBiasClamp)));
	}
}

void UAnimGraphNode_ApplyMeshSpaceAdditive::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaScaleBias) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing) || 
		PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing))
	{
		ReconstructNode();
	}
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaInputType))
	{
		UEdGraphPin* PinToBreakLinksTo = nullptr;

		// Break links to pins that are getting hidden.
		for (UEdGraphPin* Pin: Pins)
		{
			if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, Alpha) && 
				Node.AlphaInputType != EAnimAlphaInputType::Float)
			{
				PinToBreakLinksTo = Pin;
				break;
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, bAlphaBoolEnabled) && 
					 Node.AlphaInputType != EAnimAlphaInputType::Bool)
			{
				PinToBreakLinksTo = Pin;
				break;
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ApplyMeshSpaceAdditive, AlphaCurveName) &&
					 Node.AlphaInputType != EAnimAlphaInputType::Curve)
			{
				PinToBreakLinksTo = Pin;
				break;
			}
		}

		if (PinToBreakLinksTo != nullptr)
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeAlphaInputType", "Change Alpha Input Type"));
			Modify();
			PinToBreakLinksTo->BreakAllPinLinks();
			RemoveBindings(PinToBreakLinksTo->PinName);
			ReconstructNode();

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		}

	}



	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FString UAnimGraphNode_ApplyMeshSpaceAdditive::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

void UAnimGraphNode_ApplyMeshSpaceAdditive::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if (UAnimationSettings::Get()->bEnablePerformanceLog)
	{
		if (Node.LODThreshold < 0)
		{
			MessageLog.Warning(TEXT("@@ contains no LOD Threshold."), this);
		}
	}
}
#undef LOCTEXT_NAMESPACE
