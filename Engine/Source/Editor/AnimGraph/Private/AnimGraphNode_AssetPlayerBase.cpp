// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_AssetPlayerBase.h"
#include "EdGraphSchema_K2.h"
#include "Animation/AnimComposite.h"
#include "BlueprintActionFilter.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintAssetNodeSpawner.h"
#include "UObject/UObjectIterator.h"
#include "Animation/AnimLayerInterface.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"
#include "IAnimBlueprintCompilationContext.h"
#include "Animation/AnimSync.h"
#include "Animation/AnimAttributes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeTemplateCache.h"
#include "Animation/AnimNode_AssetPlayerBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_AssetPlayerBase)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_AssetPlayerBase"

void UAnimGraphNode_AssetPlayerBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if(Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimSyncGroupsExplicitSyncMethod)
	{
		if(SyncGroup_DEPRECATED.GroupName != NAME_None)
		{
			SyncGroup_DEPRECATED.Method = EAnimSyncMethod::SyncGroup;
		}
	}

	if(Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimNodeConstantDataRefactorPhase0)
	{
		FStructProperty* NodeProperty = GetFNodeProperty();
		if(NodeProperty->Struct->IsChildOf(FAnimNode_AssetPlayerBase::StaticStruct()))
		{
			FAnimNode_AssetPlayerBase* Node = NodeProperty->ContainerPtrToValuePtr<FAnimNode_AssetPlayerBase>(this);
			Node->SetGroupName(SyncGroup_DEPRECATED.GroupName);
			Node->SetGroupRole(SyncGroup_DEPRECATED.GroupRole);
			Node->SetGroupMethod(SyncGroup_DEPRECATED.Method);
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FStructProperty* NodeProperty = GetFNodeProperty();
	if(NodeProperty->Struct->IsChildOf(FAnimNode_AssetPlayerBase::StaticStruct()))
	{
		if(PropertyChangedEvent.GetPropertyName() == TEXT("Method"))
		{
			FAnimNode_AssetPlayerBase* Node = NodeProperty->ContainerPtrToValuePtr<FAnimNode_AssetPlayerBase>(this);
			if(Node->GetGroupMethod() != EAnimSyncMethod::SyncGroup)
			{
				Node->SetGroupName(NAME_None);
				Node->SetGroupRole(EAnimGroupRole::CanBeLeader);
			}
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		// recache visualization now an asset pin's connection is changed
		if (const UEdGraphSchema* Schema = GetSchema())
		{
			Schema->ForceVisualizationCacheClear();
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		// recache visualization now an asset pin's default value has changed
		if (const UEdGraphSchema* Schema = GetSchema())
		{
			Schema->ForceVisualizationCacheClear();
		}
	}
}

FText UAnimGraphNode_AssetPlayerBase::GetTooltipText() const
{
	bool const bIsTemplateNode = GetGraph() == nullptr || FBlueprintNodeTemplateCache::IsTemplateOuter(GetGraph());
	if(bIsTemplateNode)
	{
		return FText::GetEmpty();
	}
	else
	{
		// FText::Format() is slow, so we utilize the cached list title
		return GetNodeTitle(ENodeTitleType::ListView);
	}
}

void UAnimGraphNode_AssetPlayerBase::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UBlueprint* Blueprint = GetBlueprint();

	// Process Asset Player nodes to, if necessary cache off their node index for retrieval at runtime (used for evaluating Automatic Rule Transitions when using Layer nodes)
	auto ProcessGraph = [this, &OutCompiledData](UEdGraph* Graph)
	{
		FString GraphName = Graph->GetName();
		// Also make sure we do not process any empty stub graphs
		if (!GraphName.Contains(ANIM_FUNC_DECORATOR))
		{
			if (Graph->Nodes.ContainsByPredicate([this, &OutCompiledData](UEdGraphNode* Node) { return Node && Node->NodeGuid == NodeGuid; }))
			{
				if (int32* IndexPtr = OutCompiledData.GetAnimBlueprintDebugData().NodeGuidToIndexMap.Find(NodeGuid))
				{
					FGraphAssetPlayerInformation& Info = OutCompiledData.GetGraphAssetPlayerInformation().FindOrAdd(FName(*GraphName));
					Info.PlayerNodeIndices.AddUnique(*IndexPtr);
				}
			}
		}
	};

	// Check for any definition of a layer graph
	for (UEdGraph* Graph : Blueprint->FunctionGraphs)
	{
		ProcessGraph(Graph);
	}

	// Check for any implemented AnimLayer interface graphs
	for (FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
	{
		// Only process Anim Layer interfaces
		if (InterfaceDesc.Interface->IsChildOf<UAnimLayerInterface>())
		{
			for (UEdGraph* Graph : InterfaceDesc.Graphs)
			{
				ProcessGraph(Graph);
			}
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	FStructProperty* NodeProperty = GetFNodeProperty();
	if(NodeProperty->Struct->IsChildOf(FAnimNode_AssetPlayerBase::StaticStruct()))
	{
		FAnimNode_AssetPlayerBase* Node = NodeProperty->ContainerPtrToValuePtr<FAnimNode_AssetPlayerBase>(this);

		const FName GroupName = Node->GetGroupName();
		if(Node->GetGroupMethod() == EAnimSyncMethod::SyncGroup && GroupName == NAME_None)
		{
			MessageLog.Error(*LOCTEXT("NoSyncGroupSupplied", "Node @@ is set to use sync groups, but no sync group has been supplied").ToString(), this);
		}
		else if(Node->GetGroupMethod() != EAnimSyncMethod::SyncGroup && GroupName != NAME_None)
		{
			FText const ErrorFormat = LOCTEXT("InvalidSyncGroupSupplied", "Node @@ is set to not use named sync groups, but a sync group {0} is set.");
			MessageLog.Error( *FText::Format(ErrorFormat, FText::FromName(GroupName)).ToString(), this );
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
	OutAttributes.Add(UE::Anim::FAttributes::Attributes);

	FStructProperty* NodeProperty = GetFNodeProperty();
	if(NodeProperty->Struct->IsChildOf(FAnimNode_AssetPlayerBase::StaticStruct()))
	{
		const FAnimNode_AssetPlayerBase* Node = NodeProperty->ContainerPtrToValuePtr<FAnimNode_AssetPlayerBase>(this);
		if(Node->GetGroupMethod() == EAnimSyncMethod::Graph)
		{
			OutAttributes.Add(UE::Anim::FAnimSync::Attribute);
		}
	}
}

UClass* GetNodeClassForAsset(const UClass* AssetClass)
{
	UClass* NodeClass = nullptr;

	// Iterate over all classes..
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass *Class = *ClassIt;
		// Look for AnimGraphNode classes
		if (Class->IsChildOf(UAnimGraphNode_Base::StaticClass()))
		{
			// See if this node is the 'primary handler' for this asset type
			const UAnimGraphNode_Base* NodeCDO = Class->GetDefaultObject<UAnimGraphNode_Base>();
			if (NodeCDO->SupportsAssetClass(AssetClass) == EAnimAssetHandlerType::PrimaryHandler)
			{
				NodeClass = Class;
				break;
			}
		}
	}

	return NodeClass;
}

bool SupportNodeClassForAsset(const UClass* AssetClass, UClass* NodeClass)
{
	// Get node CDO
	const UAnimGraphNode_Base* NodeCDO = NodeClass->GetDefaultObject<UAnimGraphNode_Base>();
	// See if this node supports this asset type (primary or not)
	return (NodeCDO->SupportsAssetClass(AssetClass) != EAnimAssetHandlerType::NotSupported);
}

void UAnimGraphNode_AssetPlayerBase::SetupNewNode(UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
{
	UAnimGraphNode_AssetPlayerBase* GraphNode = CastChecked<UAnimGraphNode_AssetPlayerBase>(InNewNode);
	
	if(InAssetData.IsValid())
	{
		InAssetData.GetTagValue("Skeleton", GraphNode->UnloadedSkeletonName);
		if(GraphNode->UnloadedSkeletonName == TEXT("None"))
		{
			GraphNode->UnloadedSkeletonName.Empty();
		}
		
		if(!bInIsTemplateNode)
		{
			GraphNode->SetAnimationAsset(CastChecked<UAnimationAsset>(InAssetData.GetAsset()));
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::GetMenuActionsHelper(
	FBlueprintActionDatabaseRegistrar& InActionRegistrar,
	TSubclassOf<UAnimGraphNode_Base> InNodeClass,
	const TArray<TSubclassOf<UObject>>& InAssetTypes,
	const TArray<TSubclassOf<UObject>>& InExcludedAssetTypes,
	const TFunctionRef<FText(const FAssetData&, UClass*)>& InMenuNameFunction,
	const TFunctionRef<FText(const FAssetData&, UClass*)>& InMenuTooltipFunction,
	const TFunction<void(UEdGraphNode*, bool, const FAssetData)>& InSetupNewNodeFromAssetFunction,
	const TFunction<void(UEdGraphNode*, bool, TSubclassOf<UObject>)>& InSetupNewNodeFromClassFunction,
	const TFunction<FText(const FAssetData&)>& InMenuCategoryFunction)
{
	auto MakeActionFromAsset = [&InActionRegistrar, &InMenuNameFunction, &InMenuTooltipFunction, InSetupNewNodeFromAssetFunction, InNodeClass, InMenuCategoryFunction](const FAssetData& InAssetData)
	{
		auto AssetSetup = [InSetupNewNodeFromAssetFunction, InAssetData, InMenuCategoryFunction](UEdGraphNode* InNewNode, bool bInIsTemplateNode)
		{
			InSetupNewNodeFromAssetFunction(InNewNode, bInIsTemplateNode, InAssetData);
		};

		UBlueprintAssetNodeSpawner* NodeSpawner = UBlueprintAssetNodeSpawner::Create(InNodeClass.Get(), InAssetData);
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(AssetSetup);
		NodeSpawner->DefaultMenuSignature.MenuName = InMenuNameFunction(InAssetData, InAssetData.GetClass());
		NodeSpawner->DefaultMenuSignature.Category = InMenuCategoryFunction != nullptr ? InMenuCategoryFunction(InAssetData) : FText::GetEmpty();
		NodeSpawner->DefaultMenuSignature.Tooltip = InMenuTooltipFunction(InAssetData, InAssetData.GetClass());
		InActionRegistrar.AddBlueprintAction(InAssetData, NodeSpawner);
	};

	auto MakeActionFromClass = [&InActionRegistrar, &InMenuNameFunction, &InMenuTooltipFunction, InSetupNewNodeFromClassFunction, InNodeClass](const TSubclassOf<UObject>& InAssetClass)
	{
		if(InSetupNewNodeFromClassFunction != nullptr)
		{
			auto AssetSetup = [InSetupNewNodeFromClassFunction, InAssetClass](UEdGraphNode* InNewNode, bool bInIsTemplateNode)
			{
				InSetupNewNodeFromClassFunction(InNewNode, bInIsTemplateNode, InAssetClass);
			};

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(InNodeClass.Get());
			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(AssetSetup);
			NodeSpawner->DefaultMenuSignature.MenuName = InMenuNameFunction(FAssetData(), InAssetClass.Get());
			NodeSpawner->DefaultMenuSignature.Tooltip = InMenuTooltipFunction(FAssetData(), InAssetClass.Get());
			InActionRegistrar.AddBlueprintAction(InAssetClass, NodeSpawner);
		}
	};
	
	const UObject* QueryObject = InActionRegistrar.GetActionKeyFilter();
	bool bIsObjectOfAssetType = false;
	for(const TSubclassOf<UObject>& AssetType : InAssetTypes)
	{
		if(QueryObject && QueryObject->IsA(AssetType.Get()))
		{
			bIsObjectOfAssetType = true;
			break;
		}
	}
	
	if (QueryObject == nullptr || QueryObject == InNodeClass.Get())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		FARFilter Filter;
		for(const TSubclassOf<UObject>& AssetType : InAssetTypes)
		{
			Filter.ClassPaths.Add(AssetType.Get()->GetClassPathName());
		}
		for(const TSubclassOf<UObject>& ExcludedAssetType : InExcludedAssetTypes)
		{
			Filter.RecursiveClassPathsExclusionSet.Add(ExcludedAssetType.Get()->GetClassPathName());
		}	
		Filter.bRecursiveClasses = true;
		
		TArray<FAssetData> Assets;
		AssetRegistryModule.Get().GetAssets(Filter, Assets);

		for (const FAssetData& AssetData : Assets)
		{
			if(AssetData.IsUAsset())
			{
				MakeActionFromAsset(AssetData);
			}
		}

		if(InSetupNewNodeFromClassFunction != nullptr)
		{
			// Add 'class' nodes
			for(const TSubclassOf<UObject>& AssetType : InAssetTypes)
			{
				MakeActionFromClass(AssetType);
			}
		}
		else
		{
			// Add 'empty' asset node
			MakeActionFromAsset(FAssetData());
		}
	}
	else if (bIsObjectOfAssetType)
	{
		MakeActionFromAsset(FAssetData(QueryObject));
	}
}

bool UAnimGraphNode_AssetPlayerBase::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UBlueprint* Blueprint : FilterContext.Blueprints)
	{
		UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint);
		if (AnimBlueprint)
		{
			UAnimationAsset* Asset = GetAnimationAsset();
			if(Asset)
			{
				if (AnimBlueprint->TargetSkeleton == nullptr || !AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(Asset->GetSkeleton()))
				{
					// Asset does not use a compatible skeleton with the Blueprint, cannot use
					bIsFilteredOut = true;
					break;
				}
			}
			else if(!UnloadedSkeletonName.IsEmpty())
			{
				if(AnimBlueprint->TargetSkeleton == nullptr || !AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(UnloadedSkeletonName))
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
		else
		{
			// Not an animation Blueprint, cannot use
			bIsFilteredOut = true;
			break;
		}
	}
	return bIsFilteredOut;
}

FText UAnimGraphNode_AssetPlayerBase::GetNodeTitleHelper(ENodeTitleType::Type InTitleType, UEdGraphPin* InAssetPin, const FText& InAssetDesc, const TFunction<FText(UAnimationAsset*)> InPostFixFunctionRef) const
{
	UAnimationAsset* Asset = GetAnimationAsset();
	if (Asset == nullptr)
	{
		// Check for bindings
		bool bHasBinding = false;
		if(InAssetPin != nullptr)
		{
			if (HasBinding(InAssetPin->GetFName()))
			{
				bHasBinding = true;
			}
		}

		// Also check for links
		if (bHasBinding || (InAssetPin && InAssetPin->LinkedTo.Num() > 0))
		{
			return InAssetDesc;
		}
		// check for a default value on the pin
		else if (InAssetPin && InAssetPin->DefaultObject != nullptr)
		{
			return GetNodeTitleForAsset(InTitleType, CastChecked<UAnimationAsset>(InAssetPin->DefaultObject), InAssetDesc, InPostFixFunctionRef);
		}
		else
		{
			return InAssetDesc;
		}
	}
	else
	{
		return GetNodeTitleForAsset(InTitleType, Asset, InAssetDesc, InPostFixFunctionRef);
	}
}

FText UAnimGraphNode_AssetPlayerBase::GetNodeTitleForAsset(ENodeTitleType::Type InTitleType, UAnimationAsset* InAsset, const FText& InAssetDesc, const TFunction<FText(UAnimationAsset*)> InPostFixFunctionRef) const
{
	UAnimationAsset* Asset = GetAnimationAsset();
	check(Asset);
	const FText AssetName = FText::FromString(Asset->GetName());

	if (InTitleType == ENodeTitleType::ListView || InTitleType == ENodeTitleType::MenuTitle)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetName"), AssetName);
		Args.Add(TEXT("AssetDesc"), InAssetDesc);

		if(InPostFixFunctionRef != nullptr)
		{
			const FText PostFix = InPostFixFunctionRef(InAsset);
			if(!PostFix.IsEmpty())
			{
				Args.Add(TEXT("PostFix"), PostFix);
				static const FTextFormat FormatWithPostFix(LOCTEXT("AssetPlayerTitlewithPostFix", "{AssetDesc} {PostFix} '{AssetName}'"));
				return FText::Format(FormatWithPostFix, Args);
			}
		}

		static const FTextFormat FormatWithoutPostFix(LOCTEXT("AssetPlayerTitle", "{AssetDesc} '{AssetName}'"));
		return FText::Format(FormatWithoutPostFix, Args);
	}
	else
	{
		FFormatNamedArguments TitleArgs;
		TitleArgs.Add(TEXT("AssetName"), AssetName);
		TitleArgs.Add(TEXT("AssetDesc"), InAssetDesc);
		FText Title = FText::Format(LOCTEXT("AssetPlayerFullTitle", "{AssetName}\n{AssetDesc}"), TitleArgs);

		AddSyncGroupToNodeTitle(InTitleType, Title);

		return Title;
	}
}

void UAnimGraphNode_AssetPlayerBase::AddSyncGroupToNodeTitle(ENodeTitleType::Type InTitleType, FText& InOutTitle) const
{
	if (InTitleType == ENodeTitleType::FullTitle)
	{
		FStructProperty* NodeProperty = GetFNodeProperty();
		if(NodeProperty->Struct->IsChildOf(FAnimNode_AssetPlayerBase::StaticStruct()))
		{
			const FAnimNode_AssetPlayerBase* Node = NodeProperty->ContainerPtrToValuePtr<FAnimNode_AssetPlayerBase>(this);
				
			FFormatNamedArguments Args;
			Args.Add(TEXT("Title"), InOutTitle);

			if(Node->GetGroupMethod() == EAnimSyncMethod::SyncGroup)
			{
				Args.Add(TEXT("SyncGroupName"), FText::FromName(Node->GetGroupName()));
				static const FTextFormat FormatAssetPlayerNodeSyncGroupSubtitle(LOCTEXT("AssetPlayerNodeSyncGroupSubtitle", "{Title}\nSync group {SyncGroupName}"));
				InOutTitle = FText::Format(FormatAssetPlayerNodeSyncGroupSubtitle, Args);
			}
			else if(Node->GetGroupMethod() == EAnimSyncMethod::Graph)
			{
				static const FTextFormat FormatAssetPlayerNodeGraphSyncGroupSubtitle(LOCTEXT("AssetPlayerNodeGraphSyncGroupSubtitle", "{Title}\nGraph sync group"));
				InOutTitle = FText::Format(FormatAssetPlayerNodeGraphSyncGroupSubtitle, Args);

				UObject* ObjectBeingDebugged = GetAnimBlueprint()->GetObjectBeingDebugged();
				UAnimBlueprintGeneratedClass* GeneratedClass = GetAnimBlueprint()->GetAnimBlueprintGeneratedClass();
				if (ObjectBeingDebugged && GeneratedClass)
				{
					int32 NodeIndex = GeneratedClass->GetNodeIndexFromGuid(NodeGuid);
					if(NodeIndex != INDEX_NONE)
					{
						if(const FName* SyncGroupNamePtr = GeneratedClass->GetAnimBlueprintDebugData().NodeSyncsThisFrame.Find(NodeIndex))
						{
							Args.Add(TEXT("SyncGroupName"), FText::FromName(*SyncGroupNamePtr));
							static const FTextFormat FormatAssetPlayerNodeDynamicGraphSyncGroupSubtitle(LOCTEXT("AssetPlayerNodeDynamicGraphSyncGroupSubtitle", "{Title}\nGraph sync group {SyncGroupName}"));
							InOutTitle = FText::Format(FormatAssetPlayerNodeDynamicGraphSyncGroupSubtitle, Args);
						}
					}
				}
			}
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::ValidateAnimNodeDuringCompilationHelper(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog, UAnimationAsset* InAsset, TSubclassOf<UAnimationAsset> InAssetType, UEdGraphPin* InExposedPin, FName InPropertyName)
{
	UAnimationAsset* AssetToCheck = InAsset;
	if(InExposedPin != nullptr && AssetToCheck == nullptr)
	{
		AssetToCheck = Cast<UAnimationAsset>(InExposedPin->DefaultObject);
	}

	if(AssetToCheck == nullptr)
	{
		if(!GetAnimBlueprint()->bIsTemplate)
		{
			// Check for bindings
			bool bHasBinding = false;
			bool bAlwaysDynamic = false;
			if(InExposedPin != nullptr)
			{
				if(HasBinding(InExposedPin->GetFName()))
				{
					bHasBinding = true;
				}
			}

			if(AlwaysDynamicProperties.Contains(InPropertyName))
			{
				bAlwaysDynamic = true;
			}
			
			// we may have a connected node or binding
			if(!bAlwaysDynamic)
			{
				if (InExposedPin == nullptr || (InExposedPin->LinkedTo.Num() == 0 && !bHasBinding))
				{
					MessageLog.Error(*FText::Format(LOCTEXT("MissingAssetFormat", "@@ references an unknown {0}"), InAssetType->GetDisplayNameText()).ToString(), this);
				}
			}
		}
	}
	else 
	{
		USkeleton* AssetSkeleton = AssetToCheck->GetSkeleton();
		
		// if asset doesn't have a skeleton, it might be due to the asset no being not loaded yet
		if(AssetSkeleton == nullptr)
		{
			MessageLog.Error(*FText::Format(LOCTEXT("MissingSkeletonFormat", "@@ references {0} that uses a missing skeleton @@"), InAssetType->GetDisplayNameText()).ToString(), this, AssetSkeleton);
		}
	}
}

void UAnimGraphNode_AssetPlayerBase::PreloadRequiredAssetsHelper(UAnimationAsset* InAsset, UEdGraphPin* InExposedPin)
{
	UAnimationAsset* AssetToLoad = InAsset;
	if(InExposedPin != nullptr && AssetToLoad == nullptr)
	{
		AssetToLoad = Cast<UAnimationAsset>(InExposedPin->DefaultObject);
	}

	if(AssetToLoad)
	{
		// HandleAnimReferenceCollection adds 'this' as well as any recursive dependencies
		TSet<UAnimationAsset*> PreloadedAssets;
		TSet<UAnimationAsset*> DiscoveredAssets;
		TArray<UAnimationAsset*> AssetAndDependencies;
		AssetToLoad->HandleAnimReferenceCollection(AssetAndDependencies, true);
		DiscoveredAssets.Append(AssetAndDependencies);

		while(AssetAndDependencies.Num() != 0)
		{
			UAnimationAsset* Asset = AssetAndDependencies.Pop(EAllowShrinking::No);
			if (!PreloadedAssets.Contains(Asset))
			{
				PreloadedAssets.Add(Asset);
				PreloadObject(Asset);

				// catch any extra dependencies found after preloading the object
				// this will have some redundancy, but that can't be avoided without 
				// reworking HandleAnimReferenceCollection
				TArray<UAnimationAsset*> Dependencies;
				Asset->HandleAnimReferenceCollection(Dependencies, true);
				for (UAnimationAsset* AssetDependency : Dependencies)
				{
					if (!DiscoveredAssets.Contains(AssetDependency))
					{
						DiscoveredAssets.Add(AssetDependency);
						AssetAndDependencies.Add(AssetDependency);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
