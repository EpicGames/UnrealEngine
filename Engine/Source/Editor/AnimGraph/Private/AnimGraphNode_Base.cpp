// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimGraphNode_Base.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/AnimInstance.h"
#include "AnimationGraphSchema.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Components/SkeletalMeshComponent.h"
#include "AnimBlueprintNodeOptionalPinManager.h"
#include "IAnimNodeEditMode.h"
#include "AnimNodeEditModes.h"
#include "AnimationGraph.h"
#include "EditorModeManager.h"

#include "AnimationEditorUtils.h"
#include "UObject/UnrealType.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AnimBlueprintCompiler.h"
#include "AnimBlueprintExtension_Base.h"
#include "AnimBlueprintExtension_Attributes.h"
#include "AnimBlueprintExtension_PropertyAccess.h"
#include "AnimBlueprintExtension_Tag.h"
#include "IAnimBlueprintCompilationContext.h"
#include "AnimBlueprintCompilationContext.h"
#include "AnimBlueprintExtension.h"
#include "FindInBlueprintManager.h"
#include "IPropertyAccessBlueprintBinding.h"
#include "IPropertyAccessEditor.h"
#include "ScopedTransaction.h"
#include "Algo/Accumulate.h"
#include "Fonts/FontMeasure.h"
#include "UObject/ReleaseObjectVersion.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Fonts/FontMeasure.h"
#include "ObjectEditorUtils.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "AnimGraphNodeBinding_Base.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Base)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_Base"

/////////////////////////////////////////////////////
// UAnimGraphNode_Base

UAnimGraphNode_Base::UAnimGraphNode_Base(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_Base::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	TUniquePtr<IAnimBlueprintCompilationContext> CompilationContext = IAnimBlueprintCompilationContext::Get(CompilerContext);
	UAnimBlueprintExtension_Base* Extension = UAnimBlueprintExtension_Base::GetExtension<UAnimBlueprintExtension_Base>(GetAnimBlueprint());
	Extension->CreateEvaluationHandlerForNode(*CompilationContext.Get(), this);

	if (Binding)
	{
		Binding->OnExpandNode(*CompilationContext.Get(), this, SourceGraph);
	}
}

void UAnimGraphNode_Base::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))
	{
		FOptionalPinManager::CacheShownPins(ShowPinForProperties, OldShownPins);
	}
}

void UAnimGraphNode_Base::PostEditUndo()
{
	Super::PostEditUndo();

	if(HasValidBlueprint())
	{
		// Node may have been removed or added by undo/redo so make sure extensions are refreshed
		GetAnimBlueprint()->RequestRefreshExtensions();
	}
}

void UAnimGraphNode_Base::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin) && MemberPropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, ShowPinForProperties))
	{
		FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
		GetSchema()->ReconstructNode(*this);
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, InitialUpdateFunction))
	{
		GetFNode()->InitialUpdateFunction.SetFromFunction(InitialUpdateFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		GetAnimBlueprint()->RequestRefreshExtensions();
		GetSchema()->ReconstructNode(*this);
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, BecomeRelevantFunction))
	{
		GetFNode()->BecomeRelevantFunction.SetFromFunction(BecomeRelevantFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		GetAnimBlueprint()->RequestRefreshExtensions();
		GetSchema()->ReconstructNode(*this);
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, UpdateFunction))
	{
		GetFNode()->UpdateFunction.SetFromFunction(UpdateFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		GetSchema()->ReconstructNode(*this);
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, Tag))
	{
		GetAnimBlueprint()->RequestRefreshExtensions();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, Binding))
	{
		GetAnimBlueprint()->RequestRefreshExtensions();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	PropertyChangeEvent.Broadcast(PropertyChangedEvent);
}

void UAnimGraphNode_Base::SetPinVisibility(bool bInVisible, int32 InOptionalPinIndex)
{
	if(ShowPinForProperties[InOptionalPinIndex].bShowPin != bInVisible)
	{
		FOptionalPinManager::CacheShownPins(ShowPinForProperties, OldShownPins);

		ShowPinForProperties[InOptionalPinIndex].bShowPin = bInVisible;
		
		FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
		ReconstructNode();

		PinVisibilityChangedEvent.Broadcast(bInVisible, InOptionalPinIndex);
	}
}

void UAnimGraphNode_Base::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::AnimationGraphNodeBindingsDisplayedAsPins)
		{
			// Push any bindings to optional pins
			bool bPushedBinding = false;
			for(const TPair<FName, FAnimGraphNodePropertyBinding>& BindingPair : PropertyBindings_DEPRECATED)
			{
				for(FOptionalPinFromProperty& OptionalPin : ShowPinForProperties)
				{
					if(OptionalPin.bCanToggleVisibility && !OptionalPin.bShowPin && OptionalPin.PropertyName == BindingPair.Key)
					{
						OptionalPin.bShowPin = true;
						bPushedBinding = true;
					}
				}
			}

			if(bPushedBinding)
			{
				FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
			}
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimGraphNodeBindingExtensions)
		{
			// Move internal bindings to extension
			UAnimGraphNodeBinding_Base* NewBinding = NewObject<UAnimGraphNodeBinding_Base>(this);
			NewBinding->PropertyBindings = PropertyBindings_DEPRECATED;
			Binding = NewBinding;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FixMissingAnimGraphNodeBindingExtensions)
		{
			if(Binding == nullptr)
			{
				Binding = NewObject<UAnimGraphNodeBinding_Base>(this);
			}
		}
	}
}

void UAnimGraphNode_Base::EnsureBindingsArePresent()
{
	if(Binding == nullptr)
	{
		if (UAnimBlueprint* AnimBlueprint = GetTypedOuter<UAnimBlueprint>())
		{
			UClass* BindingClass = AnimBlueprint->GetDefaultBindingClass();
			if (BindingClass == nullptr)
			{
				BindingClass = UAnimGraphNodeBinding_Base::StaticClass();
			}

			Binding = NewObject<UAnimGraphNodeBinding>(this, BindingClass);
		}
	}
}

void UAnimGraphNode_Base::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	EnsureBindingsArePresent();

	// This makes sure that all anim BP extensions are registered that this node needs
	UAnimBlueprintExtension::RequestExtensionsForNode(this);
}

void UAnimGraphNode_Base::PostPasteNode()
{
	Super::PostPasteNode();

	EnsureBindingsArePresent();

	// This makes sure that all anim BP extensions are registered that this node needs
	UAnimBlueprintExtension::RequestExtensionsForNode(this);
}

void UAnimGraphNode_Base::DestroyNode()
{
	if(HasValidBlueprint())
	{
		// This node may have been the last using its extension, so refresh
		GetAnimBlueprint()->RequestRefreshExtensions();
		
		// Cleanup the pose watch if one exists on this node
		AnimationEditorUtils::RemovePoseWatchFromNode(this, GetAnimBlueprint());
	}

	Super::DestroyNode();
}

void UAnimGraphNode_Base::CreateOutputPins()
{
	if (!IsSinkNode())
	{
		CreatePin(EGPD_Output, UAnimationGraphSchema::PC_Struct, FPoseLink::StaticStruct(), TEXT("Pose"));
	}
}

void UAnimGraphNode_Base::ValidateFunctionRef(FName InPropertyName, const FMemberReference& InRef, const FText& InFunctionName, FCompilerResultsLog& MessageLog)
{
	if (InRef.GetMemberName() != NAME_None)
	{
		const UFunction* Function = InRef.ResolveMember<UFunction>(GetAnimBlueprint()->SkeletonGeneratedClass);
		if(Function == nullptr)
		{
			MessageLog.Error(*FText::Format(LOCTEXT("CouldNotResolveFunctionErrorFormat", "Could not resolve {0} function @@"), InFunctionName).ToString(), this);
		}
		else
		{
			// Check signatures match
			const FProperty* Property = GetClass()->FindPropertyByName(InPropertyName);
			check(Property);
			const FString& PrototypeFunctionName = Property->GetMetaData("PrototypeFunction");
			const UFunction* PrototypeFunction = PrototypeFunctionName.IsEmpty() ? nullptr : FindObject<UFunction>(nullptr, *PrototypeFunctionName);
			if (PrototypeFunction != nullptr && !PrototypeFunction->IsSignatureCompatibleWith(Function))
			{
				MessageLog.Error(*FText::Format(LOCTEXT("FunctionSignatureErrorFormat", "{0} function's signature is not compatible @@"), InFunctionName).ToString(), this);
			}

			// Check thread safety
			if (!FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(Function))
			{
				MessageLog.Error(*FText::Format(LOCTEXT("FunctionThreadSafetyErrorFormat", "{0} function is not thread safe @@"), InFunctionName).ToString(), this);
			}
		}
	}
}

void UAnimGraphNode_Base::GetBoundFunctionsInfo(TArray<TPair<FName, FName>>& InOutBindingsInfo)
{
	FName CategoryName = TEXT("Functions");

	if (InitialUpdateFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, InitialUpdateFunction));
	}
	
	if (BecomeRelevantFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, BecomeRelevantFunction));
	}

	if (UpdateFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, UpdateFunction));
	}
};

void UAnimGraphNode_Base::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	// Validate any bone references we have
	for(const TPair<const FStructProperty*, const void*>& PropertyValuePair : TPropertyValueRange<FStructProperty>(GetClass(), this))
	{
		if(PropertyValuePair.Key->Struct == FBoneReference::StaticStruct())
		{
			const FBoneReference& BoneReference = *(const FBoneReference*)PropertyValuePair.Value;

			// Temporary fix where skeleton is not fully loaded during AnimBP compilation and thus virtual bone name check is invalid UE-39499 (NEED FIX) 
			if (ForSkeleton && !ForSkeleton->HasAnyFlags(RF_NeedPostLoad))
			{
				if (BoneReference.BoneName != NAME_None)
				{
					if (ForSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneReference.BoneName) == INDEX_NONE)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("BoneName"), FText::FromName(BoneReference.BoneName));

						MessageLog.Warning(*FText::Format(LOCTEXT("NoBoneFoundToModify", "@@ - Bone {BoneName} not found in Skeleton"), Args).ToString(), this);
					}
				}
			}
		}
	}

	bool bBaseClassIsExperimental = false;
	bool bBaseClassIsEarlyAccess = false;
	FString MostDerivedDevelopmentClassName;
	FObjectEditorUtils::GetClassDevelopmentStatus(GetClass(), bBaseClassIsExperimental, bBaseClassIsEarlyAccess, MostDerivedDevelopmentClassName);
	if (bBaseClassIsExperimental)
	{
		MessageLog.Note(*(LOCTEXT("ExperimentalNode", "@@ - Node is experimental")).ToString(), this);
	}
	if (bBaseClassIsEarlyAccess)
	{
		MessageLog.Note(*(LOCTEXT("EarlyAccessNode", "@@ - Node is in early access")).ToString(), this);
	}

	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, InitialUpdateFunction), InitialUpdateFunction, LOCTEXT("InitialUpdateFunctionName", "Initial Update"), MessageLog);
	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, BecomeRelevantFunction), BecomeRelevantFunction, LOCTEXT("BecomeRelevantFunctionName", "Become Relevant"), MessageLog);
	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, UpdateFunction), UpdateFunction, LOCTEXT("UpdateFunctionName", "Update"), MessageLog);
}

void UAnimGraphNode_Base::CopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	InPerNodeContext.GetTargetProperty()->CopyCompleteValue(InPerNodeContext.GetDestinationPtr(), InPerNodeContext.GetSourcePtr());

	OnCopyTermDefaultsToDefaultObject(InCompilationContext, InPerNodeContext, OutCompiledData);
}

void UAnimGraphNode_Base::OverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const
{
	OnOverrideAssets(InContext);
}

void UAnimGraphNode_Base::InternalPinCreation(TArray<UEdGraphPin*>* OldPins)
{
	// preload required assets first before creating pins
	PreloadRequiredAssets();

	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();
	if (const FStructProperty* NodeStruct = GetFNodeProperty())
	{
		// Display any currently visible optional pins
		{
			UObject* NodeDefaults = GetArchetype();
			FAnimBlueprintNodeOptionalPinManager OptionalPinManager(this, OldPins);
			OptionalPinManager.AllocateDefaultPins(NodeStruct->Struct, NodeStruct->ContainerPtrToValuePtr<uint8>(this), NodeDefaults ? NodeStruct->ContainerPtrToValuePtr<uint8>(NodeDefaults) : nullptr);
		}

		// Create the output pin, if needed
		CreateOutputPins();
	}

	if (Binding)
	{
		Binding->OnInternalPinCreation(this);
	}
}

void UAnimGraphNode_Base::AllocateDefaultPins()
{
	InternalPinCreation(nullptr);

	CreateCustomPins(nullptr);
}

void UAnimGraphNode_Base::ReconstructNode()
{
	if (Binding)
	{
		Binding->OnReconstructNode(this);
	}
	
	Super::ReconstructNode();
}

void UAnimGraphNode_Base::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	InternalPinCreation(&OldPins);

	CreateCustomPins(&OldPins);

	RestoreSplitPins(OldPins);
}

bool UAnimGraphNode_Base::CanJumpToDefinition() const
{
	return GetJumpTargetForDoubleClick() != nullptr;
}

void UAnimGraphNode_Base::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(HyperlinkTarget);
	}
}

FLinearColor UAnimGraphNode_Base::GetNodeTitleColor() const
{
	return FLinearColor::Black;
}

UScriptStruct* UAnimGraphNode_Base::GetFNodeType() const
{
	UScriptStruct* BaseFStruct = FAnimNode_Base::StaticStruct();

	for (TFieldIterator<FProperty> PropIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = CastField<FStructProperty>(*PropIt))
		{
			if (StructProp->Struct->IsChildOf(BaseFStruct))
			{
				return StructProp->Struct;
			}
		}
	}

	return NULL;
}

FStructProperty* UAnimGraphNode_Base::GetFNodeProperty() const
{
	UScriptStruct* BaseFStruct = FAnimNode_Base::StaticStruct();

	for (TFieldIterator<FProperty> PropIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
	{
		if (FStructProperty* StructProp = CastField<FStructProperty>(*PropIt))
		{
			if (StructProp->Struct->IsChildOf(BaseFStruct))
			{
				return StructProp;
			}
		}
	}

	return NULL;
}

FAnimNode_Base* UAnimGraphNode_Base::GetFNode()
{
	if(FStructProperty* Property = GetFNodeProperty())
	{
		return Property->ContainerPtrToValuePtr<FAnimNode_Base>(this);
	}

	return nullptr;
}

FString UAnimGraphNode_Base::GetNodeCategory() const
{
	return TEXT("Misc.");
}

void UAnimGraphNode_Base::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "AnimGraphNode" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetName() ));
}

void UAnimGraphNode_Base::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UAnimGraphNode_Base::GetMenuCategory() const
{
	return FText::FromString(GetNodeCategory());
}

void UAnimGraphNode_Base::GetPinAssociatedProperty(const UScriptStruct* NodeType, const UEdGraphPin* InputPin, FProperty*& OutProperty, int32& OutIndex) const
{
	OutProperty = nullptr;
	OutIndex = INDEX_NONE;

	//@TODO: Name-based hackery, avoid the roundtrip and better indicate when it's an array pose pin
	const FString PinNameStr = InputPin->PinName.ToString();
	const int32 UnderscoreIndex = PinNameStr.Find(TEXT("_"), ESearchCase::CaseSensitive);
	if (UnderscoreIndex != INDEX_NONE)
	{
		const FString ArrayName = PinNameStr.Left(UnderscoreIndex);

		if (FArrayProperty* ArrayProperty = FindFProperty<FArrayProperty>(NodeType, *ArrayName))
		{
			const int32 ArrayIndex = FCString::Atoi(*(PinNameStr.Mid(UnderscoreIndex + 1)));

			OutProperty = ArrayProperty;
			OutIndex = ArrayIndex;
		}
	}
	
	// If the array check failed or we have no underscores
	if(OutProperty == nullptr)
	{
		if (FProperty* Property = FindFProperty<FProperty>(NodeType, InputPin->PinName))
		{
			OutProperty = Property;
			OutIndex = INDEX_NONE;
		}
	}
}

FPoseLinkMappingRecord UAnimGraphNode_Base::GetLinkIDLocation(const UScriptStruct* NodeType, UEdGraphPin* SourcePin)
{
	if (SourcePin->LinkedTo.Num() > 0)
	{
		if (UAnimGraphNode_Base* LinkedNode = Cast<UAnimGraphNode_Base>(FBlueprintEditorUtils::FindFirstCompilerRelevantNode(SourcePin->LinkedTo[0])))
		{
			//@TODO: Name-based hackery, avoid the roundtrip and better indicate when it's an array pose pin
			const FString SourcePinName = SourcePin->PinName.ToString();
			const int32 UnderscoreIndex = SourcePinName.Find(TEXT("_"), ESearchCase::CaseSensitive);
			if (UnderscoreIndex != INDEX_NONE)
			{
				const FString ArrayName = SourcePinName.Left(UnderscoreIndex);

				if (FArrayProperty* ArrayProperty = FindFProperty<FArrayProperty>(NodeType, *ArrayName))
				{
					if (FStructProperty* Property = CastField<FStructProperty>(ArrayProperty->Inner))
					{
						if (Property->Struct->IsChildOf(FPoseLinkBase::StaticStruct()))
						{
							const int32 ArrayIndex = FCString::Atoi(*(SourcePinName.Mid(UnderscoreIndex + 1)));
							return FPoseLinkMappingRecord::MakeFromArrayEntry(this, LinkedNode, ArrayProperty, ArrayIndex);
						}
					}
				}
			}
			else
			{
				if (FStructProperty* Property = FindFProperty<FStructProperty>(NodeType, SourcePin->PinName))
				{
					if (Property->Struct->IsChildOf(FPoseLinkBase::StaticStruct()))
					{
						return FPoseLinkMappingRecord::MakeFromMember(this, LinkedNode, Property);
					}
				}
			}
		}
	}

	return FPoseLinkMappingRecord::MakeInvalid();
}

void UAnimGraphNode_Base::CreatePinsForPoseLink(FProperty* PoseProperty, int32 ArrayIndex)
{
	UScriptStruct* A2PoseStruct = FA2Pose::StaticStruct();

	// pose input
	const FName NewPinName = (ArrayIndex == INDEX_NONE) ? PoseProperty->GetFName() : *FString::Printf(TEXT("%s_%d"), *(PoseProperty->GetName()), ArrayIndex);
	CreatePin(EGPD_Input, UAnimationGraphSchema::PC_Struct, A2PoseStruct, NewPinName);
}

void UAnimGraphNode_Base::PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const
{
	if (Pin->Direction == EGPD_Output)
	{
		if (Pin->PinName == TEXT("Pose"))
		{
			DisplayName.Reset();
		}
	}
}

bool UAnimGraphNode_Base::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	return DesiredSchema->GetClass()->IsChildOf(UAnimationGraphSchema::StaticClass());
}

FString UAnimGraphNode_Base::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/Animation");
}

void UAnimGraphNode_Base::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	if (UAnimationGraphSchema::IsLocalSpacePosePin(Pin.PinType))
	{
		HoverTextOut = TEXT("Animation Pose");
	}
	else if (UAnimationGraphSchema::IsComponentSpacePosePin(Pin.PinType))
	{
		HoverTextOut = TEXT("Animation Pose (Component Space)");
	}
	else
	{
		Super::GetPinHoverText(Pin, HoverTextOut);
	}
}

void UAnimGraphNode_Base::ProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimBlueprintExtension_Base* Extension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_Base>(GetAnimBlueprint());

	// Record pose pins for later patchup
	Extension->ProcessPosePins(this, InCompilationContext, OutCompiledData);

	// Process bindings on this node
	if(Binding)
	{
		Binding->ProcessDuringCompilation(InCompilationContext, OutCompiledData);
	}

	// Resolve functions
	GetFNode()->InitialUpdateFunction.SetFromFunction(InitialUpdateFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
	GetFNode()->BecomeRelevantFunction.SetFromFunction(BecomeRelevantFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
	GetFNode()->UpdateFunction.SetFromFunction(UpdateFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode())); 

	// Insert tag, if any
	if(Tag != NAME_None)
	{
		UAnimBlueprintExtension_Tag* TagExtension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_Tag>(GetAnimBlueprint());
		TagExtension->AddTaggedNode(this, InCompilationContext);
	}
	
	// Call the override point
	OnProcessDuringCompilation(InCompilationContext, OutCompiledData);
}

void UAnimGraphNode_Base::HandleAnimReferenceCollection(UAnimationAsset* AnimAsset, TArray<UAnimationAsset*>& AnimationAssets) const
{
	if(AnimAsset)
	{
		AnimAsset->HandleAnimReferenceCollection(AnimationAssets, true);
	}
}

void UAnimGraphNode_Base::OnNodeSelected(bool bInIsSelected, FEditorModeTools& InModeTools, FAnimNode_Base* InRuntimeNode)
{
	const FEditorModeID ModeID = GetEditorMode();
	if (ModeID != NAME_None)
	{
		if (bInIsSelected)
		{
			InModeTools.ActivateMode(ModeID);
			if (FEdMode* EdMode = InModeTools.GetActiveMode(ModeID))
			{
				static_cast<IAnimNodeEditMode*>(EdMode)->EnterMode(this, InRuntimeNode);
			}
		}
		else
		{
			if (FEdMode* EdMode = InModeTools.GetActiveMode(ModeID))
			{
				static_cast<IAnimNodeEditMode*>(EdMode)->ExitMode();
			}
			InModeTools.DeactivateMode(ModeID);
		}
	}
}

void UAnimGraphNode_Base::OnPoseWatchChanged(const bool IsPoseWatchEnabled, TObjectPtr<class UPoseWatch> InPoseWatch, FEditorModeTools& InModeTools, FAnimNode_Base* InRuntimeNode)
{
	const FEditorModeID ModeID = GetEditorMode();
	if (ModeID != NAME_None)
	{
		if (IsPoseWatchEnabled)
		{
			InModeTools.ActivateMode(ModeID);
			if (FEdMode* EdMode = InModeTools.GetActiveMode(ModeID))
			{
				const bool bSupportsPoseWatch =
					   static_cast<IAnimNodeEditMode*>(EdMode)->SupportsPoseWatch()
					|| EdMode->GetID() == AnimNodeEditModes::AnimNode;

				if (bSupportsPoseWatch)
				{
					static_cast<IAnimNodeEditMode*>(EdMode)->RegisterPoseWatchedNode(this, InRuntimeNode);
				}
				else
				{
					InModeTools.DeactivateMode(ModeID);
				}
			}
		}
		else
		{
			if (FEdMode* EdMode = InModeTools.GetActiveMode(ModeID))
			{
				static_cast<IAnimNodeEditMode*>(EdMode)->ExitMode();
			}
			InModeTools.DeactivateMode(ModeID);
		}
	}
}

FEditorModeID UAnimGraphNode_Base::GetEditorMode() const
{
	return AnimNodeEditModes::AnimNode;
}

void UAnimGraphNode_Base::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp, const bool IsSelected, const bool IsPoseWatchEnabled) const
{
	if (IsSelected)
	{
		Draw(PDI, PreviewSkelMeshComp); 
	}
}

FAnimNode_Base* UAnimGraphNode_Base::FindDebugAnimNode(USkeletalMeshComponent* InPreviewSkelMeshComp) const
{
	FAnimNode_Base* DebugNode = nullptr;

	if (InPreviewSkelMeshComp != nullptr)
	{
		auto FindDebugNode = [this](UAnimInstance* InAnimInstance) -> FAnimNode_Base* 
		{
			if (!InAnimInstance)
			{
				return nullptr;
			}
			UAnimBlueprintGeneratedClass* AnimBlueprintClass = Cast<UAnimBlueprintGeneratedClass>(InAnimInstance->GetClass());
			if (!AnimBlueprintClass)
			{
				return nullptr;
			}

			// Search for the node by GUID, since we can have multiple instantiations of this node, and therefore
			// different pointers from ourselves even though they represent the same node in the graph.
			const int32 AnimNodeIndex = AnimBlueprintClass->GetNodeIndexFromGuid(NodeGuid, EPropertySearchMode::Hierarchy);
			if (AnimNodeIndex == INDEX_NONE)
			{
				return nullptr;
			}
			
			return AnimBlueprintClass->GetAnimNodeProperties()[AnimNodeIndex]->ContainerPtrToValuePtr<FAnimNode_Base>(InAnimInstance);
		};
	
		// Try to find this node on the anim BP.
		DebugNode = FindDebugNode(InPreviewSkelMeshComp->GetAnimInstance());
		
		// Failing that, try the post-process BP instead.
		if (!DebugNode)
		{
			DebugNode = FindDebugNode(InPreviewSkelMeshComp->GetPostProcessInstance());			
		}
	}

	return DebugNode;
}

EAnimAssetHandlerType UAnimGraphNode_Base::SupportsAssetClass(const UClass* AssetClass) const
{
	return EAnimAssetHandlerType::NotSupported;
}


void UAnimGraphNode_Base::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	CopyPinDefaultsToNodeData(Pin);

	if(UAnimationGraph* AnimationGraph = Cast<UAnimationGraph>(GetGraph()))
	{
		AnimationGraph->OnPinDefaultValueChanged.Broadcast(Pin);
	}
}

FString UAnimGraphNode_Base::GetPinMetaData(FName InPinName, FName InKey)
{
	FString MetaData = Super::GetPinMetaData(InPinName, InKey);
	if(MetaData.IsEmpty())
	{
		// Check properties of our anim node
		if(FStructProperty* NodeStructProperty = GetFNodeProperty())
		{
			for (TFieldIterator<FProperty> It(NodeStructProperty->Struct); It; ++It)
			{
				const FProperty* Property = *It;
				if (Property && Property->GetFName() == InPinName)
				{
					return Property->GetMetaData(InKey);
				}
			}
		}
	}
	return MetaData;
}

void UAnimGraphNode_Base::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	const auto ConditionallyTagNodeFuncRef = [&OutTaggedMetaData](const FMemberReference& FuncMember, const FText& LocText)
	{
		if (IsPotentiallyBoundFunction(FuncMember))
		{
			const FText FunctionName = FText::FromName(FuncMember.GetMemberName());
			OutTaggedMetaData.Add(FSearchTagDataPair(LocText, FunctionName));
		}
	};

	// Conditionally include anim node function references as part of the node's search metadata
	ConditionallyTagNodeFuncRef(InitialUpdateFunction, LOCTEXT("InitialUpdateFunctionName", "Initial Update"));
	ConditionallyTagNodeFuncRef(BecomeRelevantFunction, LOCTEXT("BecomeRelevantFunctionName", "Become Relevant"));
	ConditionallyTagNodeFuncRef(UpdateFunction, LOCTEXT("UpdateFunctionName", "Update"));

	if(Tag != NAME_None)
	{
		OutTaggedMetaData.Add(FSearchTagDataPair(LOCTEXT("Tag", "Tag"), FText::FromName(Tag)));
	}
}

void UAnimGraphNode_Base::AddPinSearchMetaDataInfo(const UEdGraphPin* InPin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddPinSearchMetaDataInfo(InPin, OutTaggedMetaData);
	
	if(Binding)
	{
		FName BindingName;
		FProperty* PinProperty;
		int32 OptionalPinIndex;
		if (GetPinBindingInfo(InPin->GetFName(), BindingName, PinProperty, OptionalPinIndex))
		{
			Binding->AddPinSearchMetaDataInfo(InPin, BindingName, OutTaggedMetaData);
		}
	}
}

bool UAnimGraphNode_Base::IsPinExposedAndLinked(const FString& InPinName, const EEdGraphPinDirection InDirection) const
{
	UEdGraphPin* Pin = FindPin(InPinName, InDirection);
	return Pin != nullptr && Pin->LinkedTo.Num() > 0 && Pin->LinkedTo[0] != nullptr;
}

bool UAnimGraphNode_Base::IsPinExposedAndBound(const FString& InPinName, const EEdGraphPinDirection InDirection) const
{
	UEdGraphPin* Pin = FindPin(InPinName, InDirection);
	return Pin != nullptr && Pin->LinkedTo.Num() == 0 && Binding && Binding->HasBinding(Pin->GetFName(), true);
}

bool UAnimGraphNode_Base::IsPinUnlinkedUnboundAndUnset(const FString& InPinName, const EEdGraphPinDirection InDirection) const
{
	UEdGraphPin* Pin = FindPin(InPinName, InDirection);
	if (Pin)
	{
		return Pin->LinkedTo.Num() == 0 && (Binding == nullptr || !Binding->HasBinding(Pin->GetFName(), true)) && Pin->DefaultValue == Pin->AutogeneratedDefaultValue;
	}
	return false;
}

bool UAnimGraphNode_Base::IsPinBindable(const UEdGraphPin* InPin) const
{
	if(const FProperty* PinProperty = GetPinProperty(InPin->GetFName()))
	{
		const int32 OptionalPinIndex = ShowPinForProperties.IndexOfByPredicate([PinProperty](const FOptionalPinFromProperty& InOptionalPin)
		{
			return PinProperty->GetFName() == InOptionalPin.PropertyName;
		});

		return OptionalPinIndex != INDEX_NONE;
	}

	return false;
}

bool UAnimGraphNode_Base::GetPinBindingInfo(FName InPinName, FName& OutBindingName, FProperty*& OutPinProperty, int32& OutOptionalPinIndex) const
{
	OutPinProperty = GetPinProperty(InPinName);
	if(OutPinProperty)
	{
		OutOptionalPinIndex = ShowPinForProperties.IndexOfByPredicate([OutPinProperty](const FOptionalPinFromProperty& InOptionalPin)
		{
			return OutPinProperty->GetFName() == InOptionalPin.PropertyName;
		});

		OutBindingName = InPinName;
		return OutOptionalPinIndex != INDEX_NONE;
	}

	return false;
}

bool UAnimGraphNode_Base::HasBinding(FName InBindingName) const
{
	if (Binding)
	{
		return Binding->HasBinding(InBindingName, false);
	}

	return false;
}

bool UAnimGraphNode_Base::RedirectBinding(FName OldPropertyBindingName, FName NewPropertyBindingName)
{
	bool bRenamed = false;

	if (Binding)
	{
		const FString OldPropertyBindingNameStr = OldPropertyBindingName.ToString();
		const FString NewPropertyBindingNameStr = NewPropertyBindingName.ToString();
		
		Binding->UpdateBindingNames([this, &bRenamed, &OldPropertyBindingNameStr, &NewPropertyBindingNameStr](const FString& InBindingName) -> FString
		{
			// Ignore any property binding that does not match exactly with what we requested.
			if (InBindingName != OldPropertyBindingNameStr)
			{
				return InBindingName;
			}

			// We found our target binding name, replace it!
			bRenamed = true;
			return NewPropertyBindingNameStr;
		});

		if (bRenamed)
		{
			// Remove old binding name if it still exists (UpdateBindingNames() doesn't remove old binding entry).
			if (HasBinding(OldPropertyBindingName))
			{
				Binding->RemoveBindings(OldPropertyBindingName);
			}
			
			Binding->Modify();
		}
	}

	return bRenamed;
}

void UAnimGraphNode_Base::RemoveBindings(FName InBindingName)
{
	if (Binding)
	{
		return Binding->RemoveBindings(InBindingName);
	}
}

void UAnimGraphNode_Base::SetTag(FName InTag)
{
	FName OldTag = Tag;
	Tag = InTag;
	if(Tag != OldTag)
	{
		GetAnimBlueprint()->RequestRefreshExtensions();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

FProperty* UAnimGraphNode_Base::GetPinProperty(const UEdGraphPin* InPin) const
{
	return GetPinProperty(InPin->GetFName());
}


FProperty* UAnimGraphNode_Base::GetPinProperty(FName InPinName) const
{
	// Compare FName without number to make sure we catch array properties that are split into multiple pins
	FName ComparisonName = InPinName;
	ComparisonName.SetNumber(0);
	
	return GetFNodeType()->FindPropertyByName(ComparisonName);
}

void UAnimGraphNode_Base::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	
	if(Pin->LinkedTo.Num() > 0)
	{
		if (Binding)
		{
			// If we have links, clear any bindings
			// Compare FName without number to make sure we catch array properties that are split into multiple pins
			FName ComparisonName = Pin->GetFName();
			ComparisonName.SetNumber(0);

			Binding->RemoveBindings(ComparisonName);
		}
	}
}

void UAnimGraphNode_Base::AutowireNewNode(UEdGraphPin* FromPin)
{
	// Ensure the pin is valid, a pose pin, and has a single link
	if (FromPin && UAnimationGraphSchema::IsPosePin(FromPin->PinType))
	{
		auto FindFirstPosePinInDirection = [this](EEdGraphPinDirection Direction) -> UEdGraphPin*
		{
			UEdGraphPin** PinToConnectTo = Pins.FindByPredicate([Direction](UEdGraphPin* Pin) -> bool
            {
                return Pin && Pin->Direction == Direction && UAnimationGraphSchema::IsPosePin(Pin->PinType);
            });

			return PinToConnectTo ? *PinToConnectTo : nullptr;
		};
		
		// Get the linked pin, if valid, and ensure it iss also a pose pin
		UEdGraphPin* LinkedPin = FromPin->LinkedTo.Num() == 1 ? FromPin->LinkedTo[0] : nullptr;
		if (LinkedPin && UAnimationGraphSchema::IsPosePin(LinkedPin->PinType))
		{
			// Retrieve the first pin, of similar direction, from this node
			UEdGraphPin* PinToConnectTo = FindFirstPosePinInDirection(FromPin->Direction);
			if (PinToConnectTo)
			{
				ensure(GetSchema()->TryCreateConnection(LinkedPin, PinToConnectTo));
			}
		}

		// Link this node to the FromPin, so find the first pose pin of opposite direction on this node
		UEdGraphPin* PinToConnectTo = FindFirstPosePinInDirection(FromPin->Direction == EEdGraphPinDirection::EGPD_Input ? EEdGraphPinDirection::EGPD_Output : EEdGraphPinDirection::EGPD_Input);
		if (PinToConnectTo)
		{
			ensure(GetSchema()->TryCreateConnection(FromPin, PinToConnectTo));
		}
	}
	else
	{
		UK2Node::AutowireNewNode(FromPin);
	}
}

TSharedRef<SWidget> UAnimGraphNode_Base::MakePropertyBindingWidget(const FAnimPropertyBindingWidgetArgs& InArgs)
{
	const UAnimGraphNode_Base* FirstAnimGraphNode = InArgs.Nodes[0];
	UClass* BindingClass = FirstAnimGraphNode->Binding ? FirstAnimGraphNode->Binding->GetClass() : nullptr;
	UAnimGraphNodeBinding* BindingCDO = BindingClass ? CastChecked<UAnimGraphNodeBinding>(BindingClass->GetDefaultObject()) : nullptr;

	// Check all nodes have the same kinds of binding - we cant do multi-edit with bindings of differing types
	if (InArgs.Nodes.Num() > 0)
	{
		for (int32 NodeIndex = 1; NodeIndex < InArgs.Nodes.Num(); ++NodeIndex)
		{
			UAnimGraphNode_Base* OtherAnimGraphNode = InArgs.Nodes[NodeIndex];
			UClass* OtherBindingClass = OtherAnimGraphNode->Binding ? OtherAnimGraphNode->Binding->GetClass() : nullptr;
			if (OtherBindingClass == nullptr || BindingClass != OtherBindingClass)
			{
				BindingCDO = nullptr;
				break;
			}
		}
	}
	
	if (BindingCDO)
	{
		return BindingCDO->MakePropertyBindingWidget(InArgs);
	}
	else
	{
		return SNew(SSpacer);
	}
}

void UAnimGraphNode_Base::HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName)
{
	if (Binding)
	{
		Binding->HandleVariableRenamed(InBlueprint, InVariableClass, InGraph, InOldVarName, InNewVarName);
	}
}

void UAnimGraphNode_Base::HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName)
{
	if (Binding)
	{
		Binding->HandleFunctionRenamed(InBlueprint, InFunctionClass, InGraph, InOldFuncName, InNewFuncName);
	}
}

void UAnimGraphNode_Base::ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement)
{
	if (Binding)
	{
		Binding->ReplaceReferences(InBlueprint, InReplacementBlueprint, InSource, InReplacement);
	}
}

bool UAnimGraphNode_Base::ReferencesVariable(const FName& InVarName, const UStruct* InScope) const
{
	if (Binding)
	{
		return Binding->ReferencesVariable(InVarName, InScope);
	}

	return false;
}

bool UAnimGraphNode_Base::ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const
{
	if (Binding && Binding->ReferencesFunction(InFunctionName, InScope))
	{
		return true;
	}

	// Check private member anim node function binding 
	return InitialUpdateFunction.GetMemberName() == InFunctionName || BecomeRelevantFunction.GetMemberName() == InFunctionName || UpdateFunction.GetMemberName() == InFunctionName;
}

void UAnimGraphNode_Base::PostEditRefreshDebuggedComponent()
{
	if(FAnimNode_Base* DebuggedAnimNode = GetDebuggedAnimNode())
	{
		if(const IAnimClassInterface* AnimClassInterface = DebuggedAnimNode->GetAnimClassInterface())
		{
			if(const UAnimInstance* HostingInstance = Cast<UAnimInstance>(IAnimClassInterface::GetObjectPtrFromAnimNode(AnimClassInterface, DebuggedAnimNode)))
			{
				if(UWorld* World = HostingInstance->GetWorld())
				{
					if(World->IsPaused() || World->WorldType == EWorldType::Editor)
					{
						if(USkeletalMeshComponent* SkelMeshComponent = HostingInstance->GetSkelMeshComponent())
						{
							SkelMeshComponent->RefreshBoneTransforms();
						}
					}
				}
			}
		}
	}
}

UObject* UAnimGraphNode_Base::GetObjectBeingDebugged() const
{
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this))
	{
		return Blueprint->GetObjectBeingDebugged();
	}

	return nullptr;
}

bool UAnimGraphNode_Base::IsPotentiallyBoundFunction(const FMemberReference& FunctionReference)
{
	return FunctionReference.GetMemberGuid().IsValid() || FunctionReference.GetMemberName() != NAME_None;
}

#undef LOCTEXT_NAMESPACE
