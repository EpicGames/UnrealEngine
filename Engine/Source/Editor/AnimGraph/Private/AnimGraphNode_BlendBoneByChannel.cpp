// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendBoneByChannel.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_BlendBoneByChannel

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendBoneByChannel)

#define LOCTEXT_NAMESPACE "AnimGraphNode_BlendBoneByChannel"

UAnimGraphNode_BlendBoneByChannel::UAnimGraphNode_BlendBoneByChannel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UAnimGraphNode_BlendBoneByChannel::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

FLinearColor UAnimGraphNode_BlendBoneByChannel::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.75f);
}

FText UAnimGraphNode_BlendBoneByChannel::GetTooltipText() const
{
	return LOCTEXT("BlendBoneByChannelTooltip", "Blend bones by channel from two poses together.");
}

FText UAnimGraphNode_BlendBoneByChannel::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("BlendBoneByChannel", "Blend Bone By Channel");
}

#undef LOCTEXT_NAMESPACE

