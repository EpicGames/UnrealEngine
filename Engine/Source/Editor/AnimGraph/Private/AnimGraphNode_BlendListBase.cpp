// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendListBase.h"
#include "AnimNodes/AnimNode_BlendListBase.h"
#include "Animation/AnimNode_Inertialization.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_BlendListBase

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendListBase)

UAnimGraphNode_BlendListBase::UAnimGraphNode_BlendListBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RemovedPinArrayIndex(INDEX_NONE)
{
}

void UAnimGraphNode_BlendListBase::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		FAnimNode_BlendListBase* Node = static_cast<FAnimNode_BlendListBase*>(GetFNode());
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node->BlendType = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		FAnimNode_BlendListBase* Node = static_cast<FAnimNode_BlendListBase*>(GetFNode());

		if (Node != nullptr)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Handle deprecation
			if (Node->bResetChildOnActivation_DEPRECATED)
			{
				Node->ChildUpateMode = EBlendListChildUpdateMode::ResetChildOnActivate;
				Node->bResetChildOnActivation_DEPRECATED = false;
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

FLinearColor UAnimGraphNode_BlendListBase::GetNodeTitleColor() const
{
	return FLinearColor(0.2f, 0.8f, 0.2f);
}

void UAnimGraphNode_BlendListBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	if ((PropertyName == TEXT("Node")))
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FString UAnimGraphNode_BlendListBase::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

void UAnimGraphNode_BlendListBase::RemovePinsFromOldPins(TArray<UEdGraphPin*>& OldPins, int32 RemovedArrayIndex)
{
	TArray<FString> RemovedPropertyNames;
	TArray<FName> NewPinNames;

	// Store new pin names to compare with old pin names
	for (int32 NewPinIndx = 0; NewPinIndx < Pins.Num(); NewPinIndx++)
	{
		NewPinNames.Add(Pins[NewPinIndx]->PinName);
	}

	// don't know which pins are removed yet so find removed pins comparing NewPins and OldPins
	for (int32 OldPinIdx = 0; OldPinIdx < OldPins.Num(); OldPinIdx++)
	{
		const FName OldPinName = OldPins[OldPinIdx]->PinName;
		if (!NewPinNames.Contains(OldPinName))
		{
			const FString OldPinNameStr = OldPinName.ToString();
			const int32 UnderscoreIndex = OldPinNameStr.Find(TEXT("_"), ESearchCase::CaseSensitive);
			if (UnderscoreIndex != INDEX_NONE)
			{
				FString PropertyName = OldPinNameStr.Left(UnderscoreIndex);
				RemovedPropertyNames.Add(MoveTemp(PropertyName));
			}
		}
	}

	for (int32 PinIdx = 0; PinIdx < OldPins.Num(); PinIdx++)
	{
		// Separate the pin name into property name and index
		const FString OldPinNameStr = OldPins[PinIdx]->PinName.ToString();
		const int32 UnderscoreIndex = OldPinNameStr.Find(TEXT("_"), ESearchCase::CaseSensitive);

		if (UnderscoreIndex != INDEX_NONE)
		{
			const FString PropertyName = OldPinNameStr.Left(UnderscoreIndex);

			if (RemovedPropertyNames.Contains(PropertyName))
			{
				const int32 ArrayIndex = FCString::Atoi(*(OldPinNameStr.Mid(UnderscoreIndex + 1)));

				// if array index is matched, removes pins 
				// and if array index is greater than removed index, decrease index
				if (ArrayIndex == RemovedArrayIndex)
				{
					OldPins[PinIdx]->MarkAsGarbage();
					OldPins.RemoveAt(PinIdx);
					--PinIdx;
				}
				else if (ArrayIndex > RemovedArrayIndex)
				{
					OldPins[PinIdx]->PinName = *FString::Printf(TEXT("%s_%d"), *PropertyName, ArrayIndex - 1);
				}
			}
		}
	}
}

void UAnimGraphNode_BlendListBase::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	// Delete Pins by removed pin info 
	if (RemovedPinArrayIndex != INDEX_NONE)
	{
		RemovePinsFromOldPins(OldPins, RemovedPinArrayIndex);
		// Clears removed pin info to avoid to remove multiple times
		// @TODO : Considering receiving RemovedPinArrayIndex as an argument of ReconstructNode()
		RemovedPinArrayIndex = INDEX_NONE;
	}
}

void UAnimGraphNode_BlendListBase::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if(GetFNodeProperty()->Struct->IsChildOf(FAnimNode_BlendListBase::StaticStruct()))
	{
		const FAnimNode_BlendListBase* BaseNode = GetFNodeProperty()->ContainerPtrToValuePtr<FAnimNode_BlendListBase>(this);

		if(BaseNode->GetTransitionType() == EBlendListTransitionType::Inertialization)
		{
			OutAttributes.Add(UE::Anim::IInertializationRequester::Attribute);
		}
	}
}
