// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendListByBool.h"


/////////////////////////////////////////////////////
// UAnimGraphNode_BlendListByBool

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendListByBool)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_BlendListByBool::UAnimGraphNode_BlendListByBool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Node.AddPose();
	Node.AddPose();
}

FText UAnimGraphNode_BlendListByBool::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UAnimGraphNode_BlendListByBool_Tooltip", "Blend Poses by bool");
}

FText UAnimGraphNode_BlendListByBool::GetTooltipText() const
{
	return LOCTEXT("UAnimGraphNode_BlendListByBool_Title", "Blend List (by bool)");
}

void UAnimGraphNode_BlendListByBool::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const 
{
	FName BlendPoses(TEXT("BlendPose"));
	FName BlendTimes(TEXT("BlendTime"));

	if (ArrayIndex != INDEX_NONE)
	{
		// Note: This is intentionally flipped, as it looks better with true as the topmost element!
		FFormatNamedArguments Args;
		Args.Add(TEXT("TrueFalse"), (ArrayIndex == 0) ? LOCTEXT("True", "True") : LOCTEXT("False", "False"));

		if (SourcePropertyName == BlendPoses)
		{
			Pin->PinFriendlyName = FText::Format(LOCTEXT("BoolPoseFriendlyName", "{TrueFalse} Pose"), Args);
		}
		else if (SourcePropertyName == BlendTimes)
		{
			Pin->PinFriendlyName = FText::Format(LOCTEXT("BoolBlendTimeFriendlyName", "{TrueFalse} Blend Time"), Args);
		}
	}
}

#undef LOCTEXT_NAMESPACE
