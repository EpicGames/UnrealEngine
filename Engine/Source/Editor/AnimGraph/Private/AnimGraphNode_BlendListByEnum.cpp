// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendListByEnum.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "ToolMenus.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "ScopedTransaction.h"
#include "Kismet2/CompilerResultsLog.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintEditorSettings.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "AnimGraphSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendListByEnum)

#define LOCTEXT_NAMESPACE "BlendListByEnum"

/////////////////////////////////////////////////////
// UAnimGraphNode_BlendListByEnum

TArray<TSharedPtr<FString>> UAnimGraphNode_BlendListByEnum::AvailableEnums;

UAnimGraphNode_BlendListByEnum::UAnimGraphNode_BlendListByEnum(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Make sure we start out with a pin
	Node.AddPose();
}

FText UAnimGraphNode_BlendListByEnum::GetMenuCategory() const
{
	return LOCTEXT("AnimGraphNode_BlendListByEnum_GetMenuCategory", "Animation|Blends");
}

FText UAnimGraphNode_BlendListByEnum::GetTooltipText() const
{
	// FText::Format() is slow, so we utilize the cached list title
	return GetNodeTitle(ENodeTitleType::ListView);
}

FText UAnimGraphNode_BlendListByEnum::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (BoundEnum == nullptr)
	{
		return LOCTEXT("AnimGraphNode_BlendListByEnum_TitleError", "Blend Poses by enum");
	}
	// @TODO: don't know enough about this node type to comfortably assert that
	//        the BoundEnum won't change after the node has spawned... until
	//        then, we'll leave this optimization off
	else //if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("EnumName"), FText::FromString(BoundEnum->GetName()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("AnimGraphNode_BlendListByEnum_Title", "Blend Poses ({EnumName})"), Args), this);
	}
	return CachedNodeTitle;
}

void UAnimGraphNode_BlendListByEnum::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	if (!GetDefault<UAnimGraphSettings>()->bShowInstancedEnumBlendAnimNodeBlueprintActions)
	{
		Super::GetMenuActions(ActionRegistrar);
		return;
	}

	struct GetMenuActions_Utils
	{
		static void SetNodeEnum(UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TWeakObjectPtr<UEnum> NonConstEnumPtr)
		{
			UAnimGraphNode_BlendListByEnum* BlendListEnumNode = CastChecked<UAnimGraphNode_BlendListByEnum>(NewNode);
			BlendListEnumNode->BoundEnum = NonConstEnumPtr.Get();
		}
	};

	UClass* NodeClass = GetClass();
	// add all blendlist enum entries
	ActionRegistrar.RegisterEnumActions( FBlueprintActionDatabaseRegistrar::FMakeEnumSpawnerDelegate::CreateLambda([NodeClass](const UEnum* Enum)->UBlueprintNodeSpawner*
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(NodeClass);
		check(NodeSpawner != nullptr);
		TWeakObjectPtr<UEnum> NonConstEnumPtr = MakeWeakObjectPtr(const_cast<UEnum*>(Enum));
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(GetMenuActions_Utils::SetNodeEnum, NonConstEnumPtr);

		return NodeSpawner;
	}) );
}

void UAnimGraphNode_BlendListByEnum::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging && BoundEnum)
	{
		if (Context->Pin && (Context->Pin->Direction == EGPD_Input))
		{
			int32 RawArrayIndex = 0;
			bool bIsPosePin = false;
			bool bIsTimePin = false;
			GetPinInformation(Context->Pin->PinName.ToString(), /*out*/ RawArrayIndex, /*out*/ bIsPosePin, /*out*/ bIsTimePin);

			if (bIsPosePin || bIsTimePin)
			{
				const int32 ExposedEnumIndex = RawArrayIndex - 1;

				if (ExposedEnumIndex != INDEX_NONE)
				{
					// Offer to remove this specific pin
					FUIAction Action = FUIAction( FExecuteAction::CreateUObject( const_cast<UAnimGraphNode_BlendListByEnum*>(this), &UAnimGraphNode_BlendListByEnum::RemovePinFromBlendList, const_cast<UEdGraphPin*>(Context->Pin)) );
					FToolMenuSection& Section = Menu->AddSection("RemovePose");
					Section.AddMenuEntry("RemovePose", LOCTEXT("RemovePose", "Remove Pose"), FText::GetEmpty(), FSlateIcon(), Action);
				}
			}
		}

		// Offer to add any not-currently-visible pins
		FToolMenuSection* Section = nullptr;
		const int32 MaxIndex = BoundEnum->NumEnums() - 1; // we don't want to show _MAX enum
		for (int32 Index = 0; Index < MaxIndex; ++Index)
		{
			FName ElementName = BoundEnum->GetNameByIndex(Index);
			if (!VisibleEnumEntries.Contains(ElementName))
			{
				FText PrettyElementName = BoundEnum->GetDisplayNameTextByIndex(Index);

				// Offer to add this entry
				if (!Section)
				{
					Section = &Menu->AddSection("AnimGraphNodeAddElementPin", LOCTEXT("ExposeHeader", "Add pin for element"));
				}

				FUIAction Action = FUIAction( FExecuteAction::CreateUObject( const_cast<UAnimGraphNode_BlendListByEnum*>(this), &UAnimGraphNode_BlendListByEnum::ExposeEnumElementAsPin, ElementName) );
				Section->AddMenuEntry(NAME_None, PrettyElementName, PrettyElementName, FSlateIcon(), Action);
			}
		}
	}
}

void UAnimGraphNode_BlendListByEnum::ExposeEnumElementAsPin(FName EnumElementName)
{
	if (!VisibleEnumEntries.Contains(EnumElementName))
	{
		FScopedTransaction Transaction( LOCTEXT("ExposeElement", "ExposeElement") );
		Modify();

		VisibleEnumEntries.Add(EnumElementName);

		Node.AddPose();
	
		ReconstructNode();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_BlendListByEnum::RemovePinFromBlendList(UEdGraphPin* Pin)
{
	int32 RawArrayIndex = 0;
	bool bIsPosePin = false;
	bool bIsTimePin = false;
	GetPinInformation(Pin->PinName.ToString(), /*out*/ RawArrayIndex, /*out*/ bIsPosePin, /*out*/ bIsTimePin);

	const int32 ExposedEnumIndex = (bIsPosePin || bIsTimePin) ? (RawArrayIndex - 1) : INDEX_NONE;

	if (ExposedEnumIndex != INDEX_NONE)
	{
		FScopedTransaction Transaction( LOCTEXT("RemovePin", "RemovePin") );
		Modify();

		// Record it as no longer exposed
		VisibleEnumEntries.RemoveAt(ExposedEnumIndex);

		// Remove the pose from the node
		FProperty* AssociatedProperty;
		int32 ArrayIndex;
		GetPinAssociatedProperty(GetFNodeType(), Pin, /*out*/ AssociatedProperty, /*out*/ ArrayIndex);

		ensure(ArrayIndex == (ExposedEnumIndex + 1));

		// setting up removed pins info 
		RemovedPinArrayIndex = ArrayIndex;
		Node.RemovePose(ArrayIndex);
		Pin->SetSavePinIfOrphaned(false);
		ReconstructNode();
		//@TODO: Just want to invalidate the visual representation currently
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_BlendListByEnum::GetPinInformation(const FString& InPinName, int32& Out_PinIndex, bool& Out_bIsPosePin, bool& Out_bIsTimePin)
{
	const int32 UnderscoreIndex = InPinName.Find(TEXT("_"), ESearchCase::CaseSensitive);
	if (UnderscoreIndex != INDEX_NONE)
	{
		const FString ArrayName = InPinName.Left(UnderscoreIndex);
		Out_PinIndex = FCString::Atoi(*(InPinName.Mid(UnderscoreIndex + 1)));

		Out_bIsPosePin = (ArrayName == TEXT("BlendPose"));
		Out_bIsTimePin = (ArrayName == TEXT("BlendTime"));
	}
	else
	{
		Out_bIsPosePin = false;
		Out_bIsTimePin = false;
		Out_PinIndex = INDEX_NONE;
	}
}

void UAnimGraphNode_BlendListByEnum::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	// if pin name starts with BlendPose or BlendWeight, change to enum name 
	bool bIsPosePin;
	bool bIsTimePin;
	int32 RawArrayIndex;
	GetPinInformation(Pin->PinName.ToString(), /*out*/ RawArrayIndex, /*out*/ bIsPosePin, /*out*/ bIsTimePin);
	checkSlow(RawArrayIndex == ArrayIndex);

	if (bIsPosePin || bIsTimePin)
	{
		if (RawArrayIndex > 0)
		{
			const int32 ExposedEnumPinIndex = RawArrayIndex - 1;

			// find pose index and see if it's mapped already or not
			if (VisibleEnumEntries.IsValidIndex(ExposedEnumPinIndex) && (BoundEnum != NULL))
			{
				const FName& EnumElementName = VisibleEnumEntries[ExposedEnumPinIndex];
				const int32 EnumIndex = BoundEnum->GetIndexByName(EnumElementName);
				if (EnumIndex != INDEX_NONE)
				{
					Pin->PinFriendlyName = BoundEnum->GetDisplayNameTextByIndex(EnumIndex);
				}
				else
				{
					Pin->PinFriendlyName = FText::FromName(EnumElementName);
				}
			}
			else
			{
				Pin->PinFriendlyName = LOCTEXT("InvalidIndex", "Invalid index");
			}
		}
		else if (ensure(RawArrayIndex == 0))
		{
			Pin->PinFriendlyName = LOCTEXT("Default", "Default");
		}

		// Append the pin type
		if (bIsPosePin)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("PinFriendlyName"), Pin->PinFriendlyName);
			Pin->PinFriendlyName = FText::Format(LOCTEXT("FriendlyNamePose", "{PinFriendlyName} Pose"), Args);
		}

		if (bIsTimePin)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("PinFriendlyName"), Pin->PinFriendlyName);
			Pin->PinFriendlyName = FText::Format(LOCTEXT("FriendlyNameBlendTime", "{PinFriendlyName} Blend Time"), Args);
		}
	}
}

void UAnimGraphNode_BlendListByEnum::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		if (BoundEnum != NULL)
		{
			PreloadObject(BoundEnum);
			BoundEnum->ConditionalPostLoad();

			for (auto ExposedIt = VisibleEnumEntries.CreateIterator(); ExposedIt; ++ExposedIt)
			{
				FName& EnumElementName = *ExposedIt;
				const int32 EnumIndex = BoundEnum->GetIndexByName(EnumElementName);

				if (EnumIndex != INDEX_NONE)
				{
					// This handles redirectors, we need to update the VisibleEnumEntries if the name has changed
					FName NewElementName = BoundEnum->GetNameByIndex(EnumIndex);

					if (NewElementName != EnumElementName)
					{
						EnumElementName = NewElementName;
					}
				}
			}
		}
	}
}

void UAnimGraphNode_BlendListByEnum::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if (BoundEnum == NULL)
	{
		MessageLog.Error(TEXT("@@ references an unknown enum; please select the enum to use"), this);
	}
}

void UAnimGraphNode_BlendListByEnum::PreloadRequiredAssets()
{
	PreloadObject(BoundEnum);

	Super::PreloadRequiredAssets();
}

void UAnimGraphNode_BlendListByEnum::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	if (BoundEnum != NULL)
	{
		PreloadObject(BoundEnum);
		BoundEnum->ConditionalPostLoad();

		// Zero the array out so it looks up the default value, and stat counting at index 1
		TArray<int32> EnumToPoseIndex;
		EnumToPoseIndex.Empty();
		EnumToPoseIndex.AddZeroed(BoundEnum->NumEnums());
		int32 PinIndex = 1;

		// Run thru the enum entries
		for (auto ExposedIt = VisibleEnumEntries.CreateConstIterator(); ExposedIt; ++ExposedIt)
		{
			const FName& EnumElementName = *ExposedIt;
			const int32 EnumIndex = BoundEnum->GetIndexByName(EnumElementName);

			if (EnumIndex != INDEX_NONE)
			{
				EnumToPoseIndex[EnumIndex] = PinIndex;
			}
			else
			{
				MessageLog.Error(*FString::Printf(TEXT("@@ references an unknown enum entry %s"), *EnumElementName.ToString()), this);
			}

			++PinIndex;
		}

		Node.SetEnumToPoseIndex(EnumToPoseIndex);
	}
}

void UAnimGraphNode_BlendListByEnum::ReloadEnum(class UEnum* InEnum)
{
	BoundEnum = InEnum;
	CachedNodeTitle.MarkDirty();
}

void UAnimGraphNode_BlendListByEnum::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory("Settings", LOCTEXT("SettingsCategoryName", "Settings"));

	RefreshAvailableEnums();

	TSharedPtr<FString> SelectedEnum = nullptr;
	if (BoundEnum != nullptr)
	{
		TSharedPtr<FString>* FoundEnum = AvailableEnums.FindByPredicate([this](const TSharedPtr<FString> Candidate)
		{
			return *Candidate == BoundEnum.GetPathName();
		});

		if (FoundEnum == nullptr)
		{
			ClearEnum();
		}
		else
		{
			SelectedEnum = *FoundEnum;
		}
	}

	SettingsCategory.AddCustomRow(LOCTEXT("BoundEnum", "Bound Enum"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BoundEnum", "Bound Enum"))
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
		]
		.ValueContent()
		[
			SNew(SSearchableComboBox)
			.InitiallySelectedItem(SelectedEnum)
			.OptionsSource(&AvailableEnums)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
				{
					return SNew(STextBlock).Text(FText::FromString(*InItem));
				})
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NewChoice, ESelectInfo::Type SelectType)
				{
					FScopedTransaction Transaction( LOCTEXT("ModifyBoundEnum", "Modify Bound Enum") );
					Modify();

					ClearEnum();

					BoundEnum = FindObject<UEnum>(nullptr, **NewChoice);
					ReconstructNode();
				})
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return FText::FromString(BoundEnum->GetFullName());
				})
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];
}

void UAnimGraphNode_BlendListByEnum::RefreshAvailableEnums()
{
	AvailableEnums.Empty();
	const FPathPermissionList& Permissions = GetMutableDefault<UBlueprintEditorSettings>()->GetEnumPermissions();

	for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
	{
		if (!EnumIt->HasAnyEnumFlags(EEnumFlags::NewerVersionExists) && UEdGraphSchema_K2::IsAllowableBlueprintVariableType(*EnumIt))
		{
			if (Permissions.HasFiltering())
			{
				TStringBuilder<256> ResultBuilder;
				EnumIt->GetPathName(nullptr, ResultBuilder);
				if (!Permissions.PassesFilter(ResultBuilder.ToView()))
				{
					return;
				}
			}

			AvailableEnums.Add(MakeShared<FString>(EnumIt->GetPathName()));
		}
	}
}

void UAnimGraphNode_BlendListByEnum::ClearEnum()
{
	for (int32 EnumPoseIndex = VisibleEnumEntries.Num() - 1; EnumPoseIndex >= 0; --EnumPoseIndex)
	{
		Node.RemovePose(EnumPoseIndex + 1);
	}
	VisibleEnumEntries.Empty();

	BoundEnum = nullptr;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
