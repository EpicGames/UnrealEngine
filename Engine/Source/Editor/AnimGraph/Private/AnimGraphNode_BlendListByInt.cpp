// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimGraphNode_BlendListByInt.h"
#include "ToolMenus.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "AnimGraphCommands.h"
#include "ScopedTransaction.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_BlendListByInt

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendListByInt)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_BlendListByInt::UAnimGraphNode_BlendListByInt(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Make sure we start out with a pin
	Node.AddPose();
}

FText UAnimGraphNode_BlendListByInt::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_BlendListByInt_Tooltip", "Blend List (by int)");
}

FText UAnimGraphNode_BlendListByInt::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_BlendListByInt_Title", "Blend Poses by int");
}

void UAnimGraphNode_BlendListByInt::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	Node.AddPose();
	ReconstructNode();
}

void UAnimGraphNode_BlendListByInt::AddPinToBlendList()
{
	FScopedTransaction Transaction( NSLOCTEXT("A3Nodes", "AddBlendListPin", "AddBlendListPin") );
	Modify();

	Node.AddPose();
	ReconstructNode();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UAnimGraphNode_BlendListByInt::RemovePinFromBlendList(UEdGraphPin* Pin)
{
	FScopedTransaction Transaction( NSLOCTEXT("A3Nodes", "RemoveBlendListPin", "RemoveBlendListPin") );
	Modify();

	FProperty* AssociatedProperty;
	int32 ArrayIndex;
	GetPinAssociatedProperty(GetFNodeType(), Pin, /*out*/ AssociatedProperty, /*out*/ ArrayIndex);

	if (ArrayIndex != INDEX_NONE)
	{
		//@TODO: ANIMREFACTOR: Need to handle moving pins below up correctly
		// setting up removed pins info
		RemovedPinArrayIndex = ArrayIndex;
		Node.RemovePose(ArrayIndex);
		// removes the selected pin and related properties in reconstructNode()
		// @TODO: Considering passing "RemovedPinArrayIndex" to ReconstructNode as the argument
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_BlendListByInt::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphBlendList", NSLOCTEXT("A3Nodes", "BlendListHeader", "BlendList"));
			if (Context->Pin != NULL)
			{
				// we only do this for normal BlendList/BlendList by enum, BlendList by Bool doesn't support add/remove pins
				if ( Context->Pin->Direction == EGPD_Input )
				{
					Section.AddMenuEntry(FAnimGraphCommands::Get().RemoveBlendListPin);
				}
			}
			else
			{
				Section.AddMenuEntry(FAnimGraphCommands::Get().AddBlendListPin);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
