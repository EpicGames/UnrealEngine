// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendSpaceBase.h"
#include "EdGraphSchema_K2_Actions.h"
#include "AnimGraphNode_BlendSpacePlayer.h"
#include "AnimGraphNode_RotationOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "Animation/AnimRootMotionProvider.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendSpaceBase)

#define LOCTEXT_NAMESPACE "AnimGraphNode_BlendSpaceBase"

/////////////////////////////////////////////////////
// UAnimGraphNode_BlendSpaceBase

UAnimGraphNode_BlendSpaceBase::UAnimGraphNode_BlendSpaceBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_BlendSpaceBase::GetNodeTitleColor() const
{
	return FLinearColor(0.2f, 0.8f, 0.2f);
}

FSlateIcon UAnimGraphNode_BlendSpaceBase::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.BlendSpace");
}

void UAnimGraphNode_BlendSpaceBase::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	UBlendSpace * BlendSpace = GetBlendSpace();

	if (BlendSpace != NULL)
	{
		if (SourcePropertyName == TEXT("X"))
		{
			Pin->PinFriendlyName = FText::FromString(BlendSpace->GetBlendParameter(0).DisplayName);
		}
		else if (SourcePropertyName == TEXT("Y"))
		{
			Pin->PinFriendlyName = FText::FromString(BlendSpace->GetBlendParameter(1).DisplayName);
			Pin->bHidden = BlendSpace->IsA<UBlendSpace1D>() ? 1 : 0;
		}
		else if (SourcePropertyName == TEXT("Z"))
		{
			Pin->PinFriendlyName = FText::FromString(BlendSpace->GetBlendParameter(2).DisplayName);
		}
	}
}


void UAnimGraphNode_BlendSpaceBase::PreloadRequiredAssets()
{
	PreloadRequiredAssetsHelper(GetBlendSpace(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_BlendSpacePlayer, BlendSpace)));

	Super::PreloadRequiredAssets();
}

void UAnimGraphNode_BlendSpaceBase::PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const
{
	if(Pin->Direction == EGPD_Input)
	{
		UBlendSpace * BlendSpace = GetBlendSpace();

		if(BlendSpace != NULL)
		{
			if(Pin->PinName == TEXT("X"))
			{
				DisplayName = BlendSpace->GetBlendParameter(0).DisplayName;
			}
			else if(Pin->PinName == TEXT("Y"))
			{
				DisplayName = BlendSpace->GetBlendParameter(1).DisplayName;
			}
			else if(Pin->PinName == TEXT("Z"))
			{
				DisplayName = BlendSpace->GetBlendParameter(2).DisplayName;
			}
		}
	}

	Super::PostProcessPinName(Pin, DisplayName);
}

void UAnimGraphNode_BlendSpaceBase::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	Super::GetOutputLinkAttributes(OutAttributes);

	if (UE::Anim::IAnimRootMotionProvider::Get())
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_BlendSpaceBase::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const UBlendSpace* BlendSpace = GetBlendSpace();
	if (BlendSpace)
	{
		TSharedRef<IPropertyHandle> XHandle = InDetailBuilder.GetProperty(TEXT("Node.X"), GetClass());
		XHandle->SetPropertyDisplayName(FText::FromString(BlendSpace->GetBlendParameter(0).DisplayName));
		TSharedRef<IPropertyHandle> YHandle = InDetailBuilder.GetProperty(TEXT("Node.Y"), GetClass());
		if (BlendSpace->IsA<UBlendSpace1D>())
		{
			InDetailBuilder.HideProperty(YHandle);
		}
		else
		{
			YHandle->SetPropertyDisplayName(FText::FromString(BlendSpace->GetBlendParameter(1).DisplayName));
		}
	}
}


FText UAnimGraphNode_BlendSpaceBase::GetMenuCategory() const
{
	return LOCTEXT("BlendSpaceCategory_Label", "Animation|Blend Spaces");
}

bool UAnimGraphNode_BlendSpaceBase::IsAimOffsetBlendSpace(const UClass* BlendSpaceClass)
{
	return  BlendSpaceClass->IsChildOf(UAimOffsetBlendSpace::StaticClass()) ||
		BlendSpaceClass->IsChildOf(UAimOffsetBlendSpace1D::StaticClass());
}

#undef LOCTEXT_NAMESPACE
