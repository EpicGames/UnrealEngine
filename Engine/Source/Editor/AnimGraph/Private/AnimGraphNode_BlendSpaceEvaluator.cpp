// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendSpaceEvaluator.h"
#include "ToolMenus.h"
#include "AnimGraphCommands.h"
#include "BlueprintNodeSpawner.h"
#include "Kismet2/CompilerResultsLog.h"
#include "IAnimBlueprintNodeOverrideAssetsContext.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeTemplateCache.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_BlendSpaceEvaluator

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendSpaceEvaluator)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_BlendSpaceEvaluator"

UAnimGraphNode_BlendSpaceEvaluator::UAnimGraphNode_BlendSpaceEvaluator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_BlendSpaceEvaluator::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_BlendSpaceEvaluator, BlendSpace));
	return GetNodeTitleHelper(TitleType, BlendSpacePin, LOCTEXT("PlayerDesc", "Blendspace Evaluator"));
}

FText UAnimGraphNode_BlendSpaceEvaluator::GetTooltipText() const
{
	return LOCTEXT("BlendSpaceEvaluatorTooltip", 
		"Evaluates a BlendSpace at a specific using a specific time input rather than advancing time "
		"internally. Typically the playback position of the animation for this node will represent "
		"something other than time, like jump height. Note that events output from the sequences playing "
		"and being blended together should not be used. In addition, synchronisation of animations "
		"will potentially be discontinuous if the blend weights are updated, as the leader/follower changes.");
}


void UAnimGraphNode_BlendSpaceEvaluator::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UBlendSpace::StaticClass() },
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescFormat", "Blendspace Evaluator '{0}'"), FText::FromName(InAssetData.AssetName));
			}
			else
			{
				return LOCTEXT("MenuDesc", "Blendspace Evaluator");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescTooltipFormat", "Blendspace Evaluator\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
			}
			else
			{
				return LOCTEXT("MenuDescTooltip", "Blendspace Evaluator");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_AssetPlayerBase::SetupNewNode(InNewNode, bInIsTemplateNode, InAssetData);
		});	
}

void UAnimGraphNode_BlendSpaceEvaluator::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	ValidateAnimNodeDuringCompilationHelper(ForSkeleton, MessageLog, Node.GetBlendSpace(), UBlendSpace::StaticClass(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_BlendSpaceEvaluator, BlendSpace)), GET_MEMBER_NAME_CHECKED(FAnimNode_BlendSpaceEvaluator, BlendSpace));
}

void UAnimGraphNode_BlendSpaceEvaluator::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	AnimBlueprint->FindOrAddGroup(Node.GetGroupName());
}

void UAnimGraphNode_BlendSpaceEvaluator::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to single frame
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeBlendSpacePlayer", LOCTEXT("BlendSpaceHeading", "Blend Space"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().OpenRelatedAsset);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToBSPlayer);
		}
	}
}

void UAnimGraphNode_BlendSpaceEvaluator::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Asset))
	{
		Node.SetBlendSpace(BlendSpace);
	}
}

void UAnimGraphNode_BlendSpaceEvaluator::CopySettingsFromAnimationAsset(UAnimationAsset* Asset)
{
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Asset))
	{
		Node.SetLoop(BlendSpace->bLoop);
	}
}

void UAnimGraphNode_BlendSpaceEvaluator::OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const
{
	if(InContext.GetAssets().Num() > 0)
	{
		if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(InContext.GetAssets()[0]))
		{
			FAnimNode_BlendSpaceEvaluator& AnimNode = InContext.GetAnimNode<FAnimNode_BlendSpaceEvaluator>();
			AnimNode.SetBlendSpace(BlendSpace);
		}
	}
}

bool UAnimGraphNode_BlendSpaceEvaluator::DoesSupportTimeForTransitionGetter() const
{
	return true;
}

UAnimationAsset* UAnimGraphNode_BlendSpaceEvaluator::GetAnimationAsset() const 
{
	UBlendSpace* BlendSpace = Node.GetBlendSpace();
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_BlendSpaceEvaluator, BlendSpace));
	if (BlendSpacePin != nullptr && BlendSpace == nullptr)
	{
		BlendSpace = Cast<UBlendSpace>(BlendSpacePin->DefaultObject);
	}

	return BlendSpace;
}

const TCHAR* UAnimGraphNode_BlendSpaceEvaluator::GetTimePropertyName() const 
{
	return TEXT("InternalTimeAccumulator");
}

UScriptStruct* UAnimGraphNode_BlendSpaceEvaluator::GetTimePropertyStruct() const 
{
	return FAnimNode_BlendSpaceEvaluator::StaticStruct();
}

EAnimAssetHandlerType UAnimGraphNode_BlendSpaceEvaluator::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UBlendSpace::StaticClass()) && !IsAimOffsetBlendSpace(AssetClass))
	{
		return EAnimAssetHandlerType::Supported;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

#undef LOCTEXT_NAMESPACE
