// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendSpaceGraph.h"

#include "AnimGraphNode_AssetPlayerBase.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendSpaceGraph)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_BlendSpaceGraph"

FText UAnimGraphNode_BlendSpaceGraph::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if(BlendSpaceGraph || BlendSpace)
	{
		const FText BlendSpaceName = FText::FromString(BlendSpaceGraph ? GetBlendSpaceGraphName() : GetBlendSpaceName());

		if(TitleType == ENodeTitleType::EditableTitle)
		{
			return BlendSpaceName;
		}
		else if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("BlendSpaceName"), BlendSpaceName);
			return FText::Format(LOCTEXT("BlendspaceListTitle", "Blendspace '{BlendSpaceName}'"), Args);
		}
		else
		{
			FFormatNamedArguments TitleArgs;
			TitleArgs.Add(TEXT("BlendSpaceName"), BlendSpaceName);
			FText Title = FText::Format(LOCTEXT("BlendSpaceFullTitle", "{BlendSpaceName}\nBlendspace"), TitleArgs);

			if ((TitleType == ENodeTitleType::FullTitle) && (Node.GetGroupName() != NAME_None))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Title"), Title);
				Args.Add(TEXT("SyncGroupName"), FText::FromName(Node.GetGroupName()));
				Title = FText::Format(LOCTEXT("BlendSpaceNodeGroupSubtitle", "{Title}\nSync group {SyncGroupName}"), Args);
			}

			return Title;
		}
	}
	else if(BlendSpaceClass.Get())
	{
		return BlendSpaceClass.Get()->GetDisplayNameText();
	}
	else
	{
		return LOCTEXT("EmptyBlendspaceListTitle", "Blendspace");
	}
}

void UAnimGraphNode_BlendSpaceGraph::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	UAnimGraphNode_AssetPlayerBase::GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UBlendSpace::StaticClass(), UBlendSpace1D::StaticClass() },
		{ UAimOffsetBlendSpace::StaticClass(), UAimOffsetBlendSpace1D::StaticClass() },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescFormat", "Blendspace '{0}'"), FText::FromName(InAssetData.AssetName));
			}
			else if(InClass != nullptr)
			{
				return InClass->GetDisplayNameText();
			}
			else
			{
				return LOCTEXT("MenuDesc", "Blendspace");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescTooltipFormat", "Blendspace\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
			}
			else if(InClass != nullptr)
			{
				return InClass->GetDisplayNameText();
			}	
			else
			{
				return LOCTEXT("MenuDesc", "Blendspace");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_BlendSpaceGraph* GraphNode = CastChecked<UAnimGraphNode_BlendSpaceGraph>(InNewNode);
			GraphNode->SetupFromAsset(InAssetData, bInIsTemplateNode);
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const TSubclassOf<UObject> InClass)
		{
			UAnimGraphNode_BlendSpaceGraph* GraphNode = CastChecked<UAnimGraphNode_BlendSpaceGraph>(InNewNode);
			GraphNode->SetupFromClass(InClass.Get(), bInIsTemplateNode);
		});
}

void UAnimGraphNode_BlendSpaceGraph::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	AnimBlueprint->FindOrAddGroup(Node.GetGroupName());
}

#undef LOCTEXT_NAMESPACE
