// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "GraphEditorActions.h"
#include "Kismet2/CompilerResultsLog.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "Animation/AnimSync.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"
#include "AnimationGraph.h"
#include "AnimationGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlendSpaceGraph.h"
#include "BlueprintEditorModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "EdGraphUtilities.h"
#include "IAnimBlueprintCompilationContext.h"
#include "AnimationBlendSpaceSampleGraph.h"
#include "AnimBlueprintExtension.h"
#include "AnimGraphNode_BlendSpaceSampleResult.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "AnimNodes/AnimNode_BlendSpaceGraphBase.h"
#include "BlueprintEditor.h"
#include "Animation/AnimSequence.h"
#include "AnimBlueprintExtension_BlendSpaceGraph.h"
#include "AnimGraphNode_RandomPlayer.h"
#include "BlueprintNodeTemplateCache.h"
#include "Animation/AnimLayerInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendSpaceGraphBase)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_BlendSpaceGraphBase"

UAnimGraphNode_BlendSpaceGraphBase::UAnimGraphNode_BlendSpaceGraphBase()
{
	bCanRenameNode = true;
}

FText UAnimGraphNode_BlendSpaceGraphBase::GetMenuCategory() const
{
	return LOCTEXT("BlendSpaceCategory_Label", "Animation|Blend Spaces");
}

FLinearColor UAnimGraphNode_BlendSpaceGraphBase::GetNodeTitleColor() const
{
	return FLinearColor(0.8f, 0.8f, 0.8f);
}

FSlateIcon UAnimGraphNode_BlendSpaceGraphBase::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.BlendSpace");
}

FText UAnimGraphNode_BlendSpaceGraphBase::GetTooltipText() const
{
	bool const bIsTemplateNode = GetGraph() == nullptr || FBlueprintNodeTemplateCache::IsTemplateOuter(GetGraph());
	if(bIsTemplateNode)
	{
		return FText::GetEmpty();
	}
	else
	{
		return GetNodeTitle(ENodeTitleType::ListView);
	}
}

UAnimGraphNode_Base* UAnimGraphNode_BlendSpaceGraphBase::ExpandGraphAndProcessNodes(UEdGraph* SourceGraph, UAnimGraphNode_Base* SourceRootNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	// Clone the nodes from the source graph
	// Note that we outer this graph to the ConsolidatedEventGraph to allow ExpansionStep to 
	// correctly retrieve the context for any expanded function calls (custom make/break structs etc.)
	UEdGraph* ClonedGraph = FEdGraphUtilities::CloneGraph(SourceGraph, InCompilationContext.GetConsolidatedEventGraph(), &InCompilationContext.GetMessageLog(), true);

	// Grab all the animation nodes and find the corresponding root node in the cloned set
	UAnimGraphNode_Base* TargetRootNode = nullptr;
	TArray<UAnimGraphNode_Base*> AnimNodeList;

	for (auto NodeIt = ClonedGraph->Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphNode* ClonedNode = *NodeIt;

		if (UAnimGraphNode_Base* TestNode = Cast<UAnimGraphNode_Base>(ClonedNode))
		{
			AnimNodeList.Add(TestNode);

			//@TODO: There ought to be a better way to determine this
			if (InCompilationContext.GetMessageLog().FindSourceObject(TestNode) == InCompilationContext.GetMessageLog().FindSourceObject(SourceRootNode))
			{
				TargetRootNode = TestNode;
			}
		}
	}
	check(TargetRootNode);

	// Run another expansion pass to catch the graph we just added (this is slightly wasteful)
	InCompilationContext.ExpansionStep(ClonedGraph, false);

	// Validate graph now we have expanded/pruned
	InCompilationContext.ValidateGraphIsWellFormed(ClonedGraph);

	// Move the cloned nodes into the consolidated event graph
	const bool bIsLoading = InCompilationContext.GetBlueprint()->bIsRegeneratingOnLoad || IsAsyncLoading();
	const bool bIsCompiling = InCompilationContext.GetBlueprint()->bBeingCompiled;
	ClonedGraph->MoveNodesToAnotherGraph(InCompilationContext.GetConsolidatedEventGraph(), bIsLoading, bIsCompiling);

	// Process any animation nodes
	{
		TArray<UAnimGraphNode_Base*> RootSet;
		RootSet.Add(TargetRootNode);

		InCompilationContext.PruneIsolatedAnimationNodes(RootSet, AnimNodeList);

		InCompilationContext.ProcessAnimationNodes(AnimNodeList);
	}

	// Returns the processed cloned version of SourceRootNode
	return TargetRootNode;	
}

void UAnimGraphNode_BlendSpaceGraphBase::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	auto ProcessGraph = [this, &OutCompiledData](UEdGraph* Graph, TArray<UAnimGraphNode_AssetPlayerBase*> AssetPlayerNodes, TArray<UAnimGraphNode_RandomPlayer*> AssetRandomPlayerPlayerNodes)
	{
		FString FunctionGraphName = Graph->GetName();
		// Also make sure we do not process any empty stub graphs
		if (!FunctionGraphName.Contains(ANIM_FUNC_DECORATOR))
		{
			if (Graph->Nodes.ContainsByPredicate([this, &OutCompiledData](UEdGraphNode* Node) { return Node && Node->NodeGuid == NodeGuid; }))
			{
				for (UAnimGraphNode_AssetPlayerBase* Node : AssetPlayerNodes)
				{
					if (int32* IndexPtr = OutCompiledData.GetAnimBlueprintDebugData().NodeGuidToIndexMap.Find(Node->NodeGuid))
					{
						FGraphAssetPlayerInformation& Info = OutCompiledData.GetGraphAssetPlayerInformation().FindOrAdd(FName(*FunctionGraphName));
						Info.PlayerNodeIndices.AddUnique(*IndexPtr);
					}
				}

				for (UAnimGraphNode_RandomPlayer* Node : AssetRandomPlayerPlayerNodes)
				{
					if (int32* IndexPtr = OutCompiledData.GetAnimBlueprintDebugData().NodeGuidToIndexMap.Find(Node->NodeGuid))
					{
						FGraphAssetPlayerInformation& Info = OutCompiledData.GetGraphAssetPlayerInformation().FindOrAdd(FName(*FunctionGraphName));
						Info.PlayerNodeIndices.AddUnique(*IndexPtr);
					}
				}
			}
		}
	};
	
	FStructProperty* NodeProperty = GetFNodeProperty();
	FArrayProperty* PoseLinksProperty = CastFieldChecked<FArrayProperty>(NodeProperty->Struct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimNode_BlendSpaceGraphBase, SamplePoseLinks)));

	// Resize pose links to match graphs
	FAnimNode_BlendSpaceGraphBase* AnimNode = NodeProperty->ContainerPtrToValuePtr<FAnimNode_BlendSpaceGraphBase>(this);
	AnimNode->SamplePoseLinks.SetNum(Graphs.Num());

	TArray<UAnimGraphNode_AssetPlayerBase*> AssetPlayerNodes;
	TArray<UAnimGraphNode_RandomPlayer*> AssetRandomPlayerPlayerNodes;

	for(int32 PoseIndex = 0; PoseIndex < Graphs.Num(); ++PoseIndex)
	{
		UAnimationBlendSpaceSampleGraph* SampleGraph = CastChecked<UAnimationBlendSpaceSampleGraph>(Graphs[PoseIndex]);
		UAnimGraphNode_Base* RootNode = ExpandGraphAndProcessNodes(SampleGraph, SampleGraph->ResultNode, InCompilationContext, OutCompiledData);

		InCompilationContext.AddPoseLinkMappingRecord(FPoseLinkMappingRecord::MakeFromArrayEntry(this, RootNode, PoseLinksProperty, PoseIndex));

		SampleGraph->GetNodesOfClass(AssetPlayerNodes);
		SampleGraph->GetNodesOfClass(AssetRandomPlayerPlayerNodes);
	}

	// Append player nodes to the owning graph's list
	UBlueprint* Blueprint = GetBlueprint();
	for (UEdGraph* FunctionGraph : Blueprint->FunctionGraphs)
	{
		ProcessGraph(FunctionGraph, AssetPlayerNodes, AssetRandomPlayerPlayerNodes);
	}

	// Now do the same for implemented layer interfaces
	for (FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
	{
		// Only process Anim Layer interfaces
		if (InterfaceDesc.Interface->IsChildOf<UAnimLayerInterface>())
		{
			for (UEdGraph* Graph : InterfaceDesc.Graphs)
			{
				ProcessGraph(Graph, AssetPlayerNodes, AssetRandomPlayerPlayerNodes);
			}
		}
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	FAnimNode_BlendSpaceGraphBase* DestinationNode = reinterpret_cast<FAnimNode_BlendSpaceGraphBase*>(InPerNodeContext.GetDestinationPtr());
	UAnimBlueprintExtension_BlendSpaceGraph* Extension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_BlendSpaceGraph>(GetAnimBlueprint());
	DestinationNode->BlendSpace = Extension->AddBlendSpace(BlendSpace);
}

void UAnimGraphNode_BlendSpaceGraphBase::SetupFromAsset(const FAssetData& InAssetData, bool bInIsTemplateNode)
{
	if(InAssetData.IsValid())
	{
		InAssetData.GetTagValue("Skeleton", SkeletonName);
		if(SkeletonName == TEXT("None"))
		{
			SkeletonName.Empty();
		}
		
		if(!bInIsTemplateNode)
		{
			UBlendSpace* Asset = CastChecked<UBlendSpace>(InAssetData.GetAsset());
			
			UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
			const UAnimationGraphSchema* AnimationGraphSchema = GetDefault<UAnimationGraphSchema>();

			BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(FBlueprintEditorUtils::CreateNewGraph(this, Asset->GetFName(), UBlendSpaceGraph::StaticClass(), UEdGraphSchema::StaticClass()));
			BlendSpaceGraph->BlendSpace = BlendSpace = DuplicateObject(Asset, BlendSpaceGraph);
			BlendSpaceGraph->BlendSpace->ClearFlags(RF_Public | RF_Standalone | RF_Transient);
			BlendSpaceGraph->BlendSpace->SetFlags(RF_Transactional);
			BlendSpaceGraph->BlendSpace->SetSkeleton(nullptr);
			BlendSpaceGraph->BlendSpace->EmptyMetaData();
			BlendSpaceGraph->BlendSpace->RemoveUserDataOfClass(UAssetUserData::StaticClass());
			BlendSpaceGraph->bAllowDeletion = false;
			BlendSpaceGraph->bAllowRenaming = true;

			// Add the new graph as a child of our parent graph
			UEdGraph* ParentGraph = GetGraph();
		
			if(ParentGraph->SubGraphs.Find(BlendSpaceGraph) == INDEX_NONE)
			{
				ParentGraph->Modify();
				ParentGraph->SubGraphs.Add(BlendSpaceGraph);
			}

			for(int32 SampleIndex = 0; SampleIndex < BlendSpace->SampleData.Num(); ++SampleIndex)
			{
				FBlendSample& Sample = BlendSpace->SampleData[SampleIndex];

				FName SampleName = Sample.Animation != nullptr ? Sample.Animation->GetFName() : FName("Sample", SampleIndex);

				UAnimationBlendSpaceSampleGraph* SampleGraph = AddGraph(SampleName, Sample.Animation);

				// Clear the animation now we have created the point
				Sample.Animation = nullptr;
			}
		}
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::SetupFromClass(TSubclassOf<UBlendSpace> InBlendSpaceClass, bool bInIsTemplateNode)
{
	if(bInIsTemplateNode)
	{
		BlendSpaceClass = InBlendSpaceClass;
	}
	else
	{
		BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(FBlueprintEditorUtils::CreateNewGraph(this, InBlendSpaceClass.Get()->GetFName(), UBlendSpaceGraph::StaticClass(), UEdGraphSchema::StaticClass()));
		BlendSpaceGraph->BlendSpace = BlendSpace = NewObject<UBlendSpace>(BlendSpaceGraph, InBlendSpaceClass.Get());
		BlendSpaceGraph->BlendSpace->ClearFlags(RF_Public | RF_Standalone | RF_Transient);
		BlendSpaceGraph->BlendSpace->SetFlags(RF_Transactional);
		BlendSpaceGraph->bAllowDeletion = false;
		BlendSpaceGraph->bAllowRenaming = true;

		// Add the new graph as a child of our parent graph
		UEdGraph* ParentGraph = GetGraph();
	
		if(ParentGraph->SubGraphs.Find(BlendSpaceGraph) == INDEX_NONE)
		{
			ParentGraph->Modify();
			ParentGraph->SubGraphs.Add(BlendSpaceGraph);
		}
	}
}

UObject* UAnimGraphNode_BlendSpaceGraphBase::GetJumpTargetForDoubleClick() const
{
	return BlendSpaceGraph;
}

void UAnimGraphNode_BlendSpaceGraphBase::JumpToDefinition() const
{
	TSharedPtr<IBlueprintEditor> BlueprintEditor = FKismetEditorUtilities::GetIBlueprintEditorForObject(BlendSpace, true);
	if(BlueprintEditor.IsValid())
	{
		BlueprintEditor->JumpToHyperlink(BlendSpaceGraph, false);
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	if (BlendSpace)
	{
		IDetailCategoryBuilder& BlendSpaceCategory = InDetailBuilder.EditCategory("Blend Space");
		IDetailPropertyRow* BlendSpaceRow = BlendSpaceCategory.AddExternalObjects( { BlendSpace } );
		BlendSpaceRow->ShouldAutoExpand(true);

		TSharedRef<IPropertyHandle> XHandle = InDetailBuilder.GetProperty(TEXT("Node.X"), GetClass());
		XHandle->SetPropertyDisplayName(FText::FromString(BlendSpace->GetBlendParameter(0).DisplayName));
		TSharedRef<IPropertyHandle> YHandle = InDetailBuilder.GetProperty(TEXT("Node.Y"), GetClass());
		if (BlendSpace->IsA<UBlendSpace1D>())
		{
			InDetailBuilder.HideProperty(YHandle);
		}
		else
		{
			YHandle->SetPropertyDisplayName(FText::FromString(BlendSpace->GetBlendParameter(1).DisplayName));
		}
	}
}

TArray<UEdGraph*> UAnimGraphNode_BlendSpaceGraphBase::GetSubGraphs() const
{
	return TArray<UEdGraph*>( { BlendSpaceGraph } );
}

void UAnimGraphNode_BlendSpaceGraphBase::DestroyNode()
{
	UBlueprint* Blueprint = GetBlueprint();
	for (UEdGraph* SampleGraph : Graphs)
	{
		SampleGraph->Modify();
		FBlueprintEditorUtils::RemoveGraph(Blueprint, SampleGraph);
	}

	UEdGraph* GraphToRemove = BlendSpaceGraph;
	BlendSpaceGraph = nullptr;
	Graphs.Empty();

	Super::DestroyNode();

	if (GraphToRemove)
	{
		GraphToRemove->Modify();
		FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphToRemove, EGraphRemoveFlags::Recompile);
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::PreloadRequiredAssets()
{
	PreloadObject(BlendSpace);

	Super::PreloadRequiredAssets();
}

TSharedPtr<INameValidatorInterface> UAnimGraphNode_BlendSpaceGraphBase::MakeNameValidator() const
{
	class FNameValidator : public FStringSetNameValidator
	{
	public:
		FNameValidator(const UAnimGraphNode_BlendSpaceGraphBase* InBlendSpaceGraphNode)
			: FStringSetNameValidator(FString())
		{
			TArray<UAnimGraphNode_BlendSpaceGraphBase*> Nodes;

			UAnimationGraph* AnimGraph = CastChecked<UAnimationGraph>(InBlendSpaceGraphNode->GetOuter());
			AnimGraph->GetNodesOfClass<UAnimGraphNode_BlendSpaceGraphBase>(Nodes);

			for (UAnimGraphNode_BlendSpaceGraphBase* Node : Nodes)
			{
				if (Node != InBlendSpaceGraphNode)
				{
					Names.Add(Node->GetBlendSpaceGraphName());
				}
			}
		}
	};

	return MakeShared<FNameValidator>(this);
}

void UAnimGraphNode_BlendSpaceGraphBase::OnRenameNode(const FString& NewName)
{
	FBlueprintEditorUtils::RenameGraph(BlendSpaceGraph, NewName);
}

FString UAnimGraphNode_BlendSpaceGraphBase::GetBlendSpaceGraphName() const
{
	return (BlendSpaceGraph != nullptr) ? *(BlendSpaceGraph->GetName()) : TEXT("(null)");
}

FString UAnimGraphNode_BlendSpaceGraphBase::GetBlendSpaceName() const
{
	return (BlendSpace != nullptr) ? *(BlendSpace->GetName()) : TEXT("(null)");
}

void UAnimGraphNode_BlendSpaceGraphBase::PostPasteNode()
{
	Super::PostPasteNode();

	if(BlendSpaceGraph != nullptr)
	{
		// Add the new graph as a child of our parent graph
		UEdGraph* ParentGraph = GetGraph();

		if(ParentGraph->SubGraphs.Find(BlendSpaceGraph) == INDEX_NONE)
		{
			ParentGraph->SubGraphs.Add(BlendSpaceGraph);
		}

		// Refresh sample graphs
		for (UEdGraph* SampleGraph : Graphs)
		{
			for(UEdGraphNode* GraphNode : SampleGraph->Nodes)
			{
				GraphNode->CreateNewGuid();
				GraphNode->PostPasteNode();
				GraphNode->ReconstructNode();
			}
		}

		// Find an interesting name
		TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
		FBlueprintEditorUtils::RenameGraphWithSuggestion(BlendSpaceGraph, NameValidator, BlendSpaceGraph->GetName());

		// Restore transactional flag that is lost during copy/paste process
		BlendSpaceGraph->SetFlags(RF_Transactional);
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	
	// Create a new graph & blendspace if we havent been set up already
	if(BlendSpaceGraph == nullptr)
	{
		check(BlendSpace == nullptr);

		BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(FBlueprintEditorUtils::CreateNewGraph(this, NAME_None, UBlendSpaceGraph::StaticClass(), UEdGraphSchema::StaticClass()));
		BlendSpaceGraph->BlendSpace = BlendSpace = NewObject<UBlendSpace>();
		
		// Find an interesting name
		TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
		FBlueprintEditorUtils::RenameGraphWithSuggestion(BlendSpaceGraph, NameValidator, TEXT("New Blend Space"));

		// Add the new graph as a child of our parent graph
		UEdGraph* ParentGraph = GetGraph();
	
		if(ParentGraph->SubGraphs.Find(BlendSpaceGraph) == INDEX_NONE)
		{
			ParentGraph->Modify();
			ParentGraph->SubGraphs.Add(BlendSpaceGraph);
		}
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	if (BlendSpace != nullptr)
	{
		if (SourcePropertyName == TEXT("X"))
		{
			Pin->PinFriendlyName = FText::FromString(BlendSpace->GetBlendParameter(0).DisplayName);
		}
		else if (SourcePropertyName == TEXT("Y"))
		{
			Pin->PinFriendlyName = FText::FromString(BlendSpace->GetBlendParameter(1).DisplayName);
			Pin->bHidden = BlendSpace->IsA<UBlendSpace1D>() ? 1 : 0;
		}
		else if (SourcePropertyName == TEXT("Z"))
		{
			Pin->PinFriendlyName = FText::FromString(BlendSpace->GetBlendParameter(2).DisplayName);
		}
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const
{
	if(Pin->Direction == EGPD_Input)
	{
		if(BlendSpace != nullptr)
		{
			if(Pin->PinName == TEXT("X"))
			{
				DisplayName = BlendSpace->GetBlendParameter(0).DisplayName;
			}
			else if(Pin->PinName == TEXT("Y"))
			{
				DisplayName = BlendSpace->GetBlendParameter(1).DisplayName;
			}
			else if(Pin->PinName == TEXT("Z"))
			{
				DisplayName = BlendSpace->GetBlendParameter(2).DisplayName;
			}
		}
	}

	Super::PostProcessPinName(Pin, DisplayName);
}

int32 UAnimGraphNode_BlendSpaceGraphBase::GetSampleIndex(const UEdGraph* Graph) const
{
	for (int32 Index = 0 ; Index != Graphs.Num() ; ++Index)
	{
		if (Graphs[Index] == Graph)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

UAnimationBlendSpaceSampleGraph* UAnimGraphNode_BlendSpaceGraphBase::AddGraph(FName InSampleName, UAnimSequence* InSequence)
{
	UAnimationBlendSpaceSampleGraph* NewGraph = AddGraphInternal(InSampleName, InSequence);

	Graphs.Add(NewGraph);

	return NewGraph;
}

UAnimationBlendSpaceSampleGraph* UAnimGraphNode_BlendSpaceGraphBase::AddGraphInternal(FName InSampleName, UAnimSequence* InSequence)
{
	const UAnimationGraphSchema* AnimationGraphSchema = GetDefault<UAnimationGraphSchema>();

	Modify();

	const FName NewGraphName = FBlueprintEditorUtils::GenerateUniqueGraphName(BlendSpaceGraph, InSampleName.ToString());
	UAnimationBlendSpaceSampleGraph* SampleGraph = CastChecked<UAnimationBlendSpaceSampleGraph>(FBlueprintEditorUtils::CreateNewGraph(BlendSpaceGraph, NewGraphName, UAnimationBlendSpaceSampleGraph::StaticClass(), UAnimationGraphSchema::StaticClass()));

	FGraphNodeCreator<UAnimGraphNode_BlendSpaceSampleResult> ResultNodeCreator(*SampleGraph);
	UAnimGraphNode_BlendSpaceSampleResult* ResultSinkNode = ResultNodeCreator.CreateNode();
	ResultNodeCreator.Finalize();
	AnimationGraphSchema->SetNodeMetaData(ResultSinkNode, FNodeMetadata::DefaultGraphNode);
	SampleGraph->ResultNode = ResultSinkNode;
	SampleGraph->bAllowDeletion = false;
	SampleGraph->bAllowRenaming = true;

	if(InSequence != nullptr)
	{
		// Attach an asset player if a valid animation is supplied
		FGraphNodeCreator<UAnimGraphNode_SequencePlayer> SequencePlayerNodeCreator(*SampleGraph);
		UAnimGraphNode_SequencePlayer* SequencePlayer = SequencePlayerNodeCreator.CreateNode();
		SequencePlayer->SetAnimationAsset(InSequence);
		SequencePlayer->Node.SetGroupMethod(EAnimSyncMethod::Graph);
		SequencePlayerNodeCreator.Finalize();

		// Offset node in X
		SequencePlayer->NodePosX = SampleGraph->ResultNode->NodePosX - 400;

		UEdGraphPin* OutputPin = SequencePlayer->FindPinChecked(TEXT("Pose"), EGPD_Output);
		UEdGraphPin* InputPin = SampleGraph->ResultNode->FindPinChecked(TEXT("Result"), EGPD_Input);
		OutputPin->MakeLinkTo(InputPin);
	}

	BlendSpaceGraph->Modify();
	BlendSpaceGraph->SubGraphs.Add(SampleGraph);

	return SampleGraph;
}

void UAnimGraphNode_BlendSpaceGraphBase::RemoveGraph(int32 InSampleIndex)
{
	Modify();

	check(Graphs.IsValidIndex(InSampleIndex));
	UEdGraph* GraphToRemove = Graphs[InSampleIndex];
	TSharedPtr<FBlueprintEditor> BlueprintEditor = StaticCastSharedPtr<FBlueprintEditor>(FKismetEditorUtilities::GetIBlueprintEditorForObject(GraphToRemove, false));
	check(BlueprintEditor.IsValid());

	// 'Asset' blendspace sample deletion uses a RemoveAtSwap, so we must mirror it here to maintain the same indices
	Graphs.RemoveAtSwap(InSampleIndex);

	FBlueprintEditorUtils::RemoveGraph(GetAnimBlueprint(), GraphToRemove);
	BlueprintEditor->CloseDocumentTab(GraphToRemove);
}

void UAnimGraphNode_BlendSpaceGraphBase::ReplaceGraph(int32 InSampleIndex, UAnimSequence* InSequence)
{
	Modify();

	check(Graphs.IsValidIndex(InSampleIndex));
	UEdGraph* GraphToRemove = Graphs[InSampleIndex];
	FName GraphName = GraphToRemove->GetFName();
	TSharedPtr<FBlueprintEditor> BlueprintEditor = StaticCastSharedPtr<FBlueprintEditor>(FKismetEditorUtilities::GetIBlueprintEditorForObject(GraphToRemove, false));
	check(BlueprintEditor.IsValid());

	FBlueprintEditorUtils::RemoveGraph(GetAnimBlueprint(), GraphToRemove);
	BlueprintEditor->CloseDocumentTab(GraphToRemove);

	Graphs[InSampleIndex] = AddGraphInternal(GraphName, InSequence);
}

FName UAnimGraphNode_BlendSpaceGraphBase::GetSyncGroupName() const
{
	FStructProperty* NodeProperty = GetFNodeProperty();
	const FAnimNode_BlendSpaceGraphBase* AnimNode = NodeProperty->ContainerPtrToValuePtr<FAnimNode_BlendSpaceGraphBase>(this);

	return AnimNode->GroupName;
}

void UAnimGraphNode_BlendSpaceGraphBase::SetSyncGroupName(FName InName)
{
	FStructProperty* NodeProperty = GetFNodeProperty();
	FAnimNode_BlendSpaceGraphBase* AnimNode = NodeProperty->ContainerPtrToValuePtr<FAnimNode_BlendSpaceGraphBase>(this);
	AnimNode->GroupName = InName;
}

void UAnimGraphNode_BlendSpaceGraphBase::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if(GetSyncGroupName() != NAME_None)
	{
		OutAttributes.Add(UE::Anim::FAnimSync::Attribute);
	}
}

void UAnimGraphNode_BlendSpaceGraphBase::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	OutExtensions.Add(UAnimBlueprintExtension_BlendSpaceGraph::StaticClass());
}

bool UAnimGraphNode_BlendSpaceGraphBase::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;

	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UBlueprint* Blueprint : FilterContext.Blueprints)
	{
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
		{
			if(!SkeletonName.IsEmpty())
			{
				if(AnimBlueprint->TargetSkeleton == nullptr || !AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(SkeletonName))
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
		else
		{
			// Not an animation Blueprint, cannot use
			bIsFilteredOut = true;
			break;
		}
	}

	return bIsFilteredOut;
}

#undef LOCTEXT_NAMESPACE
