// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendSpacePlayer.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "ToolMenus.h"
#include "GraphEditorActions.h"
#include "Kismet2/CompilerResultsLog.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "AnimGraphCommands.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "IAnimBlueprintNodeOverrideAssetsContext.h"
#include "AssetRegistry/AssetRegistryModule.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_BlendSpacePlayer

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendSpacePlayer)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_BlendSpacePlayer"

UAnimGraphNode_BlendSpacePlayer::UAnimGraphNode_BlendSpacePlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_BlendSpacePlayer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_BlendSpacePlayer, BlendSpace));
	return GetNodeTitleHelper(TitleType, BlendSpacePin, LOCTEXT("PlayerDesc", "Blendspace Player"));
}

void UAnimGraphNode_BlendSpacePlayer::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	ValidateAnimNodeDuringCompilationHelper(ForSkeleton, MessageLog, Node.GetBlendSpace(), UBlendSpace::StaticClass(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_BlendSpacePlayer, BlendSpace)), GET_MEMBER_NAME_CHECKED(FAnimNode_BlendSpacePlayer, BlendSpace));
}

void UAnimGraphNode_BlendSpacePlayer::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	AnimBlueprint->FindOrAddGroup(Node.GetGroupName());
}

void UAnimGraphNode_BlendSpacePlayer::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to single frame
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeBlendSpaceEvaluator", LOCTEXT("BlendSpaceHeading", "Blend Space"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().OpenRelatedAsset);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToBSEvaluator);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToBSGraph);
		}
	}
}

void UAnimGraphNode_BlendSpacePlayer::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UBlendSpace::StaticClass() },
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescFormat", "Blendspace Player '{0}'"), FText::FromName(InAssetData.AssetName));
			}
			else
			{
				return LOCTEXT("MenuDesc", "Blendspace Player");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescTooltipFormat", "Blendspace Player\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
			}
			else
			{
				return LOCTEXT("MenuDescTooltip", "Blendspace Player");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_AssetPlayerBase::SetupNewNode(InNewNode, bInIsTemplateNode, InAssetData);
		});
}

FBlueprintNodeSignature UAnimGraphNode_BlendSpacePlayer::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddSubObject(Node.GetBlendSpace());

	return NodeSignature;
}

void UAnimGraphNode_BlendSpacePlayer::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Asset))
	{
		Node.SetBlendSpace(BlendSpace);
	}
}

void UAnimGraphNode_BlendSpacePlayer::CopySettingsFromAnimationAsset(UAnimationAsset* Asset)
{
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Asset))
	{
		Node.SetLoop(BlendSpace->bLoop);
	}
}

void UAnimGraphNode_BlendSpacePlayer::OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const
{
	if(InContext.GetAssets().Num() > 0)
	{
		if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(InContext.GetAssets()[0]))
		{
			FAnimNode_BlendSpacePlayer& AnimNode = InContext.GetAnimNode<FAnimNode_BlendSpacePlayer>();
			AnimNode.SetBlendSpace(BlendSpace);
		}
	}
}

void UAnimGraphNode_BlendSpacePlayer::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const
{
	if(Node.GetBlendSpace())
	{
		HandleAnimReferenceCollection(Node.BlendSpace, AnimationAssets);
	}
}

void UAnimGraphNode_BlendSpacePlayer::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(Node.BlendSpace, AnimAssetReplacementMap);
}

bool UAnimGraphNode_BlendSpacePlayer::DoesSupportTimeForTransitionGetter() const
{
	return true;
}

UAnimationAsset* UAnimGraphNode_BlendSpacePlayer::GetAnimationAsset() const 
{
	UBlendSpace* BlendSpace = Node.GetBlendSpace();
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_BlendSpacePlayer, BlendSpace));
	if (BlendSpacePin != nullptr && BlendSpace == nullptr)
	{
		BlendSpace = Cast<UBlendSpace>(BlendSpacePin->DefaultObject);
	}

	return BlendSpace;
}

const TCHAR* UAnimGraphNode_BlendSpacePlayer::GetTimePropertyName() const 
{
	return TEXT("InternalTimeAccumulator");
}

UScriptStruct* UAnimGraphNode_BlendSpacePlayer::GetTimePropertyStruct() const 
{
	return FAnimNode_BlendSpacePlayer::StaticStruct();
}

EAnimAssetHandlerType UAnimGraphNode_BlendSpacePlayer::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UBlendSpace::StaticClass()) && !IsAimOffsetBlendSpace(AssetClass))
	{
		return EAnimAssetHandlerType::PrimaryHandler;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

#undef LOCTEXT_NAMESPACE

