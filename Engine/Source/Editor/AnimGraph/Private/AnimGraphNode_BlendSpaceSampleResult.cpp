// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendSpaceSampleResult.h"
#include "GraphEditorSettings.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "Animation/AnimSync.h"
#include "AnimationBlendSpaceSampleGraph.h"
#include "BlendSpaceGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendSpaceSampleResult)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_BlendSpaceSampleResult"

FLinearColor UAnimGraphNode_BlendSpaceSampleResult::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText UAnimGraphNode_BlendSpaceSampleResult::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNodeStateResult_Title", "Output Animation Pose");
}

FText UAnimGraphNode_BlendSpaceSampleResult::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNodeStateResult_Tooltip", "This is the output of this animation state");
}

void UAnimGraphNode_BlendSpaceSampleResult::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Intentionally empty. This node is auto-generated when a new graph is created.
}

void UAnimGraphNode_BlendSpaceSampleResult::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	UAnimationBlendSpaceSampleGraph* SampleGraph = CastChecked<UAnimationBlendSpaceSampleGraph>(GetGraph());
	UBlendSpaceGraph* BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(SampleGraph->GetOuter());
	UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceGraphNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());

	if(BlendSpaceGraphNode->GetSyncGroupName() != NAME_None)
	{
		OutAttributes.Add(UE::Anim::FAnimSync::Attribute);
	}
}

UAnimGraphNode_Base* UAnimGraphNode_BlendSpaceSampleResult::GetProxyNodeForAttributes() const
{
	UAnimationBlendSpaceSampleGraph* SampleGraph = CastChecked<UAnimationBlendSpaceSampleGraph>(GetGraph());
	UBlendSpaceGraph* BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(SampleGraph->GetOuter());
	UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceGraphNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());
	return BlendSpaceGraphNode;
}

#undef LOCTEXT_NAMESPACE
