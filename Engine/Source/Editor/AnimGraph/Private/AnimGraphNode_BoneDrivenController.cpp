// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BoneDrivenController.h"
#include "PrimitiveDrawingUtils.h"
#include "Components/SkeletalMeshComponent.h"
#include "Widgets/Text/STextBlock.h"
#include "Kismet2/CompilerResultsLog.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "AnimationCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BoneDrivenController)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_BoneDrivenController::UAnimGraphNode_BoneDrivenController(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer)
{

}

void UAnimGraphNode_BoneDrivenController::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAnimationCustomVersion::GUID);

	const int32 AnimVersion = Ar.CustomVer(FAnimationCustomVersion::GUID);

	if (AnimVersion < FAnimationCustomVersion::BoneDrivenControllerRemapping)
	{
		if (AnimVersion < FAnimationCustomVersion::BoneDrivenControllerMatchingMaya)
		{
			// The node used to be able to only drive a single component rather than a selection of components
			Node.ConvertTargetComponentToBits();

			// The old definition of range was clamping the output, rather than the input
			if (Node.bUseRange && !FMath::IsNearlyZero(Node.Multiplier))
			{
				// Before: Output = clamp(Input * Multipler)
				// After: Output = clamp(Input) * Multiplier
				Node.RangeMin /= Node.Multiplier;
				Node.RangeMax /= Node.Multiplier;
			}
		}

		Node.RemappedMin = Node.RangeMin;
		Node.RemappedMax = Node.RangeMax;
	}
}

FText UAnimGraphNode_BoneDrivenController::GetTooltipText() const
{
	return LOCTEXT("UAnimGraphNode_BoneDrivenController_ToolTip", "Drives the transform of a bone or morph target using the transform of another bone");
}

FText UAnimGraphNode_BoneDrivenController::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((Node.SourceBone.BoneName == NAME_None) && (Node.TargetBone.BoneName == NAME_None) && ((TitleType == ENodeTitleType::ListView) || (TitleType == ENodeTitleType::MenuTitle)))
	{
		return GetControllerDescription();
	}
	else
	{
		// Determine the mapping
		FText FinalSourceExpression;
		{
			FFormatNamedArguments SourceArgs;
			SourceArgs.Add(TEXT("SourceBone"), FText::FromName(Node.SourceBone.BoneName));
			SourceArgs.Add(TEXT("SourceComponent"), ComponentTypeToText(Node.SourceComponent));

			if (Node.DrivingCurve != nullptr)
			{
				FinalSourceExpression = LOCTEXT("BoneDrivenByCurve", "curve({SourceBone}.{SourceComponent})");
			}
			else
			{
				if (Node.bUseRange)
				{
					if (Node.Multiplier == 1.0f)
					{
						FinalSourceExpression = LOCTEXT("WithRangeBoneMultiplierIs1", "remap({SourceBone}.{SourceComponent})");
					}
					else
					{
						SourceArgs.Add(TEXT("Multiplier"), FText::AsNumber(Node.Multiplier));
						FinalSourceExpression = LOCTEXT("WithRangeNonUnityMultiplier", "remap({SourceBone}.{SourceComponent}) * {Multiplier}");
					}
				}
				else
				{
					if (Node.Multiplier == 1.0f)
					{
						FinalSourceExpression = LOCTEXT("BoneMultiplierIs1", "{SourceBone}.{SourceComponent}");
					}
					else
					{
						SourceArgs.Add(TEXT("Multiplier"), FText::AsNumber(Node.Multiplier));
						FinalSourceExpression = LOCTEXT("NonUnityMultiplier", "{SourceBone}.{SourceComponent} * {Multiplier}");
					}
				}
			}

			FinalSourceExpression = FText::Format(FinalSourceExpression, SourceArgs);
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDesc"), GetControllerDescription());

		Args.Add(TEXT("ParameterName"), FText::FromName(Node.ParameterName));

		Args.Add(TEXT("TargetBone"), FText::FromName(Node.TargetBone.BoneName));

		// Determine the target component
		int32 NumComponents = 0;
		EComponentType::Type TargetComponent = EComponentType::None;

		#define UE_CHECK_TARGET_COMPONENT(ComponentProperty, ComponentChoice) if (ComponentProperty) { ++NumComponents; TargetComponent = ComponentChoice; }
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetTranslationX, EComponentType::TranslationX);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetTranslationY, EComponentType::TranslationY);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetTranslationZ, EComponentType::TranslationZ);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetRotationX, EComponentType::RotationX);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetRotationY, EComponentType::RotationY);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetRotationZ, EComponentType::RotationZ);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetScaleX, EComponentType::ScaleX);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetScaleY, EComponentType::ScaleY);
		UE_CHECK_TARGET_COMPONENT(Node.bAffectTargetScaleZ, EComponentType::ScaleZ);
		Args.Add(TEXT("TargetComponents"), (NumComponents <= 1) ? ComponentTypeToText(TargetComponent) : LOCTEXT("MultipleTargetComponents", "multiple"));

		if ((TitleType == ENodeTitleType::ListView) || (TitleType == ENodeTitleType::MenuTitle))
		{
			Args.Add(TEXT("Delim"), FText::FromString(TEXT(" - ")));
		}
		else
		{
			Args.Add(TEXT("Delim"), FText::FromString(TEXT("\n")));
		}

		Args.Add(TEXT("SourceExpression"), FinalSourceExpression);

		FText FormattedText;
		if (Node.DestinationMode == EDrivenDestinationMode::Bone)
		{
			FormattedText = FText::Format(LOCTEXT("AnimGraphNode_BoneDrivenController_Title_Bone", "{TargetBone}.{TargetComponents} = {SourceExpression}{Delim}{ControllerDesc}"), Args);
		}
		else if ( Node.DestinationMode == EDrivenDestinationMode::MorphTarget || Node.DestinationMode == EDrivenDestinationMode::MaterialParameter)
		{
			FormattedText = FText::Format(LOCTEXT("AnimGraphNode_BoneDrivenController_Title_Curve", "{ParameterName} = {SourceExpression}{Delim}{ControllerDesc}"), Args);
		}

		return FormattedText;
	}	
}

FText UAnimGraphNode_BoneDrivenController::GetControllerDescription() const
{
	return LOCTEXT("BoneDrivenController", "Bone Driven Controller");
}

void UAnimGraphNode_BoneDrivenController::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const
{	
	static const float ArrowHeadWidth = 5.0f;
	static const float ArrowHeadHeight = 8.0f;

	const int32 SourceIdx = SkelMeshComp->GetBoneIndex(Node.SourceBone.BoneName);
	const int32 TargetIdx = SkelMeshComp->GetBoneIndex(Node.TargetBone.BoneName);

	if ((SourceIdx != INDEX_NONE) && (TargetIdx != INDEX_NONE))
	{
		const FTransform SourceTM = SkelMeshComp->GetComponentSpaceTransforms()[SourceIdx] * SkelMeshComp->GetComponentTransform();
		const FTransform TargetTM = SkelMeshComp->GetComponentSpaceTransforms()[TargetIdx] * SkelMeshComp->GetComponentTransform();

		PDI->DrawLine(TargetTM.GetLocation(), SourceTM.GetLocation(), FLinearColor(0.0f, 0.0f, 1.0f), SDPG_Foreground, 0.5f);

		const FVector ToTarget = TargetTM.GetTranslation() - SourceTM.GetTranslation();
		const FVector UnitToTarget = ToTarget.GetSafeNormal();
		FVector Midpoint = SourceTM.GetTranslation() + 0.5f * ToTarget + 0.5f * UnitToTarget * ArrowHeadHeight;

		FVector YAxis;
		FVector ZAxis;
		UnitToTarget.FindBestAxisVectors(YAxis, ZAxis);
		const FMatrix ArrowMatrix(UnitToTarget, YAxis, ZAxis, Midpoint);

		DrawConnectedArrow(PDI, ArrowMatrix, FLinearColor(0.0f, 1.0f, 0.0), ArrowHeadHeight, ArrowHeadWidth, SDPG_Foreground);

		PDI->DrawPoint(SourceTM.GetTranslation(), FLinearColor(0.8f, 0.8f, 0.2f), 5.0f, SDPG_Foreground);
		PDI->DrawPoint(SourceTM.GetTranslation() + ToTarget, FLinearColor(0.8f, 0.8f, 0.2f), 5.0f, SDPG_Foreground);
	}
}

void UAnimGraphNode_BoneDrivenController::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	if (ForSkeleton->GetReferenceSkeleton().FindBoneIndex(Node.SourceBone.BoneName) == INDEX_NONE)
	{
		MessageLog.Warning(*LOCTEXT("DriverJoint_NoSourceBone", "@@ - You must pick a source bone as the Driver joint").ToString(), this);
	}

	if (Node.SourceComponent == EComponentType::None)
	{
		MessageLog.Warning(*LOCTEXT("DriverJoint_NoSourceComponent", "@@ - You must pick a source component on the Driver joint").ToString(), this);
	}
	
	if (Node.DestinationMode == EDrivenDestinationMode::Bone)
	{
		if (ForSkeleton->GetReferenceSkeleton().FindBoneIndex(Node.TargetBone.BoneName) == INDEX_NONE)
		{
			MessageLog.Warning(*LOCTEXT("DriverJoint_NoTargetBone", "@@ - You must pick a target bone as the Driven joint").ToString(), this);
		}
		
		const bool bAffectsTranslation = Node.bAffectTargetTranslationX || Node.bAffectTargetTranslationY || Node.bAffectTargetTranslationZ;
		const bool bAffectsRotation = Node.bAffectTargetRotationX || Node.bAffectTargetRotationY || Node.bAffectTargetRotationZ;
		const bool bAffectsScale = Node.bAffectTargetScaleX || Node.bAffectTargetScaleY || Node.bAffectTargetScaleZ;

		if (!bAffectsTranslation && !bAffectsRotation && !bAffectsScale)
		{
			MessageLog.Warning(*LOCTEXT("DriverJoint_NoTargetComponent", "@@ - You must pick one or more target components on the Driven joint").ToString(), this);
		}
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}


void UAnimGraphNode_BoneDrivenController::AddTripletPropertyRow(const FText& Name, const FText& Tooltip, IDetailCategoryBuilder& Category, TSharedRef<IPropertyHandle> PropertyHandle, const FName XPropertyName, const FName YPropertyName, const FName ZPropertyName, TAttribute<EVisibility> VisibilityAttribute)
{
	const float XYZPadding = 5.0f;

	TSharedPtr<IPropertyHandle> XProperty = PropertyHandle->GetChildHandle(XPropertyName);
	Category.GetParentLayout().HideProperty(XProperty);

	TSharedPtr<IPropertyHandle> YProperty = PropertyHandle->GetChildHandle(YPropertyName);
	Category.GetParentLayout().HideProperty(YProperty);

	TSharedPtr<IPropertyHandle> ZProperty = PropertyHandle->GetChildHandle(ZPropertyName);
	Category.GetParentLayout().HideProperty(ZProperty);

	Category.AddCustomRow(Name)
	.Visibility(VisibilityAttribute)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(Name)
		.ToolTipText(Tooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0.f, 0.f, XYZPadding, 0.f)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				XProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				XProperty->CreatePropertyValueWidget()
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(0.f, 0.f, XYZPadding, 0.f)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				YProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				YProperty->CreatePropertyValueWidget()
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(0.f, 0.f, XYZPadding, 0.f)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				ZProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				ZProperty->CreatePropertyValueWidget()
			]
		]
	];
}

void UAnimGraphNode_BoneDrivenController::AddRangePropertyRow(const FText& Name, const FText& Tooltip, IDetailCategoryBuilder& Category, TSharedRef<IPropertyHandle> PropertyHandle, const FName MinPropertyName, const FName MaxPropertyName, TAttribute<EVisibility> VisibilityAttribute)
{
	const float MiddlePadding = 4.0f;

	TSharedPtr<IPropertyHandle> MinProperty = PropertyHandle->GetChildHandle(MinPropertyName);
	Category.GetParentLayout().HideProperty(MinProperty);

	TSharedPtr<IPropertyHandle> MaxProperty = PropertyHandle->GetChildHandle(MaxPropertyName);
	Category.GetParentLayout().HideProperty(MaxProperty);

	Category.AddCustomRow(Name)
	.Visibility(VisibilityAttribute)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(Name)
		.ToolTipText(Tooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MinDesiredWidth(100.0f * 2.0f)
	.MaxDesiredWidth(100.0f * 2.0f)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(0.0f, 0.0f, MiddlePadding, 0.0f)
		.VAlign(VAlign_Center)
		[
			MinProperty->CreatePropertyValueWidget()
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MinMaxSpacer", ".."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]

		+SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(MiddlePadding, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			MaxProperty->CreatePropertyValueWidget()
		]
	];
}

void UAnimGraphNode_BoneDrivenController::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_BoneDrivenController, Node), GetClass());

	// if it doesn't have this node, that means this isn't really bone driven controller
	if (!NodeHandle->IsValidHandle())
	{
		return;
	}

	TAttribute<EVisibility> NotUsingCurveVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UAnimGraphNode_BoneDrivenController::AreNonCurveMappingValuesVisible, &DetailBuilder));
	TAttribute<EVisibility> MapRangeVisiblity = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UAnimGraphNode_BoneDrivenController::AreRemappingValuesVisible, &DetailBuilder));

	// Source (Driver) category
	IDetailCategoryBuilder& SourceCategory = DetailBuilder.EditCategory(TEXT("Source (Driver)"));

	// Mapping category
	IDetailCategoryBuilder& MappingCategory = DetailBuilder.EditCategory(TEXT("Mapping"));
	MappingCategory.AddProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, DrivingCurve)));

	MappingCategory.AddProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bUseRange))).Visibility(NotUsingCurveVisibility);

	AddRangePropertyRow(
		/*Name=*/ LOCTEXT("InputRangeLabel", "Source Range"),
		/*Tooltip=*/ LOCTEXT("InputRangeTooltip", "The range (relative to the reference pose) over which to limit the effect of the input component on the output component"),
		/*Category=*/ MappingCategory,
		/*PropertyHandle=*/ NodeHandle,
		/*X=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, RangeMin),
		/*Y=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, RangeMax),
		MapRangeVisiblity);
	AddRangePropertyRow(
		/*Name=*/ LOCTEXT("MappedRangeLabel", "Mapped Range"),
		/*Tooltip=*/ LOCTEXT("MappedRangeTooltip", "The range of mapped values that correspond to the input range"),
		/*Category=*/ MappingCategory,
		/*PropertyHandle=*/ NodeHandle,
		/*X=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, RemappedMin),
		/*Y=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, RemappedMax),
		MapRangeVisiblity);

	MappingCategory.AddProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, Multiplier))).Visibility(NotUsingCurveVisibility);

	// Destination visiblity
	TAttribute<EVisibility> BoneTargetVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UAnimGraphNode_BoneDrivenController::AreTargetBonePropertiesVisible, &DetailBuilder));
	TAttribute<EVisibility> CurveTargetVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UAnimGraphNode_BoneDrivenController::AreTargetCurvePropertiesVisible, &DetailBuilder));

	// Destination (Driven) category
	IDetailCategoryBuilder& TargetCategory = DetailBuilder.EditCategory(TEXT("Destination (Driven)"));

	TargetCategory.AddProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, DestinationMode)));

	TargetCategory.AddProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, ParameterName))).Visibility(CurveTargetVisibility);


	TargetCategory.AddProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, TargetBone))).Visibility(BoneTargetVisibility);
	// Add a note about the space (it is not configurable, and this helps set expectations)
	const FText TargetBoneSpaceName = LOCTEXT("TargetComponentSpace", "Target Component Space");
	TargetCategory.AddCustomRow(TargetBoneSpaceName)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(TargetBoneSpaceName)
			.Font(IDetailLayoutBuilder::GetDetailFont())			
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("TargetComponentSpaceIsAlwaysParentBoneSpace", "Parent Bone Space"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.Visibility(BoneTargetVisibility);

	AddTripletPropertyRow(
		/*Name=*/ LOCTEXT("DrivenTranslationLabel", "Translation"),
		/*Tooltip=*/ LOCTEXT("DrivenTranslationTooltip", "Should the source bone drive one or more translation components of the target bone?"),
		/*Category=*/ TargetCategory,
		/*PropertyHandle=*/ NodeHandle,
		/*X=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetTranslationX),
		/*Y=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetTranslationY),
		/*Z=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetTranslationZ),
		BoneTargetVisibility);

	AddTripletPropertyRow(
		/*Name=*/ LOCTEXT("DrivenRotationLabel", "Rotation"),
		/*Tooltip=*/ LOCTEXT("DrivenRotationTooltip", "Should the source bone drive one or more rotation components of the target bone?"),
		/*Category=*/ TargetCategory,
		/*PropertyHandle=*/ NodeHandle,
		/*X=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetRotationX),
		/*Y=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetRotationY),
		/*Z=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetRotationZ),
		BoneTargetVisibility);

	AddTripletPropertyRow(
		/*Name=*/ LOCTEXT("DrivenScaleLabel", "Scale"),
		/*Tooltip=*/ LOCTEXT("DrivenScaleTooltip", "Should the source bone drive one or more scale components of the target bone?"),
		/*Category=*/ TargetCategory,
		/*PropertyHandle=*/ NodeHandle,
		/*X=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetScaleX),
		/*Y=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetScaleY),
		/*Z=*/ GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, bAffectTargetScaleZ),
		BoneTargetVisibility);

	TargetCategory.AddProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_BoneDrivenController, ModificationMode))).Visibility(BoneTargetVisibility);
}

FText UAnimGraphNode_BoneDrivenController::ComponentTypeToText(EComponentType::Type Component)
{
	switch (Component)
	{
	case EComponentType::TranslationX:
		return LOCTEXT("ComponentType_TranslationX", "translateX");
	case EComponentType::TranslationY:
		return LOCTEXT("ComponentType_TranslationY", "translateY");
	case EComponentType::TranslationZ:
		return LOCTEXT("ComponentType_TranslationZ", "translateZ");
	case EComponentType::RotationX:
		return LOCTEXT("ComponentType_RotationX", "rotateX");
	case EComponentType::RotationY:
		return LOCTEXT("ComponentType_RotationY", "rotateY");
	case EComponentType::RotationZ:
		return LOCTEXT("ComponentType_RotationZ", "rotateZ");
	case EComponentType::Scale:
		return LOCTEXT("ComponentType_ScaleMax", "scaleMax");
	case EComponentType::ScaleX:
		return LOCTEXT("ComponentType_ScaleX", "scaleX");
	case EComponentType::ScaleY:
		return LOCTEXT("ComponentType_ScaleY", "scaleY");
	case EComponentType::ScaleZ:
		return LOCTEXT("ComponentType_ScaleZ", "scaleZ");
	default:
		return LOCTEXT("ComponentType_None", "(none)");
	}
}

EVisibility UAnimGraphNode_BoneDrivenController::AreNonCurveMappingValuesVisible(IDetailLayoutBuilder* DetailLayoutBuilder)
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList = DetailLayoutBuilder->GetSelectedObjects();
	for(TWeakObjectPtr<UObject> Object : SelectedObjectsList)
	{
		if(UAnimGraphNode_BoneDrivenController* BoneDrivenController = Cast<UAnimGraphNode_BoneDrivenController>(Object.Get()))
		{
			if (BoneDrivenController->Node.DrivingCurve == nullptr)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

EVisibility UAnimGraphNode_BoneDrivenController::AreRemappingValuesVisible(IDetailLayoutBuilder* DetailLayoutBuilder)
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList = DetailLayoutBuilder->GetSelectedObjects();
	for(TWeakObjectPtr<UObject> Object : SelectedObjectsList)
	{
		if(UAnimGraphNode_BoneDrivenController* BoneDrivenController = Cast<UAnimGraphNode_BoneDrivenController>(Object.Get()))
		{
			if((BoneDrivenController->Node.DrivingCurve == nullptr) && BoneDrivenController->Node.bUseRange)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

EVisibility UAnimGraphNode_BoneDrivenController::AreTargetBonePropertiesVisible(IDetailLayoutBuilder* DetailLayoutBuilder)
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList = DetailLayoutBuilder->GetSelectedObjects();
	for(TWeakObjectPtr<UObject> Object : SelectedObjectsList)
	{
		if(UAnimGraphNode_BoneDrivenController* BoneDrivenController = Cast<UAnimGraphNode_BoneDrivenController>(Object.Get()))
		{	
			if(BoneDrivenController->Node.DestinationMode == EDrivenDestinationMode::Bone)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

EVisibility UAnimGraphNode_BoneDrivenController::AreTargetCurvePropertiesVisible(IDetailLayoutBuilder* DetailLayoutBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjectsList = DetailLayoutBuilder->GetSelectedObjects();
	for(TWeakObjectPtr<UObject> Object : SelectedObjectsList)
	{
		if(UAnimGraphNode_BoneDrivenController* BoneDrivenController = Cast<UAnimGraphNode_BoneDrivenController>(Object.Get()))
		{
			if(BoneDrivenController->Node.DestinationMode == EDrivenDestinationMode::MorphTarget || BoneDrivenController->Node.DestinationMode == EDrivenDestinationMode::MaterialParameter)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
