// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CCDIK.h"
#include "Animation/AnimInstance.h"
#include "AnimNodeEditModes.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_CCDIK 

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CCDIK)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_CCDIK::UAnimGraphNode_CCDIK(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_CCDIK::GetControllerDescription() const
{
	return LOCTEXT("CCDIK", "CCDIK");
}

FText UAnimGraphNode_CCDIK::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("RootBoneName"), FText::FromName(Node.RootBone.BoneName));
		Args.Add(TEXT("TipBoneName"), FText::FromName(Node.TipBone.BoneName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_CCDIKBone_ListTitle", "{ControllerDescription} - Root: {RootBoneName}, Tip: {TipBoneName} "), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_CCDIKBone_Title", "{ControllerDescription}\nRoot: {RootBoneName} - Tip: {TipBoneName} "), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

void UAnimGraphNode_CCDIK::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_CCDIK* CCDIK = static_cast<FAnimNode_CCDIK*>(InPreviewNode);

	// copies Pin values from the internal node to get data which are not compiled yet
	CCDIK->EffectorLocation = Node.EffectorLocation;
}

FEditorModeID UAnimGraphNode_CCDIK::GetEditorMode() const
{
	return AnimNodeEditModes::CCDIK;
}

void UAnimGraphNode_CCDIK::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FBoneReference, BoneName))
	{
		USkeleton* Skeleton = GetAnimBlueprint()->TargetSkeleton;
		if (Node.TipBone.BoneName != NAME_None && Node.RootBone.BoneName != NAME_None)
		{
			const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
			const int32 TipBoneIndex = RefSkeleton.FindBoneIndex(Node.TipBone.BoneName);
			const int32 RootBoneIndex = RefSkeleton.FindBoneIndex(Node.RootBone.BoneName);

			if (TipBoneIndex != INDEX_NONE && RootBoneIndex != INDEX_NONE)
			{
				const int32 Depth = RefSkeleton.GetDepthBetweenBones(TipBoneIndex, RootBoneIndex);
				if (Depth >= 0)
				{
					Node.ResizeRotationLimitPerJoints(Depth + 1);
				}
				else
				{
					Node.ResizeRotationLimitPerJoints(0);
				}
			}
			else
			{
				Node.ResizeRotationLimitPerJoints(0);
			}
		}
		else
		{
			Node.ResizeRotationLimitPerJoints(0);
		}
	}
}

#undef LOCTEXT_NAMESPACE
