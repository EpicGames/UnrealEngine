// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CallFunction.h"

#include "Animation/AnimInstance.h"
#include "BlueprintCompilationManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "IAnimationBlueprintEditor.h"
#include "K2Node_CustomEvent.h"
#include "KismetCompiler.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_CallFunction.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "IAnimBlueprintCompilationContext.h"
#include "Settings/EditorStyleSettings.h"
#include "AnimBlueprintExtension_CallFunction.h"
#include "ObjectTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CallFunction)

#define LOCTEXT_NAMESPACE "AnimGraphNode_CallFunction"

void UAnimGraphNode_CallFunction::PostLoad()
{
	Super::PostLoad();

	BindDelegates();
}

FLinearColor UAnimGraphNode_CallFunction::GetNodeTitleColor() const
{
	return CallFunctionPrototype ? CallFunctionPrototype->GetNodeTitleColor() : Super::GetNodeTitleColor();
}

FText UAnimGraphNode_CallFunction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText FunctionName;
	UFunction* Function = CallFunctionPrototype ? CallFunctionPrototype->GetTargetFunction() : nullptr;
	if (Function)
	{
		FunctionName = ObjectTools::GetUserFacingFunctionName(Function);
	}
	else if(CallFunctionPrototype)
	{
		FunctionName = FText::FromName(CallFunctionPrototype->FunctionReference.GetMemberName());
		if ((GEditor != nullptr) && (GetDefault<UEditorStyleSettings>()->bShowFriendlyNames))
		{
			FunctionName = FText::FromString(FName::NameToDisplayString(FunctionName.ToString(), false));
		}
	}
	else
	{
		FunctionName = LOCTEXT("Function", "Function");
	}
	
	if(TitleType == ENodeTitleType::FullTitle)
	{
		FTextBuilder TextBuilder;
		TextBuilder.AppendLine(FunctionName);
		TextBuilder.AppendLine(UEnum::GetDisplayValueAsText(Node.CallSite));

		return TextBuilder.ToText();
	}
	else
	{
		return FunctionName;
	}
}

FText UAnimGraphNode_CallFunction::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "A node that calls user-defined functions during animation graph execution");
}

void UAnimGraphNode_CallFunction::AllocateFunctionPins()
{
	if(CallFunctionPrototype)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		
		for(UEdGraphPin* Pin : CallFunctionPrototype->Pins)
		{
			if(!K2Schema->IsExecPin(*Pin) && Pin->PinName != UEdGraphSchema_K2::PN_Self && Pin->Direction == EGPD_Input)
			{
				// Create and copy pin data from prototype K2 node
				UEdGraphPin* NewPin = CreatePin(EGPD_Input, Pin->PinType, Pin->PinName);
				NewPin->DefaultObject = Pin->DefaultObject;
                NewPin->DefaultValue = Pin->DefaultValue;
                NewPin->DefaultTextValue = Pin->DefaultTextValue;
				NewPin->AutogeneratedDefaultValue = Pin->AutogeneratedDefaultValue;
			}
		}
	}
}

void UAnimGraphNode_CallFunction::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& InOldPins)
{
	Super::ReallocatePinsDuringReconstruction(InOldPins);
	
	AllocateFunctionPins();

	RestoreSplitPins(InOldPins);
}

void UAnimGraphNode_CallFunction::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	
	AllocateFunctionPins();
}

FText UAnimGraphNode_CallFunction::GetMenuCategory() const
{
	if(CallFunctionPrototype)
	{
		if(UFunction* Function = CallFunctionPrototype->GetTargetFunction())
		{
			return UK2Node_CallFunction::GetDefaultCategoryForFunction(CallFunctionPrototype->GetTargetFunction(), LOCTEXT("BaseCategory_CallFunction", "Call Function"));
		}
	}
	return FText::GetEmpty();
}

void UAnimGraphNode_CallFunction::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	// Note we dont call super here as we dont have an 'evaluation handler'
	
	if(CallFunctionPrototype)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		UAnimBlueprintExtension_CallFunction* Extension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_CallFunction>(GetAnimBlueprint());
		
		const FName EventName = Extension->FindCustomEventName(this);

		if(EventName != NAME_None)
		{
			UK2Node_CustomEvent* CustomEventNode = CompilerContext.SpawnIntermediateNode<UK2Node_CustomEvent>(this, CompilerContext.ConsolidatedEventGraph);
			CustomEventNode->bInternalEvent = true;
			CustomEventNode->CustomFunctionName = EventName;
			CustomEventNode->AllocateDefaultPins();

			UEdGraphPin* ExecChain = K2Schema->FindExecutionPin(*CustomEventNode, EGPD_Output);

			// Add call function node
			UK2Node_CallFunction* NewCallFunctionNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, CompilerContext.ConsolidatedEventGraph);
			NewCallFunctionNode->FunctionReference = CallFunctionPrototype->FunctionReference;
			NewCallFunctionNode->AllocateDefaultPins();
			
			// link up pins
			for(UEdGraphPin* Pin : CallFunctionPrototype->Pins)
			{
				if(!K2Schema->IsExecPin(*Pin) && Pin->PinName != UEdGraphSchema_K2::PN_Self && Pin->Direction == EGPD_Input)
				{
					UEdGraphPin* AnimGraphPin = FindPinChecked(Pin->PinName);
					UEdGraphPin* NewPin = NewCallFunctionNode->FindPinChecked(Pin->PinName);

					NewPin->CopyPersistentDataFromOldPin(*AnimGraphPin);
				}
			}

			// Link function call into exec chain
			UEdGraphPin* ExecFunctionCall = K2Schema->FindExecutionPin(*NewCallFunctionNode, EGPD_Input);
			ExecChain->MakeLinkTo(ExecFunctionCall);
		}
	}
}

void UAnimGraphNode_CallFunction::SetupFromFunction(UFunction* InFunction)
{
	if(InFunction == nullptr)
	{
		return;
	}

	// Create graph and inner node
	InnerGraph = FBlueprintEditorUtils::CreateNewGraph(this, NAME_None, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());

	bool bIsConsideredSelfContext = false;
	if (const UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(this)))
	{
		bIsConsideredSelfContext = InFunction->GetOwnerClass() == AnimBlueprint->GetAnimBlueprintGeneratedClass();
	}

	FGraphNodeCreator<UK2Node_CallFunction> NodeCreator(*InnerGraph);
	CallFunctionPrototype = NodeCreator.CreateNode();
	CallFunctionPrototype->FunctionReference.SetFromField<UFunction>(InFunction, bIsConsideredSelfContext);
	NodeCreator.Finalize();

	BindDelegates();
}

void UAnimGraphNode_CallFunction::BindDelegates()
{
	if(!GraphChangedHandle.IsValid())
	{
		GraphChangedHandle = InnerGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateLambda([this](const FEdGraphEditAction& InAction)
        {
            // Reconstruct node when the inner graph changes (this catches changes to the function signature)
            ReconstructNode();
        }));
	}

	if(!PinRenamedHandle.IsValid())
	{
		PinRenamedHandle = CallFunctionPrototype->OnUserDefinedPinRenamed().AddLambda([this](UK2Node* InNode, FName InOldName, FName InNewName)
        {
            RenameUserDefinedPin(InOldName, InNewName);
        });
	}
}

bool UAnimGraphNode_CallFunction::IsFunctionDenied(const UFunction* InFunction) const
{
	return InFunction->GetFName() == GET_FUNCTION_NAME_CHECKED(UAnimInstance, BlueprintThreadSafeUpdateAnimation);
}

bool UAnimGraphNode_CallFunction::AreFunctionParamsValid(const UFunction* InFunction) const
{
	for(TFieldIterator<FProperty> It(InFunction); It; ++It)
	{
		const FProperty* Property = *It;

		// As we cant process return params, we don't allow functions with them to be called in the anim graph
		if(Property->HasAnyPropertyFlags(CPF_ReturnParm))
		{
			return false;
		}
	}

	return true;
}

bool UAnimGraphNode_CallFunction::ValidateFunction(const UFunction* InFunction, FCompilerResultsLog* InMessageLog) const
{
	bool bValid = true;

	auto InvalidateMessage = [this, &bValid, InMessageLog](const FText& InMessage)
	{
		bValid = false;
		if(InMessageLog)
		{
			InMessageLog->Error(*InMessage.ToString(), this);
		}
	};

	if(InFunction->HasAnyFunctionFlags(FUNC_BlueprintPure))
	{
		InvalidateMessage(LOCTEXT("PureFunctionError", "@@ cannot call a pure function"));
	}

	if(!FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(InFunction))
	{
		InvalidateMessage(LOCTEXT("ThreadSafetyError", "@@ call is not thread safe"));
	}

	if(!AreFunctionParamsValid(InFunction))
	{
		InvalidateMessage(LOCTEXT("FunctionParamsInvalidError", "@@ has invalid parameters. Return parameters are not allowed"));
	}

	if(InFunction->HasMetaData(FBlueprintMetadata::MD_BlueprintInternalUseOnly))
	{
		InvalidateMessage(LOCTEXT("FunctionInternalError", "@@ uses an internal-only function"));
	}

	if(IsFunctionDenied(InFunction))
	{
		InvalidateMessage(LOCTEXT("FunctionDenyListError", "@@ uses a denied function"));
	}
	
	return bValid;
}

void UAnimGraphNode_CallFunction::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	auto MakeFunctionActionsForClass = [this, &ActionRegistrar](const UClass* InClass)
	{
		auto MakeFunctionAction = [this, &ActionRegistrar, InClass](UFunction* InFunction)
		{
			if (UEdGraphSchema_K2::CanUserKismetCallFunction(InFunction) && ValidateFunction(InFunction))
			{
				auto CustomizeNode = [Function = TWeakObjectPtr<UFunction>(InFunction)](UEdGraphNode* InNode, bool bIsTemplate)
				{
					UFunction* ResolvedFunction = Function.Get();
					if(ResolvedFunction == nullptr)
					{
						return;
					}

					UAnimGraphNode_CallFunction* CallFunctionNode = CastChecked<UAnimGraphNode_CallFunction>(InNode);
					CallFunctionNode->SetupFromFunction(ResolvedFunction);
				};

				UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(UAnimGraphNode_CallFunction::StaticClass(), nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(CustomizeNode));
				FBlueprintActionUiSpec& MenuSignature = Spawner->DefaultMenuSignature;

				MenuSignature.MenuName = FText::Format(LOCTEXT("MenuNameFormat", "{0} (From Anim Graph)"), ObjectTools::GetUserFacingFunctionName(InFunction));
				MenuSignature.Category = UK2Node_CallFunction::GetDefaultCategoryForFunction(InFunction, LOCTEXT("BaseCategory", "Call Function From Anim Graph"));
				MenuSignature.Tooltip = FText::FromString(ObjectTools::GetDefaultTooltipForFunction(InFunction));
				MenuSignature.Keywords = UK2Node_CallFunction::GetKeywordsForFunction(InFunction);

				// add at least one character, so that PrimeDefaultUiSpec() doesn't attempt to query the template node
				if (MenuSignature.Keywords.IsEmpty())
				{
					MenuSignature.Keywords = FText::FromString(TEXT(" "));
				}

				ActionRegistrar.AddBlueprintAction(InClass, Spawner);
			}
		};

		for (TFieldIterator<UFunction> It(InClass, EFieldIterationFlags::IncludeInterfaces); It; ++It)
		{
			MakeFunctionAction(*It);
		}
	};

	if (const UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(ActionRegistrar.GetActionKeyFilter()))
	{
		// Add this class
		MakeFunctionActionsForClass(AnimBlueprint->GetAnimBlueprintGeneratedClass());
	}
	else if (ActionRegistrar.IsOpenForRegistration(GetClass()))
	{
		// Add function libraries too
		for (TObjectIterator<UBlueprintFunctionLibrary> It(RF_NoFlags); It; ++It)
		{
			MakeFunctionActionsForClass(It->GetClass());
		}
	}
}

bool UAnimGraphNode_CallFunction::IsActionFilteredOut(FBlueprintActionFilter const& Filter)
{
	return CallFunctionPrototype == nullptr || InnerGraph == nullptr || CallFunctionPrototype->IsActionFilteredOut(Filter);
}

void UAnimGraphNode_CallFunction::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	OutExtensions.Add(UAnimBlueprintExtension_CallFunction::StaticClass());
}

UObject* UAnimGraphNode_CallFunction::GetJumpTargetForDoubleClick() const
{
	return CallFunctionPrototype ? CallFunctionPrototype->GetTargetFunction() : nullptr;
}

void UAnimGraphNode_CallFunction::JumpToDefinition() const
{
	if(CallFunctionPrototype)
	{
		UFunction* Function = CallFunctionPrototype->GetTargetFunction();
		TSharedPtr<IBlueprintEditor> BlueprintEditor = FKismetEditorUtilities::GetIBlueprintEditorForObject(Function, true);
		if(BlueprintEditor.IsValid())
		{
			BlueprintEditor->JumpToHyperlink(Function, false);
		}
	}
}

void UAnimGraphNode_CallFunction::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	if(CallFunctionPrototype == nullptr || CallFunctionPrototype->GetTargetFunction() == nullptr)
	{
		MessageLog.Error(*LOCTEXT("MissingFunctionPrototypeError", "Missing function, node @@ is invalid").ToString(), this);
	}
	
	if(CallFunctionPrototype != nullptr)
	{
		CallFunctionPrototype->ValidateNodeDuringCompilation(MessageLog);
		
		UFunction* Function = CallFunctionPrototype->GetTargetFunction();
		if(Function)
		{
			ValidateFunction(Function, &MessageLog);
		}
	}
}

void UAnimGraphNode_CallFunction::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimBlueprintExtension_CallFunction* Extension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_CallFunction>(GetAnimBlueprint());

	const FName EventName = Extension->AddCustomEventName(this);
	Node.Function.SetFromFunctionName(EventName);
}

#undef LOCTEXT_NAMESPACE
