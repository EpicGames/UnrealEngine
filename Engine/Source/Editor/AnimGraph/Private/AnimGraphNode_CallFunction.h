// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_CallFunction.h"
#include "Engine/MemberReference.h"

#include "AnimGraphNode_CallFunction.generated.h"

class UEdGraph;
class UK2Node_CallFunction;

UCLASS(MinimalAPI, Experimental, meta=(Keywords = "Event"))
class UAnimGraphNode_CallFunction : public UAnimGraphNode_Base
{
	GENERATED_BODY()

private:
	// Inner graph we maintain to hook into the function call machinery
	UPROPERTY()
	TObjectPtr<UEdGraph> InnerGraph;

	// Inner node we maintain to hook into the function call machinery
	UPROPERTY()
	TObjectPtr<UK2Node_CallFunction> CallFunctionPrototype;
	
    UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_CallFunction Node;

	// Delegate handles used to hook into inner graph & function nodes
	FDelegateHandle GraphChangedHandle;
	FDelegateHandle PinRenamedHandle;

	// UObject interface
	virtual void PostLoad() override;
	
	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void AllocateDefaultPins() override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& InOldPins) override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual void JumpToDefinition() const override;
	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	
	// UK2Node interface
	virtual FText GetMenuCategory() const override;
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;

	// UAnimGraphNode_Base interface
	virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;
	virtual bool ShouldCreateStructEvalHandlers() const override { return false; }
	virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

private:
	void AllocateFunctionPins();

	/** Sets up node internals */
	void SetupFromFunction(UFunction* InFunction);
	void BindDelegates();

	/** Checks to see if the passed-in function is disallowed in anim graphs. */
	bool IsFunctionDenied(const UFunction* InFunction) const;

	/** Validate the function's parameters to check if it can be called in anim graphs */
	bool AreFunctionParamsValid(const UFunction* InFunction) const;

	/** Validates a function. Used for populating menu actions and verifying already-referenced functions */
	bool ValidateFunction(const UFunction* InFunction, FCompilerResultsLog* InMessageLog = nullptr) const;
};
