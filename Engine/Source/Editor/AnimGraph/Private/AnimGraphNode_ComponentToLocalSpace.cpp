// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ComponentToLocalSpace.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ComponentToLocalSpace

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ComponentToLocalSpace)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_ComponentToLocalSpace::UAnimGraphNode_ComponentToLocalSpace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_ComponentToLocalSpace::GetNodeTitleColor() const
{
	return FLinearColor(0.7f, 0.7f, 0.7f);
}

FText UAnimGraphNode_ComponentToLocalSpace::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ComponentToLocalSpace_Tooltip", "Convert Component Space Pose to Local Pose");
}

FText UAnimGraphNode_ComponentToLocalSpace::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_ComponentToLocalSpace_Title", "Component To Local");
}

FText UAnimGraphNode_ComponentToLocalSpace::GetMenuCategory() const
{
	return LOCTEXT("AnimGraphNode_LocalToComponentSpace_Category", "Animation|Convert Spaces");
}

void UAnimGraphNode_ComponentToLocalSpace::PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const
{
	DisplayName = TEXT("");
}

#undef LOCTEXT_NAMESPACE
