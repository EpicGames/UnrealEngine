// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_Constraint.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimNodeEditModes.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_Constraint

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Constraint)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_Constraint"

/* Utility function that gives transform type string for UI */
//////////////////////////////////////////////////
FString GetTransformTypeString(const ETransformConstraintType& TransformType, bool bSimple = false)
{
	switch (TransformType)
	{
	case ETransformConstraintType::Parent:
		return (bSimple) ? TEXT("P") : TEXT("Parent");
	case ETransformConstraintType::Translation:
		return (bSimple) ? TEXT("T") : TEXT("Translation");
	case ETransformConstraintType::Rotation:
		return (bSimple) ? TEXT("R") : TEXT("Rotation");
	case ETransformConstraintType::Scale:
		return (bSimple) ? TEXT("S") : TEXT("Scale");
	}

	return (bSimple) ? TEXT("U") : TEXT("Unknown");
};
//////////////////////////////////////////////////////

UAnimGraphNode_Constraint::UAnimGraphNode_Constraint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_Constraint::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	if (ForSkeleton && ForSkeleton->GetReferenceSkeleton().FindBoneIndex(Node.BoneToModify.BoneName) == INDEX_NONE)
	{
		if (Node.BoneToModify.BoneName == NAME_None)
		{
			MessageLog.Warning(*LOCTEXT("NoBoneSelectedToModify", "@@ - You must pick a bone to modify").ToString(), this);
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("BoneName"), FText::FromName(Node.BoneToModify.BoneName));

			FText Msg = FText::Format(LOCTEXT("NoBoneFoundToModify", "@@ - Bone {BoneName} not found in Skeleton"), Args);

			MessageLog.Warning(*Msg.ToString(), this);
		}
	}

	float OverallWeight = 0.f;
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->GetName().StartsWith(TEXT("ConstraintWeights")))
		{
			OverallWeight += FCString::Atof(*(Pin->DefaultValue));
		}
	}
	if (Node.ConstraintWeights.Num() > 0 && !FMath::IsNearlyEqual(OverallWeight, 1.f, ZERO_ANIMWEIGHT_THRESH * float(Node.ConstraintWeights.Num())))
	{
		MessageLog.Note(*LOCTEXT("WeightsDontSumToOne", "@@ - The weights don't add up to 1.0").ToString(), this);
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

FText UAnimGraphNode_Constraint::GetControllerDescription() const
{
	return LOCTEXT("Constraint", "Constraint");
}

FText UAnimGraphNode_Constraint::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_Constraint_Tooltip", "Constraint to another joint per transform component");
}

FText UAnimGraphNode_Constraint::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	// get simple transform type
	auto GetSimpleTransformString = [this]() -> FString
	{
		FString Ret;
		for (const FConstraint& Constraint : Node.ConstraintSetup)
		{
			Ret += GetTransformTypeString(Constraint.TransformType, true);
		}
		return Ret;
	};
	
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.BoneToModify.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	else 
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.BoneToModify.BoneName));
		Args.Add(TEXT("TransformComponents"), FText::FromString(GetSimpleTransformString()));

		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_Constraint_ListTitle", "{ControllerDescription} - {BoneName} ({TransformComponents})"), Args), this);
	}
	return CachedNodeTitles[TitleType];
}

void UAnimGraphNode_Constraint::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const
{
	if (SkelMeshComp)
	{
		if (FAnimNode_Constraint* ActiveNode = GetActiveInstanceNode<FAnimNode_Constraint>(SkelMeshComp->GetAnimInstance()))
		{
			ActiveNode->ConditionalDebugDraw(PDI, SkelMeshComp);
		}
	}
}

void UAnimGraphNode_Constraint::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		static const FName NAME_ConstraintSetup = GET_MEMBER_NAME_CHECKED(FAnimNode_Constraint, ConstraintSetup);
		const FName PropName = PropertyChangedEvent.Property->GetFName();

		if (PropName == NAME_ConstraintSetup)
		{
			int32 CurrentNum = Node.ConstraintWeights.Num();
			Node.ConstraintWeights.SetNumZeroed(Node.ConstraintSetup.Num());
			int32 NewNum = Node.ConstraintWeights.Num();

			// we want to fill up 1, not 0. 
			for (int32 Index = CurrentNum; Index < NewNum; ++Index)
			{
				Node.ConstraintWeights[Index] = 1.f;
			}

			ReconstructNode();
		}
	}
}

void UAnimGraphNode_Constraint::PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const
{
	Super::PostProcessPinName(Pin, DisplayName);

	if (Pin->Direction == EGPD_Input)
	{
		const FString ConstraintWeightPrefix = TEXT("ConstraintWeights_");
		const FString PinName = Pin->PinName.ToString();
		FString IndexString;
		if (PinName.Split(ConstraintWeightPrefix, nullptr, &IndexString))
		{
			// convert index and display better name
			const int32 Index = FCString::Atoi(*IndexString);
			if (Node.ConstraintSetup.IsValidIndex(Index))
			{
				const FConstraint& Constraint = Node.ConstraintSetup[Index];
				DisplayName = Constraint.TargetBone.BoneName.ToString();
				DisplayName += TEXT(" : ");
				DisplayName += GetTransformTypeString(Constraint.TransformType);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
