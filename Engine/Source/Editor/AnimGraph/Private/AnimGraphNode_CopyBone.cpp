// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CopyBone.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_CopyBoneSkeletalControl

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CopyBone)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_CopyBone::UAnimGraphNode_CopyBone(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_CopyBone::GetControllerDescription() const
{
	return LOCTEXT("CopyBone", "Copy Bone");
}

FText UAnimGraphNode_CopyBone::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_CopyBone_Tooltip", "The Copy Bone control copies the Transform data or any component of it - i.e. Translation, Rotation, or Scale - from one bone to another.");
}

FText UAnimGraphNode_CopyBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.TargetBone.BoneName == NAME_None) && (Node.SourceBone.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("SourceBoneName"), FText::FromName(Node.SourceBone.BoneName));
		Args.Add(TEXT("TargetBoneName"), FText::FromName(Node.TargetBone.BoneName));

		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_CopyBone_ListTitle", "{ControllerDescription} - Source Bone: {SourceBoneName} - Target Bone: {TargetBoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_CopyBone_Title", "{ControllerDescription}\nSource Bone: {SourceBoneName}\nTarget Bone: {TargetBoneName}"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

#undef LOCTEXT_NAMESPACE
