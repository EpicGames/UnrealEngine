// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CopyBoneDelta.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CopyBoneDelta)

#define LOCTEXT_NAMESPACE "CopyBoneDeltaNode"

UAnimGraphNode_CopyBoneDelta::UAnimGraphNode_CopyBoneDelta(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

FText UAnimGraphNode_CopyBoneDelta::GetTooltipText() const
{
	return LOCTEXT("TooltipText", "This node accumulates or copies the transform of the source bone relative to it's ref pose position. Whereas the copy bone node will copy the absolute position");
}

FText UAnimGraphNode_CopyBoneDelta::GetControllerDescription() const
{
	return LOCTEXT("ControllerDescription", "Copy Bone Delta");
}

FText UAnimGraphNode_CopyBoneDelta::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.TargetBone.BoneName == NAME_None) && (Node.SourceBone.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Description"), GetControllerDescription());
		Args.Add(TEXT("Source"), FText::FromName(Node.SourceBone.BoneName));
		Args.Add(TEXT("Target"), FText::FromName(Node.TargetBone.BoneName));

		if(TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			return FText::Format(LOCTEXT("ListTitle", "{Description} - Source Bone: {Source} - Target Bone: {Target}"), Args);
		}
		else
		{
			return FText::Format(LOCTEXT("Title", "{Description}\nSource Bone: {Source}\nTarget Bone: {Target}"), Args);
		}
	}
}

#undef LOCTEXT_NAMESPACE
