// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CopyPoseFromMesh.h"
#include "Animation/AnimAttributes.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_CopyPoseFromMeshSkeletalControl

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CopyPoseFromMesh)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_CopyPoseFromMesh::UAnimGraphNode_CopyPoseFromMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_CopyPoseFromMesh::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_CopyPoseFromMesh_Tooltip", "The Copy Pose From Mesh node copies the pose data from another component to this. Only works when name matches.");
}

FText UAnimGraphNode_CopyPoseFromMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("CopyPoseFromMesh", "Copy Pose From Mesh");
}

FText UAnimGraphNode_CopyPoseFromMesh::GetMenuCategory() const
{
	return LOCTEXT("AnimGraphNode_CopyPoseFromMesh_Category", "Animation|Misc.");
}

void UAnimGraphNode_CopyPoseFromMesh::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
	OutAttributes.Add(UE::Anim::FAttributes::Attributes);
}

#undef LOCTEXT_NAMESPACE
