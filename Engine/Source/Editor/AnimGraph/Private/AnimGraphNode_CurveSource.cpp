// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CurveSource.h"
#include "Animation/AnimAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CurveSource)

#define LOCTEXT_NAMESPACE "ExternalCurve"

FString UAnimGraphNode_CurveSource::GetNodeCategory() const
{
	return TEXT("Animation|Curves");
}

FText UAnimGraphNode_CurveSource::GetTooltipText() const
{
	return LOCTEXT("CurveSourceDescription", "A programmatic source for curves.\nBinds by name to an object that implements ICurveSourceInterface.\nFirst we check the actor that owns this (if any), then we check each of its components to see if we should bind to the source that matches this name.");
}

FText UAnimGraphNode_CurveSource::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType != ENodeTitleType::MenuTitle)
	{
		UEdGraphPin* SourceBindingPin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CurveSource, SourceBinding));
		if (SourceBindingPin == nullptr && Node.SourceBinding != NAME_None)
		{
			return FText::Format(LOCTEXT("AnimGraphNode_CurveSource_Title_Fmt", "Curve Source: {0}"), FText::FromName(Node.SourceBinding));
		}
	}

	return LOCTEXT("AnimGraphNode_CurveSource_Title", "Curve Source");
}

void UAnimGraphNode_CurveSource::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
}

#undef LOCTEXT_NAMESPACE
