// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CustomProperty.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "DetailLayoutBuilder.h"
#include "AnimationGraphSchema.h"
#include "CustomPropertyOptionalPinManager.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "KismetCompiler.h"
#include "IAnimBlueprintCompilationContext.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "AnimGraphNodeBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CustomProperty)

#define LOCTEXT_NAMESPACE "CustomPropNode"

void UAnimGraphNode_CustomProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if(Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::CustomPropertyAnimGraphNodesUseOptionalPinManager)
	{
		// Port exposed property names to optional pins
		for(FName KnownExposablePropertyName : KnownExposableProperties_DEPRECATED)
		{
			FOptionalPinFromProperty OptionalPin;
			OptionalPin.PropertyName = KnownExposablePropertyName;
			OptionalPin.bCanToggleVisibility = true;
			OptionalPin.bShowPin = ExposedPropertyNames_DEPRECATED.Contains(KnownExposablePropertyName);

			CustomPinProperties.Add(OptionalPin);
		}

		KnownExposableProperties_DEPRECATED.Empty();
		ExposedPropertyNames_DEPRECATED.Empty();
	}
}

void UAnimGraphNode_CustomProperty::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin) && MemberPropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, ShowPinForProperties))
	{
		FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
		ReconstructNode();
	}
}

void UAnimGraphNode_CustomProperty::CreateClassVariablesFromBlueprint(IAnimBlueprintVariableCreationContext& InCreationContext)
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (!Pin->bOrphanedPin && !UAnimationGraphSchema::IsPosePin(Pin->PinType))
		{
			// avoid to add properties which already exist on the custom node.
			// for example the ControlRig_CustomNode has a pin called "alpha" which is not custom.
			if (FStructProperty* NodeProperty = CastField<FStructProperty>(GetClass()->FindPropertyByName(TEXT("Node"))))
			{
				if(NodeProperty->Struct->FindPropertyByName(Pin->GetFName()))
				{
					continue;
				}
			}

			// Add prefix to avoid collisions
			FString PrefixedName = GetPinTargetVariableName(Pin);

			// Create a property on the new class to hold the pin data
			InCreationContext.CreateVariable(FName(*PrefixedName), Pin->PinType);
		}
	}
}

void UAnimGraphNode_CustomProperty::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (!Pin->bOrphanedPin && !UAnimationGraphSchema::IsPosePin(Pin->PinType))
		{
			// avoid to add properties which already exist on the custom node.
			// for example the ControlRig_CustomNode has a pin called "alpha" which is not custom.
			if (FStructProperty* NodeProperty = CastField<FStructProperty>(GetClass()->FindPropertyByName(TEXT("Node"))))
			{
				if(NodeProperty->Struct->FindPropertyByName(Pin->GetFName()))
				{
					continue;
				}
			}
			
			FString PrefixedName = GetPinTargetVariableName(Pin);

			// Add mappings to the node
			UStruct* InstClass = GetTargetSkeletonStruct();
			if (FProperty* FoundProperty = FindFProperty<FProperty>(InstClass, Pin->PinName))
			{
				AddSourceTargetProperties(*PrefixedName, FoundProperty->GetFName());
			}
			else
			{
				AddSourceTargetProperties(*PrefixedName, Pin->GetFName());
			}
		}
	}
}

void UAnimGraphNode_CustomProperty::OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	// Copy pin default values to generated properties
	for (UEdGraphPin* Pin : Pins)
	{
		if (!Pin->bOrphanedPin && Pin->LinkedTo.Num() == 0 && !UAnimationGraphSchema::IsPosePin(Pin->PinType))
		{
			FString PrefixedName = GetPinTargetVariableName(Pin);

			if (FProperty* GeneratedProperty = FindFProperty<FProperty>(InPerNodeContext.GetClassDefaultObject()->GetClass(), *PrefixedName))
			{
				if(!FBlueprintEditorUtils::PropertyValueFromString(GeneratedProperty, Pin->GetDefaultAsString(), (uint8*)InPerNodeContext.GetClassDefaultObject(), InPerNodeContext.GetClassDefaultObject()))
				{
					InCompilationContext.GetMessageLog().Warning(TEXT("Unable to push default value for pin @@ on @@"), Pin, this);
				}
			}
		}
	}
}

void UAnimGraphNode_CustomProperty::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	UAnimBlueprint* AnimBP = CastChecked<UAnimBlueprint>(GetBlueprint());

	UObject* OriginalNode = MessageLog.FindSourceObject(this);

	if(NeedsToSpecifyValidTargetClass())
	{
		// Check we have a class set
		const UStruct* TargetClass = GetTargetStruct();
		if(!TargetClass)
		{
			MessageLog.Error(TEXT("Linked graph node @@ has no valid instance class to spawn."), this);
		}
	}
}

void UAnimGraphNode_CustomProperty::GetInstancePinProperty(const IAnimBlueprintCompilationContext& InCompilationContext, UEdGraphPin* InInputPin, FProperty*& OutProperty)
{
	// The actual name of the instance property
	FString FullName = GetPinTargetVariableName(InInputPin);

	if(FProperty* Property = InCompilationContext.FindClassFProperty<FProperty>(*FullName))
	{
		OutProperty = Property;
	}
	else
	{
		OutProperty = nullptr;
	}
}

FString UAnimGraphNode_CustomProperty::GetPinTargetVariableName(const UEdGraphPin* InPin) const
{
	return GetPinTargetVariableName(InPin->GetFName());
}

FString UAnimGraphNode_CustomProperty::GetPinTargetVariableName(FName InPinName) const
{
	return GetPinTargetVariableNameBase(InPinName) + NodeGuid.ToString();
}

FString UAnimGraphNode_CustomProperty::GetPinTargetVariableNameBase(FName InPinName) const
{
	return TEXT("__CustomProperty_") + InPinName.ToString() + TEXT("_");
}

FText UAnimGraphNode_CustomProperty::GetPropertyTypeText(FProperty* Property)
{
	FText PropertyTypeText;

	if(FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		PropertyTypeText = StructProperty->Struct->GetDisplayNameText();
	}
	else if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
	{
		PropertyTypeText = ObjectProperty->PropertyClass->GetDisplayNameText();
	}
	else if(FFieldClass* PropClass = Property->GetClass())
	{
		PropertyTypeText = PropClass->GetDisplayNameText();
	}
	else
	{
		PropertyTypeText = LOCTEXT("PropertyTypeUnknown", "Unknown");
	}
	
	return PropertyTypeText;
}

void UAnimGraphNode_CustomProperty::OnInstanceClassChanged(IDetailLayoutBuilder* DetailBuilder)
{
	if(DetailBuilder)
	{
		DetailBuilder->ForceRefreshDetails();
	}
}

UObject* UAnimGraphNode_CustomProperty::GetJumpTargetForDoubleClick() const
{
	UClass* InstanceClass = GetTargetClass();
	
	if(InstanceClass)
	{
		return InstanceClass->ClassGeneratedBy;
	}

	return nullptr;
}

bool UAnimGraphNode_CustomProperty::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput /*= NULL*/) const
{
	UStruct* InstanceClassToUse = GetTargetStruct();

	// Add our instance class... If that changes we need a recompile
	if(InstanceClassToUse && OptionalOutput)
	{
		OptionalOutput->AddUnique(InstanceClassToUse);
	}

	bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return InstanceClassToUse || bSuperResult;
}

void UAnimGraphNode_CustomProperty::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(FName(TEXT("Settings")));

	// Customize InstanceClass
	{
		TSharedRef<IPropertyHandle> ClassHandle = DetailBuilder.GetProperty(TEXT("Node.InstanceClass"), GetClass());
		if (ClassHandle->IsValidHandle())
		{
			ClassHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateUObject(this, &UAnimGraphNode_CustomProperty::OnStructuralPropertyChanged, &DetailBuilder));
		}
	}
}

void UAnimGraphNode_CustomProperty::CreateCustomPins(TArray<UEdGraphPin*>* OldPins)
{
	if(UStruct* TargetSkeletonClass = GetTargetSkeletonStruct())
	{
		FCustomPropertyOptionalPinManager OptionalPinManager(this, OldPins);
		OptionalPinManager.CreateCustomPins(TargetSkeletonClass);
	}
}

FProperty* UAnimGraphNode_CustomProperty::GetPinProperty(FName InPinName) const
{
	if(UStruct* TargetSkeletonClass = GetTargetSkeletonStruct())
	{
		if(FProperty* Property = TargetSkeletonClass->FindPropertyByName(InPinName))
		{
			return Property;
		}
	}

	return Super::GetPinProperty(InPinName);
}

bool UAnimGraphNode_CustomProperty::IsPinBindable(const UEdGraphPin* InPin) const
{
	if(const FProperty* PinProperty = GetPinProperty(InPin->GetFName()))
	{
		const int32 OptionalPinIndex = CustomPinProperties.IndexOfByPredicate([PinProperty](const FOptionalPinFromProperty& InOptionalPin)
		{
			return PinProperty->GetFName() == InOptionalPin.PropertyName;
		});

		return OptionalPinIndex != INDEX_NONE;
	}

	return Super::IsPinBindable(InPin);
}

bool UAnimGraphNode_CustomProperty::GetPinBindingInfo(FName InPinName, FName& OutBindingName, FProperty*& OutPinProperty, int32& OutOptionalPinIndex) const
{
	OutPinProperty = GetPinProperty(InPinName);
	if(OutPinProperty)
	{
		OutOptionalPinIndex = CustomPinProperties.IndexOfByPredicate([OutPinProperty](const FOptionalPinFromProperty& InOptionalPin)
		{
			return OutPinProperty->GetFName() == InOptionalPin.PropertyName;
		});

		if (OutOptionalPinIndex != INDEX_NONE)
        {
        	OutBindingName = *GetPinTargetVariableName(InPinName); 
			return true;
        }
	}

	return Super::GetPinBindingInfo(InPinName, OutBindingName, OutPinProperty, OutOptionalPinIndex);
}

bool UAnimGraphNode_CustomProperty::HasBinding(FName InBindingName) const
{
	if (GetBinding())
	{
		return GetBinding()->HasBinding(InBindingName, true);
	}

	return false;
}

void UAnimGraphNode_CustomProperty::AddSourceTargetProperties(const FName& InSourcePropertyName, const FName& InTargetPropertyName)
{
	FAnimNode_CustomProperty* CustomPropAnimNode = GetCustomPropertyNode();
	if (CustomPropAnimNode)
	{
		CustomPropAnimNode->SourcePropertyNames.Add(InSourcePropertyName);
		CustomPropAnimNode->DestPropertyNames.Add(InTargetPropertyName);
	}
}

UClass* UAnimGraphNode_CustomProperty::GetTargetClass() const
{
	const FAnimNode_CustomProperty* CustomPropAnimNode = GetCustomPropertyNode();
	if (CustomPropAnimNode)
	{
		return CustomPropAnimNode->GetTargetClass();
	}

	return nullptr;
}

UClass* UAnimGraphNode_CustomProperty::GetTargetSkeletonClass() const
{
	UClass* TargetClass = GetTargetClass();
	if(TargetClass && TargetClass->ClassGeneratedBy)
	{
		UBlueprint* Blueprint = CastChecked<UBlueprint>(TargetClass->ClassGeneratedBy);
		if(Blueprint)
		{
			if (Blueprint->SkeletonGeneratedClass)
			{
				return Blueprint->SkeletonGeneratedClass;
			}
		}
	}
	return TargetClass;
}

void UAnimGraphNode_CustomProperty::OnStructuralPropertyChanged(IDetailLayoutBuilder* DetailBuilder)
{
	if(DetailBuilder)
	{
		DetailBuilder->ForceRefreshDetails();
	}
}

void UAnimGraphNode_CustomProperty::SetCustomPinVisibility(bool bInVisible, int32 InOptionalPinIndex)
{
	if(CustomPinProperties[InOptionalPinIndex].bShowPin != bInVisible)
	{
		FOptionalPinManager::CacheShownPins(CustomPinProperties, OldShownPins);

		CustomPinProperties[InOptionalPinIndex].bShowPin = bInVisible;
		
		FOptionalPinManager::EvaluateOldShownPins(CustomPinProperties, OldShownPins, this);
		ReconstructNode();
	}
}

void UAnimGraphNode_CustomProperty::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if(Pin->LinkedTo.Num() > 0)
	{
		// If we have links, clear any bindings
		RemoveBindings(*GetPinTargetVariableName(Pin));
	}
}

void UAnimGraphNode_CustomProperty::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (GetBinding())
	{
		const FString NewGUID = NodeGuid.ToString();

		GetMutableBinding()->UpdateBindingNames([this, NewGUID](const FString& InOldBindingName)
		{
			// Check pins
			for (UEdGraphPin* Pin : Pins)
			{
				FString PinTargetVariableNameBase = GetPinTargetVariableNameBase(Pin->GetFName());
				if (InOldBindingName.StartsWith(PinTargetVariableNameBase))
				{
					// Replace GUID with this node's if it differs
					FString OldGUID = InOldBindingName.RightChop(PinTargetVariableNameBase.Len());
					if (OldGUID != NewGUID)
					{
						return InOldBindingName.Replace(*OldGUID, *NewGUID);
					}
				}
			}

			return InOldBindingName;
		});
	}
}

#undef LOCTEXT_NAMESPACE
