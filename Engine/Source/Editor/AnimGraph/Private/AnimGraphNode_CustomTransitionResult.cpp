// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_CustomTransitionResult.h"
#include "GraphEditorSettings.h"


/////////////////////////////////////////////////////
// UAnimGraphNode_CustomTransitionResult

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_CustomTransitionResult)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_CustomTransitionResult::UAnimGraphNode_CustomTransitionResult(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_CustomTransitionResult::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText UAnimGraphNode_CustomTransitionResult::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_CustomTransitionResult_Tooltip", "Result node for a custom transition blend graph");
}

FText UAnimGraphNode_CustomTransitionResult::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_CustomTransitionResult_Title", "Custom Transition Blend Result");
}

#undef LOCTEXT_NAMESPACE
