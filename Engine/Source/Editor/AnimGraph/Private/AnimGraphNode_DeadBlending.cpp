// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_DeadBlending.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "Kismet2/CompilerResultsLog.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_DeadBlending)

#define LOCTEXT_NAMESPACE "AnimGraphNode_DeadBlending"


void UAnimGraphNode_DeadBlending::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.DefaultBlendMode = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FLinearColor UAnimGraphNode_DeadBlending::GetNodeTitleColor() const
{
	return FLinearColor(0.0f, 0.1f, 0.2f);
}

FText UAnimGraphNode_DeadBlending::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Performs inertialization using the Dead Blending algorithm.");
}

FText UAnimGraphNode_DeadBlending::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Dead Blending");
}

FText UAnimGraphNode_DeadBlending::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Misc.");
}

void UAnimGraphNode_DeadBlending::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IInertializationRequester::Attribute);
}

void UAnimGraphNode_DeadBlending::OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimGraphNode_DeadBlending* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_DeadBlending>(this);

	FAnimNode_DeadBlending* DestinationNode = reinterpret_cast<FAnimNode_DeadBlending*>(InPerNodeContext.GetDestinationPtr());
	DestinationNode->SetTag(TrueNode->GetTag());
}
#undef LOCTEXT_NAMESPACE
