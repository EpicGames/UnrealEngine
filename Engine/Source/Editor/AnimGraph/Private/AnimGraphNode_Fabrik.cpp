// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_Fabrik.h"
#include "Animation/AnimInstance.h"
#include "AnimNodeEditModes.h"
#include "AnimationCustomVersion.h"
#include "Components/SkeletalMeshComponent.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_Fabrik 

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Fabrik)

#define LOCTEXT_NAMESPACE "AnimGraphNode_Fabrik"

UAnimGraphNode_Fabrik::UAnimGraphNode_Fabrik(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_Fabrik::GetControllerDescription() const
{
	return LOCTEXT("Fabrik", "FABRIK");
}

void UAnimGraphNode_Fabrik::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent * PreviewSkelMeshComp) const
{
	if(PreviewSkelMeshComp)
	{
		if(FAnimNode_Fabrik* ActiveNode = GetActiveInstanceNode<FAnimNode_Fabrik>(PreviewSkelMeshComp->GetAnimInstance()))
		{
			ActiveNode->ConditionalDebugDraw(PDI, PreviewSkelMeshComp);
		}
	}
}

FText UAnimGraphNode_Fabrik::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

void UAnimGraphNode_Fabrik::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_Fabrik* Fabrik = static_cast<FAnimNode_Fabrik*>(InPreviewNode);

	// copies Pin values from the internal node to get data which are not compiled yet
	Fabrik->EffectorTransform = Node.EffectorTransform;
}

FEditorModeID UAnimGraphNode_Fabrik::GetEditorMode() const
{
	return AnimNodeEditModes::Fabrik;
}

void UAnimGraphNode_Fabrik::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAnimationCustomVersion::GUID);

	const int32 CustomAnimVersion = Ar.CustomVer(FAnimationCustomVersion::GUID);

	if (CustomAnimVersion < FAnimationCustomVersion::ConvertIKToSupportBoneSocketTarget)
	{
		if (Node.EffectorTransformBone_DEPRECATED.BoneName != NAME_None)
		{
			Node.EffectorTarget = FBoneSocketTarget(Node.EffectorTransformBone_DEPRECATED.BoneName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
