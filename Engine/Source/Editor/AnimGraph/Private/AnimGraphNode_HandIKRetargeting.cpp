// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_HandIKRetargeting.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_CopyBoneSkeletalControl

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_HandIKRetargeting)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_HandIKRetargeting::UAnimGraphNode_HandIKRetargeting(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_HandIKRetargeting::GetControllerDescription() const
{
	return LOCTEXT("HandIKRetargeting", "Hand IK Retargeting");
}

FText UAnimGraphNode_HandIKRetargeting::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_HandIKRetargeting_Tooltip", "Handle re-targeting of IK Bone chain. Moves IK bone chain to meet FK hand bones, based on HandFKWeight (to favor either side). (0 = favor left hand, 1 = favor right hand, 0.5 = equal weight).");
}

FText UAnimGraphNode_HandIKRetargeting::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

#undef LOCTEXT_NAMESPACE
