// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_IdentityPose.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_IdentityPose

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_IdentityPose)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_IdentityPose::UAnimGraphNode_IdentityPose(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Node.SetRefPoseType(EIT_Additive);
}

FText UAnimGraphNode_IdentityPose::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_IdentityPose_Tooltip", "Returns identity pose.");
}

FText UAnimGraphNode_IdentityPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_IdentityPose_Title", "Additive Identity Pose");
}

#undef LOCTEXT_NAMESPACE
