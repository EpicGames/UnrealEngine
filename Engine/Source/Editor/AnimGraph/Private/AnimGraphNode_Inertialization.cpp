// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_Inertialization.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "Kismet2/CompilerResultsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Inertialization)

#define LOCTEXT_NAMESPACE "AnimGraphNode_Inertialization"


FLinearColor UAnimGraphNode_Inertialization::GetNodeTitleColor() const
{
	return FLinearColor(0.0f, 0.1f, 0.2f);
}

FText UAnimGraphNode_Inertialization::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Inertialization");
}

FText UAnimGraphNode_Inertialization::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Inertialization");
}

FText UAnimGraphNode_Inertialization::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Misc.");
}

void UAnimGraphNode_Inertialization::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IInertializationRequester::Attribute);
}

void UAnimGraphNode_Inertialization::OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimGraphNode_Inertialization* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_Inertialization>(this);

	FAnimNode_Inertialization* DestinationNode = reinterpret_cast<FAnimNode_Inertialization*>(InPerNodeContext.GetDestinationPtr());
	DestinationNode->SetTag(TrueNode->GetTag());
}
#undef LOCTEXT_NAMESPACE
