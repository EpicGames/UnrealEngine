// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LayeredBoneBlend.h"
#include "ToolMenus.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "AnimGraphCommands.h"
#include "ScopedTransaction.h"

#include "DetailLayoutBuilder.h"
#include "Kismet2/CompilerResultsLog.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_LayeredBoneBlend

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LayeredBoneBlend)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_LayeredBoneBlend::UAnimGraphNode_LayeredBoneBlend(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Node.AddFirstPose();
}

FLinearColor UAnimGraphNode_LayeredBoneBlend::GetNodeTitleColor() const
{
	return FLinearColor(0.2f, 0.8f, 0.2f);
}

FText UAnimGraphNode_LayeredBoneBlend::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_LayeredBoneBlend_Tooltip", "Layered blend per bone");
}

FText UAnimGraphNode_LayeredBoneBlend::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_LayeredBoneBlend_Title", "Layered blend per bone");
}

void UAnimGraphNode_LayeredBoneBlend::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_LayeredBoneBlend, BlendMode))
	{
		// If we  change blend modes, we need to resize our containers
		FScopedTransaction Transaction(LOCTEXT("ChangeBlendMode", "Change Blend Mode"));
		Modify();
		Node.SyncBlendMasksAndLayers();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FString UAnimGraphNode_LayeredBoneBlend::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

void UAnimGraphNode_LayeredBoneBlend::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.BlendMode != ELayeredBoneBlendMode::BranchFilter)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_LayeredBoneBlend, LayerSetup)));
	}

	if (Node.BlendMode != ELayeredBoneBlendMode::BlendMask)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_LayeredBoneBlend, BlendMasks)));
	}

	Super::CustomizeDetails(DetailBuilder);
}

void UAnimGraphNode_LayeredBoneBlend::PreloadRequiredAssets()
{
	// Preload our blend profiles in case they haven't been loaded by the skeleton yet.
	if (Node.BlendMode == ELayeredBoneBlendMode::BlendMask)
	{
		int32 NumBlendMasks = Node.BlendMasks.Num();
		for (int32 MaskIndex = 0; MaskIndex < NumBlendMasks; ++MaskIndex)
		{
			UBlendProfile* BlendMask = Node.BlendMasks[MaskIndex];
			PreloadObject(BlendMask);
		}
	}

	Super::PreloadRequiredAssets();
}

void UAnimGraphNode_LayeredBoneBlend::AddPinToBlendByFilter()
{
	FScopedTransaction Transaction( LOCTEXT("AddPinToBlend", "AddPinToBlendByFilter") );
	Modify();

	Node.AddPose();
	ReconstructNode();
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UAnimGraphNode_LayeredBoneBlend::RemovePinFromBlendByFilter(UEdGraphPin* Pin)
{
	FScopedTransaction Transaction( LOCTEXT("RemovePinFromBlend", "RemovePinFromBlendByFilter") );
	Modify();

	FProperty* AssociatedProperty;
	int32 ArrayIndex;
	GetPinAssociatedProperty(GetFNodeType(), Pin, /*out*/ AssociatedProperty, /*out*/ ArrayIndex);

	if (ArrayIndex != INDEX_NONE)
	{
		//@TODO: ANIMREFACTOR: Need to handle moving pins below up correctly
		// setting up removed pins info 
		RemovedPinArrayIndex = ArrayIndex;
		Node.RemovePose(ArrayIndex);
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_LayeredBoneBlend::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeLayeredBoneblend", LOCTEXT("LayeredBoneBlend", "Layered Bone Blend"));
			if (Context->Pin != NULL)
			{
				// we only do this for normal BlendList/BlendList by enum, BlendList by Bool doesn't support add/remove pins
				if (Context->Pin->Direction == EGPD_Input)
				{
					//@TODO: Only offer this option on arrayed pins
					Section.AddMenuEntry(FAnimGraphCommands::Get().RemoveBlendListPin);
				}
			}
			else
			{
				Section.AddMenuEntry(FAnimGraphCommands::Get().AddBlendListPin);
			}
		}
	}
}

void UAnimGraphNode_LayeredBoneBlend::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	UAnimGraphNode_Base::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	bool bCompilationError = false;
	// Validate blend masks
	if (Node.BlendMode == ELayeredBoneBlendMode::BlendMask)
	{
		int32 NumBlendMasks = Node.BlendMasks.Num();
		for (int32 MaskIndex = 0; MaskIndex < NumBlendMasks; ++MaskIndex)
		{
			const UBlendProfile* BlendMask = Node.BlendMasks[MaskIndex];
			if (BlendMask == nullptr && !GetAnimBlueprint()->bIsTemplate)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("LayeredBlendNullMask", "@@ has null BlendMask for Blend Pose {0}. "), FText::AsNumber(MaskIndex)).ToString(), this, BlendMask);
				bCompilationError = true;
			}
			
			if (BlendMask && BlendMask->Mode != EBlendProfileMode::BlendMask)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("LayeredBlendProfileModeError", "@@ is using a BlendProfile(@@) without a BlendMask mode for Blend Pose {0}. "), FText::AsNumber(MaskIndex)).ToString(), this, BlendMask);
				bCompilationError = true;
			}
		}
	}

	// Don't rebuild the node's data if compilation failed. We may be attempting to do so with invalid data.
	if (bCompilationError)
	{
		return;
	}

	// ensure to cache the per-bone blend weights
 	if (!Node.ArePerBoneBlendWeightsValid(ForSkeleton))
 	{
 		Node.RebuildPerBoneBlendWeights(ForSkeleton);
 	}
}

void UAnimGraphNode_LayeredBoneBlend::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	
	if (Ar.IsLoading() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::AnimLayeredBoneBlendMasksFix)
	{
		Node.SyncBlendMasksAndLayers();
	}
}

void UAnimGraphNode_LayeredBoneBlend::PostLoad()
{
	Super::PostLoad();

	// Post-load our blend masks, in case they've been pre-loaded, but haven't had their bone references initialized yet.
	if (Node.BlendMode == ELayeredBoneBlendMode::BlendMask)
	{
		int32 NumBlendMasks = Node.BlendMasks.Num();
		for (int32 MaskIndex = 0; MaskIndex < NumBlendMasks; ++MaskIndex)
		{
			if(UBlendProfile* BlendMask = Node.BlendMasks[MaskIndex])
			{
				BlendMask->ConditionalPostLoad();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
