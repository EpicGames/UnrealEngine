// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LegIK.h"
#include "Kismet2/CompilerResultsLog.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_LegIK

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LegIK)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_LegIK::UAnimGraphNode_LegIK(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_LegIK::GetControllerDescription() const
{
	return LOCTEXT("LegIK", "Leg IK");
}

FText UAnimGraphNode_LegIK::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_LegIK_Tooltip", "IK node for multi-bone legs.");
}

FText UAnimGraphNode_LegIK::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

void UAnimGraphNode_LegIK::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	if (Node.LegsDefinition.IsEmpty())
	{
		MessageLog.Warning(*LOCTEXT("MissingLegsDefinition", "@@ - Legs Definition is empty").ToString(), this);
	}
	else if (ForSkeleton)
	{
		for (const FAnimLegIKDefinition& LegDefinition : Node.LegsDefinition)
		{
			if (ForSkeleton->GetReferenceSkeleton().FindBoneIndex(LegDefinition.FKFootBone.BoneName) == INDEX_NONE)
			{
				MessageLog.Warning(*LOCTEXT("InvalidFKFootBone", "@@ - Invalid FKFoot Bone in Legs Definition").ToString(), this);
			}

			if (ForSkeleton->GetReferenceSkeleton().FindBoneIndex(LegDefinition.IKFootBone.BoneName) == INDEX_NONE)
			{
				MessageLog.Warning(*LOCTEXT("InvalidIKFootBone", "@@ - Invalid IKFoot Bone in Legs Definition").ToString(), this);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
