// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LinkedAnimGraph.h"

#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimGraphNode_CallFunction.h"
#include "BlueprintNodeSpawner.h"
#include "Animation/AnimBlueprint.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Blueprint.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "KismetCompiler.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LinkedAnimGraph)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_LinkedAnimGraph"

void UAnimGraphNode_LinkedAnimGraph::PostPasteNode()
{
	Super::PostPasteNode();

	// Clear incompatible target class
	if(UClass* InstanceClass = GetTargetClass())
	{
		if(UAnimBlueprint* LinkedBlueprint = Cast<UAnimBlueprint>(UBlueprint::GetBlueprintFromClass(InstanceClass)))
		{
			if(UAnimBlueprint* ThisBlueprint = GetAnimBlueprint())
			{
				if(!LinkedBlueprint->bIsTemplate && !ThisBlueprint->bIsTemplate && LinkedBlueprint->TargetSkeleton != ThisBlueprint->TargetSkeleton)
				{
					Node.InstanceClass = nullptr;
				}
			}
		}
	}
}

TArray<UEdGraph*> UAnimGraphNode_LinkedAnimGraph::GetExternalGraphs() const
{
	if(UClass* InstanceClass = GetTargetClass())
	{
		if(UAnimBlueprint* LinkedBlueprint = Cast<UAnimBlueprint>(UBlueprint::GetBlueprintFromClass(InstanceClass)))
		{
			for(UEdGraph* Graph : LinkedBlueprint->FunctionGraphs)
			{
				if(Graph->GetFName() == UEdGraphSchema_K2::GN_AnimGraph)
				{
					return { Graph };
				}
			}
		}
	}

	return TArray<UEdGraph*>();
}

void UAnimGraphNode_LinkedAnimGraph::SetupFromAsset(const FAssetData& InAssetData, bool bInIsTemplateNode)
{
	if(InAssetData.IsValid())
	{
		InAssetData.GetTagValue("TargetSkeleton", SkeletonName);
		if(SkeletonName == TEXT("None"))
		{
			SkeletonName.Empty();
		}

		FString TagTemplateValue;
		InAssetData.GetTagValue("bIsTemplate", TagTemplateValue);
		bIsTemplateAnimBlueprint = TagTemplateValue.Equals(TEXT("True"));

		FString BlueprintTypeValue;
		InAssetData.GetTagValue("BlueprintType", BlueprintTypeValue);
		bIsInterfaceBlueprint = BlueprintTypeValue.Equals(TEXT("BPTYPE_Interface"));
		
		if(!bInIsTemplateNode)
		{
			UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(InAssetData.GetAsset());
			Node.InstanceClass = AnimBlueprint->GeneratedClass.Get();
		}
	}

	// Set up function reference
	FunctionReference.SetExternalMember(UEdGraphSchema_K2::GN_AnimGraph, GetTargetClass());
}

void UAnimGraphNode_LinkedAnimGraph::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	UAnimGraphNode_AssetPlayerBase::GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UAnimBlueprint::StaticClass()},
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				FText DisplayName;
				InAssetData.GetTagValue(FBlueprintTags::BlueprintDisplayName, DisplayName);
				if(!DisplayName.IsEmpty())
				{
					return DisplayName;
				}
				else
				{
					return FText::Format(LOCTEXT("MenuDescFormat", "{0} - Linked Anim Graph"), FText::FromName(InAssetData.AssetName));
				}
			}

			return LOCTEXT("MenuDesc", "Linked Anim Graph");
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				FText Description;
				InAssetData.GetTagValue(FBlueprintTags::BlueprintDescription, Description);
				if(!Description.IsEmpty())
				{
					return Description;
				}
				else
				{
					return FText::Format(LOCTEXT("MenuDescTooltipFormat", "Linked Anim Graph - Runs a linked anim graph in another instance to process animation\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
				}
			}
			else
			{
				return LOCTEXT("MenuDescTooltip", "Linked Anim Graph");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_LinkedAnimGraph* GraphNode = CastChecked<UAnimGraphNode_LinkedAnimGraph>(InNewNode);
			GraphNode->SetupFromAsset(InAssetData, bInIsTemplateNode);
		},
		nullptr,
		[](const FAssetData& InAssetData)
		{
			if(InAssetData.IsValid())
			{
				FText Category;
				InAssetData.GetTagValue(FBlueprintTags::BlueprintCategory, Category);
				return Category;
			}
			else
			{
				return FText::GetEmpty();
			}
		});
}

bool UAnimGraphNode_LinkedAnimGraph::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;

	if(bIsInterfaceBlueprint)
	{
		bIsFilteredOut = true;
	}
	else if(!SkeletonName.IsEmpty())
	{
		FBlueprintActionContext const& FilterContext = Filter.Context;

		for (UBlueprint* Blueprint : FilterContext.Blueprints)
		{
			if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
			{
				if(!AnimBlueprint->IsCompatibleByAssetString(SkeletonName, bIsTemplateAnimBlueprint, bIsInterfaceBlueprint))
				{
					bIsFilteredOut = true;
					break;
				}
			}
			else
			{
				// Not an animation Blueprint, cannot use
				bIsFilteredOut = true;
				break;
			}
		}
	}
	return bIsFilteredOut;
}

void UAnimGraphNode_LinkedAnimGraph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	
	if(Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimGraphNodeTaggingAdded)
	{
		// Transfer old tag to new system
		SetTagInternal(Node.Tag_DEPRECATED);
	}
}

FAnimNode_CustomProperty* UAnimGraphNode_LinkedAnimGraph::GetCustomPropertyNode()
{
	FAnimNode_CustomProperty* const RuntimeCustomPropertyNode = GetDebuggedAnimNode<FAnimNode_CustomProperty>();
	return RuntimeCustomPropertyNode ? RuntimeCustomPropertyNode : &Node;
}

const FAnimNode_CustomProperty* UAnimGraphNode_LinkedAnimGraph::GetCustomPropertyNode() const
{
	const FAnimNode_CustomProperty* const RuntimeCustomPropertyNode = GetDebuggedAnimNode<FAnimNode_CustomProperty>();
	return RuntimeCustomPropertyNode ? RuntimeCustomPropertyNode : &Node;
}

FAnimNode_LinkedAnimGraph* UAnimGraphNode_LinkedAnimGraph::GetLinkedAnimGraphNode()
{
	FAnimNode_LinkedAnimGraph* const RuntimeLinkedAnimGraphNode = GetDebuggedAnimNode<FAnimNode_LinkedAnimGraph>();
	return RuntimeLinkedAnimGraphNode ? RuntimeLinkedAnimGraphNode : &Node;
}

const FAnimNode_LinkedAnimGraph* UAnimGraphNode_LinkedAnimGraph::GetLinkedAnimGraphNode() const
{
	const FAnimNode_LinkedAnimGraph* const RuntimeLinkedAnimGraphNode = GetDebuggedAnimNode<FAnimNode_LinkedAnimGraph>();
	return RuntimeLinkedAnimGraphNode ? RuntimeLinkedAnimGraphNode : &Node;
}

#undef LOCTEXT_NAMESPACE
