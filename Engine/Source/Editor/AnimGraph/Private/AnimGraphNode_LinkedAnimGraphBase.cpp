// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "AnimationGraphSchema.h"
#include "Widgets/Layout/SBox.h"
#include "UObject/CoreRedirects.h"
#include "Animation/AnimNode_LinkedAnimGraph.h"
#include "AnimGraphAttributes.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "AnimBlueprintExtension_LinkedAnimGraph.h"
#include "AnimGraphNodeCustomizationInterface.h"
#include "BlueprintNodeTemplateCache.h"
#include "EdGraphSchema_K2_Actions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LinkedAnimGraphBase)

#define LOCTEXT_NAMESPACE "LinkedAnimGraph"

namespace LinkedAnimGraphGraphNodeConstants
{
	FLinearColor TitleColor(0.2f, 0.2f, 0.8f);
}


void UAnimGraphNode_LinkedAnimGraphBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if(Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::LinkedAnimGraphMemberReference)
	{
		// Upgrade name to member reference. 
		// Note that if the name has changed underneath us, we still cant recover the GUID
		// until this asset has its node refreshed & the node is resaved
		const FAnimNode_LinkedAnimGraph& RuntimeNode = *GetLinkedAnimGraphNode();
		const FName FunctionName = RuntimeNode.GetDynamicLinkFunctionName();
		UClass* TargetClass = GetTargetSkeletonClass();
		if(FunctionName != NAME_None)
		{
			if(TargetClass != nullptr)
			{
				FunctionReference.SetExternalMember(FunctionName, TargetClass);
			}
			else
			{
				FunctionReference.SetSelfMember(FunctionName);
			}
		}
	}
}

void UAnimGraphNode_LinkedAnimGraphBase::AllocatePoseLinks()
{
	FAnimNode_LinkedAnimGraph& RuntimeNode = *GetLinkedAnimGraphNode();

	RuntimeNode.InputPoses.Empty();
	RuntimeNode.InputPoseNames.Empty();

	for(UEdGraphPin* Pin : Pins)
	{
		if(!Pin->bOrphanedPin)
		{
			if (UAnimationGraphSchema::IsPosePin(Pin->PinType))
			{
				if(Pin->Direction == EGPD_Input)
				{
					RuntimeNode.InputPoses.AddDefaulted();
					RuntimeNode.InputPoseNames.Add(Pin->GetFName());
				}
			}
		}
	}
}

FLinearColor UAnimGraphNode_LinkedAnimGraphBase::GetNodeTitleColor() const
{
	TOptional<FLinearColor> Color;
	
	UClass* TargetClass = GetTargetClass();
	if(TargetClass && TargetClass->ImplementsInterface(UAnimGraphNodeCustomizationInterface::StaticClass()))
	{
		FEditorScriptExecutionGuard AllowScripts;
		Color = IAnimGraphNodeCustomizationInterface::Execute_GetTitleColor(TargetClass->GetDefaultObject());
	}
	
	if(!Color.IsSet())
	{
		return GetDefaultNodeTitleColor();
	}

	return Color.GetValue();
}

FSlateIcon UAnimGraphNode_LinkedAnimGraphBase::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimBlueprint");
}

FText UAnimGraphNode_LinkedAnimGraphBase::GetTooltipText() const
{
	TOptional<FText> TooltipText;
	UClass* TargetClass = GetTargetClass();

	// With a null class, template nodes return an empty string to let the metadata take priority
	if(TargetClass == nullptr && FBlueprintNodeTemplateCache::IsTemplateOuter(GetGraph()))
	{
		return FText::GetEmpty();
	}
	
	static const FName NAME_Tooltip(TEXT("Tooltip"));
	if(TargetClass && TargetClass->HasMetaData(NAME_Tooltip))
	{
		TooltipText = TargetClass->GetMetaDataText(NAME_Tooltip);
	}

	if(!TooltipText.IsSet() || TooltipText.GetValue().IsEmpty())
	{
		return LOCTEXT("ToolTip", "Runs a linked anim graph in another instance to process animation");
	}

	return TooltipText.GetValue();
}

FText UAnimGraphNode_LinkedAnimGraphBase::GetMenuCategory() const
{
	TOptional<FText> Category;
	UClass* TargetClass = GetTargetClass();
	static const FName NAME_Category(TEXT("Category"));
	if(TargetClass && TargetClass->HasMetaData(NAME_Category))
	{
		Category = TargetClass->GetMetaDataText(NAME_Category);
	}

	if(!Category.IsSet() || Category.GetValue().IsEmpty())
	{
		return LOCTEXT("LinkedAnimGraphCategory", "Animation|Linked Anim Graphs");
	}

	return Category.GetValue();
}

FText UAnimGraphNode_LinkedAnimGraphBase::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	TOptional<FText> Title;
	UClass* TargetClass = GetTargetClass();
	static const FName NAME_DisplayName(TEXT("DisplayName"));
	if(TargetClass && TargetClass->HasMetaData(NAME_DisplayName))
	{
		Title = TargetClass->GetMetaDataText(NAME_DisplayName);
	}

	if(!Title.IsSet() || Title.GetValue().IsEmpty())
	{
		UAnimBlueprint* TargetAnimBlueprint = TargetClass ? CastChecked<UAnimBlueprint>(TargetClass->ClassGeneratedBy) : nullptr;
		const FName TargetAnimBlueprintName = TargetAnimBlueprint ? TargetAnimBlueprint->GetFName() : NAME_None;
		
		const FAnimNode_LinkedAnimGraph& Node = *GetLinkedAnimGraphNode();

		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeType"), LOCTEXT("Title", "Linked Anim Graph"));
		Args.Add(TEXT("TargetClass"), FText::FromName(TargetAnimBlueprintName));

		if(TitleType == ENodeTitleType::MenuTitle)
		{
			Title = LOCTEXT("Title", "Linked Anim Graph");
		}
		else if(TitleType == ENodeTitleType::ListView)
		{
			if(TargetClass)
			{
				Title = FText::Format(LOCTEXT("TitleListFormat", "{TargetClass}"), Args);
			}
			else
			{
				Title = LOCTEXT("Title", "Linked Anim Graph");
			}
		}
		else
		{
			if(TargetClass)
			{
				Title = FText::Format(LOCTEXT("TitleFormat", "{TargetClass}\n{NodeType}"), Args);
			}
			else
			{
				Title = LOCTEXT("Title", "Linked Anim Graph");
			}
		}
	}

	return Title.GetValue();
}

void UAnimGraphNode_LinkedAnimGraphBase::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	UAnimBlueprint* AnimBP = CastChecked<UAnimBlueprint>(GetBlueprint());

	UObject* OriginalNode = MessageLog.FindSourceObject(this);

	if(HasInstanceLoop())
	{
		MessageLog.Error(TEXT("Detected loop in linked instance chain starting at @@ inside class @@"), this, AnimBP->GetAnimBlueprintGeneratedClass());
	}

	// Check for cycles from other linked instance nodes
	TArray<UEdGraph*> Graphs;
	AnimBP->GetAllGraphs(Graphs);

	const FAnimNode_LinkedAnimGraph& Node = *GetLinkedAnimGraphNode();

	// Check we don't try to spawn our own blueprint
	if(GetTargetClass() == AnimBP->GetAnimBlueprintGeneratedClass())
	{
		MessageLog.Error(TEXT("Linked instance node @@ targets instance class @@ which it is inside, this would cause a loop."), this, AnimBP->GetAnimBlueprintGeneratedClass());
	}

	// Check for compatibility
	if(UAnimBlueprint* TargetAnimBP = Cast<UAnimBlueprint>(UBlueprint::GetBlueprintFromClass(GetTargetClass())))
	{
		if(!AnimBP->IsCompatible(TargetAnimBP))
		{
			MessageLog.Error(TEXT("Linked instance node @@ targets instance class @@ which is incompatible."), this, GetTargetClass());
		}
	}
}

void UAnimGraphNode_LinkedAnimGraphBase::CreateOutputPins()
{
	Super::CreateOutputPins();
	
	// Grab the SKELETON class here as when we are reconstructed during during BP compilation
	// the full generated class is not yet present built.
	if(UClass* TargetClass = GetTargetSkeletonClass())
	{
		if(UFunction* Function = FunctionReference.ResolveMember<UFunction>(TargetClass))
		{
			// Could have re-resolved to a new member name via GUID 
			const FAnimNode_LinkedAnimGraph& Node = *GetLinkedAnimGraphNode();
			if(Node.GetDynamicLinkFunctionName() != FunctionReference.GetMemberName())
			{
				HandleFunctionReferenceChanged(FunctionReference.GetMemberName());
			}
			
			Function = FBlueprintEditorUtils::GetMostUpToDateFunction(Function);
			
			IterateFunctionParameters(Function, [this](FName InName, FEdGraphPinType InPinType)
			{
				if(UAnimationGraphSchema::IsPosePin(InPinType))
				{
					UEdGraphPin* NewPin = CreatePin(EEdGraphPinDirection::EGPD_Input, UAnimationGraphSchema::MakeLocalSpacePosePin(), InName);
					NewPin->PinFriendlyName = FText::FromName(InName);
					CustomizePinData(NewPin, InName, INDEX_NONE);
				}
			});
		}
	}
}

void UAnimGraphNode_LinkedAnimGraphBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRequiresNodeReconstruct = false;
	FProperty* ChangedProperty = PropertyChangedEvent.Property;

	if(ChangedProperty)
	{
		if (IsStructuralProperty(ChangedProperty))
		{
			// Set function reference if node structure changes
			const FAnimNode_LinkedAnimGraph& Node = *GetLinkedAnimGraphNode();
			UClass* TargetClass = GetTargetSkeletonClass();
			FName FunctionName = Node.GetDynamicLinkFunctionName();
			if(FunctionName != NAME_None)
			{
				if(TargetClass != nullptr)
				{
					FGuid FunctionGuid;
					FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(TargetClass), FunctionName, FunctionGuid);
					FunctionReference.SetExternalMember(FunctionName, TargetClass, FunctionGuid);
				}
				else
				{
					FunctionReference.SetSelfMember(FunctionName);
				}
			}
			else
			{
				FunctionReference = FMemberReference();
			}

			bRequiresNodeReconstruct = true;
		}
	}

	if(bRequiresNodeReconstruct)
	{
		ReconstructNode();
	}
}

bool UAnimGraphNode_LinkedAnimGraphBase::HasInstanceLoop()
{
	TArray<FGuid> VisitedList;
	TArray<FGuid> CurrentStack;
	return HasInstanceLoop_Recursive(this, VisitedList, CurrentStack);
}

bool UAnimGraphNode_LinkedAnimGraphBase::HasInstanceLoop_Recursive(UAnimGraphNode_LinkedAnimGraphBase* CurrNode, TArray<FGuid>& VisitedNodes, TArray<FGuid>& NodeStack)
{
	if(!VisitedNodes.Contains(CurrNode->NodeGuid))
	{
		VisitedNodes.Add(CurrNode->NodeGuid);
		NodeStack.Add(CurrNode->NodeGuid);

		if(UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(UBlueprint::GetBlueprintFromClass(CurrNode->GetTargetClass())))
		{
			// Check for cycles from other linked instance nodes
			TArray<UEdGraph*> Graphs;
			AnimBP->GetAllGraphs(Graphs);

			for(UEdGraph* Graph : Graphs)
			{
				TArray<UAnimGraphNode_LinkedAnimGraphBase*> LinkedInstanceNodes;
				Graph->GetNodesOfClass(LinkedInstanceNodes);

				for(UAnimGraphNode_LinkedAnimGraphBase* LinkedInstanceNode : LinkedInstanceNodes)
				{
					// If we haven't visited this node, then check it for loops, otherwise if we're pointing to a previously visited node that is in the current instance stack we have a loop
					if((!VisitedNodes.Contains(LinkedInstanceNode->NodeGuid) && HasInstanceLoop_Recursive(LinkedInstanceNode, VisitedNodes, NodeStack)) || NodeStack.Contains(LinkedInstanceNode->NodeGuid))
					{
						return true;
					}
				}
			}
		}
	}

	NodeStack.Remove(CurrNode->NodeGuid);
	return false;
}

void UAnimGraphNode_LinkedAnimGraphBase::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	GenerateExposedPinsDetails(DetailBuilder);

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(FName(TEXT("Settings")));

	// Customize InstanceClass
	{
		TSharedRef<IPropertyHandle> ClassHandle = DetailBuilder.GetProperty(TEXT("Node.InstanceClass"), GetClass());
		ClassHandle->MarkHiddenByCustomization();

		FDetailWidgetRow& ClassWidgetRow = CategoryBuilder.AddCustomRow(LOCTEXT("FilterStringInstanceClass", "Instance Class"));
		ClassWidgetRow.NameContent()
		[
			ClassHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		[
			SNew(SObjectPropertyEntryBox)
			.ObjectPath_UObject(this, &UAnimGraphNode_LinkedAnimGraphBase::GetCurrentInstanceBlueprintPath)
			.AllowedClass(UAnimBlueprint::StaticClass())
			.NewAssetFactories(TArray<UFactory*>())
			.OnShouldFilterAsset(FOnShouldFilterAsset::CreateUObject(this, &UAnimGraphNode_LinkedAnimGraphBase::OnShouldFilterInstanceBlueprint))
			.OnObjectChanged(FOnSetObject::CreateUObject(this, &UAnimGraphNode_LinkedAnimGraphBase::OnSetInstanceBlueprint, &DetailBuilder))
		];
	}
}

void UAnimGraphNode_LinkedAnimGraphBase::GenerateExposedPinsDetails(IDetailLayoutBuilder &DetailBuilder)
{
	// We dont allow multi-select here
	if(DetailBuilder.GetSelectedObjects().Num() > 1)
	{
		DetailBuilder.HideCategory(TEXT("Settings"));
		return;
	}
	
	const FStructProperty* NodeProperty = GetFNodeProperty();
	if(NodeProperty == nullptr)
	{
		return;
	}
	
	if(CustomPinProperties.Num() > 0)
	{
		TSharedRef<IPropertyHandle> NodePropertyHandle = DetailBuilder.GetProperty(NodeProperty->GetFName(), GetClass());
		
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(FName(TEXT("Exposable Properties")));

		for(int32 OptionalPinIndex = 0; OptionalPinIndex < CustomPinProperties.Num(); ++OptionalPinIndex)
		{
			const FOptionalPinFromProperty& OptionalProperty = CustomPinProperties[OptionalPinIndex];

			// Find the property of our inner class
			FProperty* Property = GetPinProperty(OptionalProperty.PropertyName);
			
			if(Property == nullptr)
			{
				continue;
			}
			
			FDetailWidgetRow& PropertyWidgetRow = CategoryBuilder.AddCustomRow(FText::FromString(Property->GetName()));

			FName PropertyName = Property->GetFName();
			FText PropertyTypeText = GetPropertyTypeText(Property);

			FFormatNamedArguments Args;
			Args.Add(TEXT("PropertyName"), FText::FromName(PropertyName));
			Args.Add(TEXT("PropertyType"), PropertyTypeText);

			FText TooltipText = FText::Format(LOCTEXT("PropertyTooltipText", "{PropertyName}\nType: {PropertyType}"), Args);

			PropertyWidgetRow.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(Property->GetName()))
				.ToolTipText(TooltipText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];

			TSharedPtr<SWidget> BindingWidget = UAnimationGraphSchema::MakeBindingWidgetForPin({ this }, PropertyName, false, true);
			if(BindingWidget.IsValid())
			{
				PropertyWidgetRow.ExtensionContent()
				[
					BindingWidget.ToSharedRef()
				];
			}
		}
	}
}

bool UAnimGraphNode_LinkedAnimGraphBase::IsStructuralProperty(FProperty* InProperty) const
{
	return InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimNode_LinkedAnimGraph, InstanceClass);
}

FString UAnimGraphNode_LinkedAnimGraphBase::GetCurrentInstanceBlueprintPath() const
{
	UClass* InstanceClass = GetTargetClass();

	if(InstanceClass)
	{
		UBlueprint* ActualBlueprint = UBlueprint::GetBlueprintFromClass(InstanceClass);

		if(ActualBlueprint)
		{
			return ActualBlueprint->GetPathName();
		}
	}

	return FString();
}

bool UAnimGraphNode_LinkedAnimGraphBase::OnShouldFilterInstanceBlueprint(const FAssetData& AssetData) const
{
	// Check recursion
	if(AssetData.IsAssetLoaded() && Cast<UBlueprint>(AssetData.GetAsset()) == GetBlueprint())
	{
		return true;
	}

	// Check skeleton & flags
	FAssetDataTagMapSharedView::FFindTagResult TargetSkeletonResult = AssetData.TagsAndValues.FindTag("TargetSkeleton");
	FAssetDataTagMapSharedView::FFindTagResult IsTemplateResult = AssetData.TagsAndValues.FindTag("bIsTemplate");
	FAssetDataTagMapSharedView::FFindTagResult BlueprintTypeResult = AssetData.TagsAndValues.FindTag("BlueprintType");
	if (TargetSkeletonResult.IsSet())
	{
		const bool bIsTemplate = IsTemplateResult.IsSet() && IsTemplateResult.Equals(TEXT("True"));
		const bool bIsInterface = BlueprintTypeResult.IsSet() && BlueprintTypeResult.Equals(TEXT("BPTYPE_Interface"));
		if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
		{
			if (!CurrentBlueprint->IsCompatibleByAssetString(TargetSkeletonResult.GetValue(), bIsTemplate, bIsInterface))
			{
				return true;
			}
		}
	}

	return false;
}

void UAnimGraphNode_LinkedAnimGraphBase::OnSetInstanceBlueprint(const FAssetData& AssetData, IDetailLayoutBuilder* InDetailBuilder)
{
	FScopedTransaction Transaction(LOCTEXT("SetInstanceBlueprint", "Set Linked Blueprint"));

	Modify();
	
	TSharedRef<IPropertyHandle> ClassHandle = InDetailBuilder->GetProperty(TEXT("Node.InstanceClass"), GetClass());
	if(UAnimBlueprint* Blueprint = Cast<UAnimBlueprint>(AssetData.GetAsset()))
	{
		ClassHandle->SetValue(Blueprint->GetAnimBlueprintGeneratedClass());
	}
	else
	{
		ClassHandle->SetValue((UObject*)nullptr);
	}
}

FLinearColor UAnimGraphNode_LinkedAnimGraphBase::GetDefaultNodeTitleColor() const
{
	return LinkedAnimGraphGraphNodeConstants::TitleColor;
}

FPoseLinkMappingRecord UAnimGraphNode_LinkedAnimGraphBase::GetLinkIDLocation(const UScriptStruct* NodeType, UEdGraphPin* SourcePin)
{
	FPoseLinkMappingRecord Record = Super::GetLinkIDLocation(NodeType, SourcePin);
	if(Record.IsValid())
	{
		return Record;	
	}
	else if(SourcePin->LinkedTo.Num() > 0 && SourcePin->Direction == EGPD_Input)
	{
		const FAnimNode_LinkedAnimGraph& Node = *GetLinkedAnimGraphNode();

		check(Node.InputPoses.Num() == Node.InputPoseNames.Num());

		// perform name-based logic for input pose pins
		if (UAnimGraphNode_Base* LinkedNode = Cast<UAnimGraphNode_Base>(FBlueprintEditorUtils::FindFirstCompilerRelevantNode(SourcePin->LinkedTo[0])))
		{
			FArrayProperty* ArrayProperty = FindFieldChecked<FArrayProperty>(NodeType, GET_MEMBER_NAME_CHECKED(FAnimNode_LinkedAnimGraph, InputPoses));
			int32 ArrayIndex = INDEX_NONE;
			if(Node.InputPoseNames.Find(SourcePin->GetFName(), ArrayIndex))
			{
				check(Node.InputPoses.IsValidIndex(ArrayIndex));
				return FPoseLinkMappingRecord::MakeFromArrayEntry(this, LinkedNode, ArrayProperty, ArrayIndex);
			}
		}
	}

	return FPoseLinkMappingRecord::MakeInvalid();
}

void UAnimGraphNode_LinkedAnimGraphBase::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	// We have the potential to output ALL registered attributes as we can contain any dynamically-linked graph
	const UAnimGraphAttributes* AnimGraphAttributes = GetDefault<UAnimGraphAttributes>();
	AnimGraphAttributes->ForEachAttribute([&OutAttributes](const FAnimGraphAttributeDesc& InDesc)
	{
		OutAttributes.Add(InDesc.Name);
	});
}

void UAnimGraphNode_LinkedAnimGraphBase::OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	Super::OnCopyTermDefaultsToDefaultObject(InCompilationContext, InPerNodeContext, OutCompiledData);
	
	UAnimGraphNode_LinkedAnimGraphBase* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_LinkedAnimGraphBase>(this);

	FAnimNode_LinkedAnimGraph* DestinationNode = reinterpret_cast<FAnimNode_LinkedAnimGraph*>(InPerNodeContext.GetDestinationPtr());
	DestinationNode->NodeIndex = InPerNodeContext.GetNodePropertyIndex();
}

void UAnimGraphNode_LinkedAnimGraphBase::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	OutExtensions.Add(UAnimBlueprintExtension_LinkedAnimGraph::StaticClass());
}

TSharedPtr<FEdGraphSchemaAction> UAnimGraphNode_LinkedAnimGraphBase::GetEventNodeAction(const FText& ActionCategory)
{
	TSharedPtr<FEdGraphSchemaAction_K2Event> NodeAction = MakeShareable(new FEdGraphSchemaAction_K2Event(ActionCategory, GetNodeTitle(ENodeTitleType::ListView), GetTooltipText(), 0));
	NodeAction->NodeTemplate = this;
	return NodeAction;
}


void UAnimGraphNode_LinkedAnimGraphBase::IterateFunctionParameters(UFunction* InFunction, TFunctionRef<void(const FName&, const FEdGraphPinType&)> InFunc) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	auto IterateParam = [K2Schema, &InFunc](FProperty* InParam, bool bInPoses)
	{
		const bool bIsFunctionInput = !InParam->HasAnyPropertyFlags(CPF_OutParm) || InParam->HasAnyPropertyFlags(CPF_ReferenceParm);
		if (bIsFunctionInput)
		{
			FEdGraphPinType PinType;
			if(K2Schema->ConvertPropertyToPinType(InParam, PinType))
			{
				if(UAnimationGraphSchema::IsPosePin(PinType) == bInPoses)
				{
					InFunc(InParam->GetFName(), PinType);
				}
			}
		}
	};

	// Iterate poses first, then params
	for (TFieldIterator<FProperty> PropIt(InFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		IterateParam(*PropIt, true);
	}

	for (TFieldIterator<FProperty> PropIt(InFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		IterateParam(*PropIt, false);
	}
}

#undef LOCTEXT_NAMESPACE
