// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LinkedAnimLayer.h"
#include "Components/SkeletalMeshComponent.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "AnimationGraphSchema.h"
#include "Widgets/Layout/SBox.h"
#include "UObject/CoreRedirects.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "AnimationStateGraph.h"
#include "BlueprintNodeSpawner.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "ObjectEditorUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorClassUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LinkedAnimLayer)

#define LOCTEXT_NAMESPACE "LinkedAnimLayerNode"

namespace LinkedAnimLayerGraphNodeConstants
{
	FLinearColor TitleColorSelfLayer(0.2f, 0.07f, 0.6f);
	FLinearColor TitleColorLinkedLayer(0.45f, 0.f, 0.7f);
}


void UAnimGraphNode_LinkedAnimLayer::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimLayerGuidConformation)
		{
			if (!InterfaceGuid.IsValid())
			{
				InterfaceGuid = GetGuidForLayer();
			}
		}
	}
}

void UAnimGraphNode_LinkedAnimLayer::ReconstructNode()
{
	if(SetObjectBeingDebuggedHandle.IsValid())
	{
		GetBlueprint()->OnSetObjectBeingDebugged().Remove(SetObjectBeingDebuggedHandle);
	}

	SetObjectBeingDebuggedHandle = GetBlueprint()->OnSetObjectBeingDebugged().AddUObject(this, &UAnimGraphNode_LinkedAnimLayer::HandleSetObjectBeingDebugged);

	Super::ReconstructNode();
}

FSlateIcon UAnimGraphNode_LinkedAnimLayer::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimLayerInterface");
}

FText UAnimGraphNode_LinkedAnimLayer::GetTooltipText() const
{
	return LOCTEXT("ToolTip", "Runs another linked animation layer graph to process animation");
}

FAnimNode_LinkedAnimLayer* UAnimGraphNode_LinkedAnimLayer::GetPreviewNode() const
{
	FAnimNode_LinkedAnimLayer* PreviewNode = nullptr;
	USkeletalMeshComponent* Component = nullptr;

	// look for a valid component in the object being debugged,
	// we might be set to something other than the preview.
	UObject* ObjectBeingDebugged = GetAnimBlueprint()->GetObjectBeingDebugged();
	if (ObjectBeingDebugged)
	{
		UAnimInstance* InstanceBeingDebugged = Cast<UAnimInstance>(ObjectBeingDebugged);
		if (InstanceBeingDebugged)
		{
			Component = InstanceBeingDebugged->GetSkelMeshComponent();
		}
	}

	if (Component != nullptr && Component->GetAnimInstance() != nullptr)
	{
		PreviewNode = static_cast<FAnimNode_LinkedAnimLayer*>(FindDebugAnimNode(Component));
	}

	return PreviewNode;
}

FText UAnimGraphNode_LinkedAnimLayer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UClass* TargetClass = *Node.Interface;
	UAnimBlueprint* TargetAnimBlueprintInterface = TargetClass ? CastChecked<UAnimBlueprint>(TargetClass->ClassGeneratedBy) : nullptr;

	const FText DefaultNodeTitle = LOCTEXT("NodeTitle", "Linked Anim Layer");
	if(TitleType == ENodeTitleType::MenuTitle)
	{
		return DefaultNodeTitle;
	}
	else
	{
		bool bIsSelf = TargetAnimBlueprintInterface == nullptr; 
		
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeType"), bIsSelf ? LOCTEXT("SelfLayerNodeTitle", "Anim Layer (self)") : DefaultNodeTitle);
		Args.Add(TEXT("TargetClass"), bIsSelf ? LOCTEXT("ClassSelf", "Self") : FText::FromString(TargetAnimBlueprintInterface->GetName()));
		Args.Add(TEXT("Layer"), (GetLayerName() == NAME_None) ? LOCTEXT("LayerNone", "None") : FText::FromName(GetLayerName()));

		if (TitleType == ENodeTitleType::ListView)
		{
			if(bIsSelf)
			{
				return FText::Format(LOCTEXT("TitleListViewFormatSelf", "{Layer}"), Args);
			}
			else
			{
				return FText::Format(LOCTEXT("TitleListViewFormat", "{TargetClass} - {Layer}"), Args);
			}
		}
		else
		{
			if (FAnimNode_LinkedAnimLayer* PreviewNode = GetPreviewNode())
			{
				if (UAnimInstance* PreviewAnimInstance = PreviewNode->GetTargetInstance<UAnimInstance>())
				{
					if (UClass* PreviewTargetClass = PreviewAnimInstance->GetClass())
					{
						bIsSelf = PreviewTargetClass == GetAnimBlueprint()->GeneratedClass;
						Args.Add(TEXT("TargetClass"), PreviewTargetClass == GetAnimBlueprint()->GeneratedClass ? LOCTEXT("ClassSelf", "Self") : FText::FromName(PreviewTargetClass->GetFName()));
					}
				}
			}

			if(bIsSelf)
			{
				return FText::Format(LOCTEXT("TitleOtherFormatSelf", "{Layer}\n{NodeType}"), Args);
			}
			else
			{
				return FText::Format(LOCTEXT("TitleOtherFormat", "{TargetClass} - {Layer}\n{NodeType}"), Args);
			}
		}
	}
}

void UAnimGraphNode_LinkedAnimLayer::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if(GetLayerName() == NAME_None)
	{
		MessageLog.Error(*LOCTEXT("NoLayerError", "Linked anim layer node @@ does not specify a layer.").ToString(), this);
	}
	else
	{
		UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint());

		// check layer actually exists in the interface
		UClass* TargetClass = *Node.Interface;
		if(TargetClass == nullptr)
		{
			// If no interface specified, use this class
			if (CurrentBlueprint)
			{
				TargetClass = *CurrentBlueprint->SkeletonGeneratedClass;
			}
		}
		else
		{
			// check we implement this interface
			bool bImplementsInterface = false;

			if (CurrentBlueprint)
			{
				for (FBPInterfaceDescription& InterfaceDesc : CurrentBlueprint->ImplementedInterfaces)
				{
					if (InterfaceDesc.Interface.Get() == TargetClass)
					{
						bImplementsInterface = true;
						break;
					}
				}
			}

			if(!bImplementsInterface)
			{
				// Its possible we have a left-over interface referenced here that needs clearing now we are a 'self' layer
				if(GetInterfaceForLayer() == nullptr)
				{
					Node.Interface = nullptr;

					// No interface any more, use this class
					if (CurrentBlueprint)
					{
						TargetClass = *CurrentBlueprint->SkeletonGeneratedClass;
					}
				}
				else
				{
					MessageLog.Error(*LOCTEXT("MissingInterfaceError", "Linked anim layer node @@ uses interface @@ that this blueprint does not implement.").ToString(), this, Node.Interface.Get());
				}
			}
		}

		if(TargetClass)
		{
			bool bFoundFunction = false;
			IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
			for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
			{
				if(AnimBlueprintFunction.Name == GetLayerName())
				{
					bFoundFunction = true;
				}
			}

			if(!bFoundFunction)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("MissingLayerError", "Linked anim layer node @@ uses invalid layer '{0}'."), FText::FromName(GetLayerName())).ToString(), this);
			}
		}

		if(CurrentBlueprint)
		{
			UAnimGraphNode_LinkedAnimLayer* OriginalThis = Cast<UAnimGraphNode_LinkedAnimLayer>(MessageLog.FindSourceObject(this));

			TArray<UEdGraph*> Graphs;
			CurrentBlueprint->GetAllGraphs(Graphs);

			auto ValidateOuterGraph = [this, OriginalThis, &MessageLog](const UEdGraph* InGraph)
			{
				static const FName DefaultAnimGraphName("AnimGraph");
				if (!InGraph->IsA<UAnimationStateGraph>() && InGraph->GetFName() != DefaultAnimGraphName && InGraph->InterfaceGuid.IsValid() && OriginalThis->InterfaceGuid.IsValid())
				{
					MessageLog.Error(*FText::Format(LOCTEXT("NestedLayer", "Linked anim layer node @@ is part of Animation Layer Graph '{0}', linked layers cannot be nested."), FText::FromName(InGraph->GetFName())).ToString(), this);
				}
			};

			// Gather all linked anim layer node instances in this blueprint 
			TArray<UAnimGraphNode_LinkedAnimLayer*> AllLayerNodes;
			for (const UEdGraph* Graph : Graphs)
			{
				Graph->GetNodesOfClass(AllLayerNodes);
			}

			// Check for duplicates
			for (const UAnimGraphNode_LinkedAnimLayer* LayerNode : AllLayerNodes)
			{
				if (LayerNode != OriginalThis)
				{
					if (LayerNode->GetLayerName() == GetLayerName())
					{
						MessageLog.Error(*FText::Format(LOCTEXT("DuplicateLayerError", "Linked anim layer node @@ is also used by graph '{0}', layers can be used only once in an animation blueprint."), FText::FromName(LayerNode->GetGraph()->GetFName())).ToString(), this);
					}
				}
			}

			// Check for circular references and indirect nesting
			for (const UEdGraph* Graph : Graphs)
			{
				if (Graph->GetFName() == OriginalThis->Node.Layer)
				{
					TArray<const UEdGraph*> GraphStack;
					bool bWithinLinkedAnimLayer = OriginalThis->InterfaceGuid.IsValid();

					UEdGraph* OuterGraph = OriginalThis->GetGraph();
					// If our outer graph is a linked layer interface function that has no instance in the blueprint, we need to validate it for nesting as it can still be instanciated through the application of a linked layer
					if (OuterGraph->InterfaceGuid.IsValid())
					{
						bool bCheckOuter = false;
						for (const UAnimGraphNode_LinkedAnimLayer* LayerNode : AllLayerNodes)
						{
							if (LayerNode->Node.Layer == OuterGraph->GetFName())
							{
								bCheckOuter = false;
								break;
							}
						}

						// Check outer graph for nested linked layers
						if (bCheckOuter)
						{
							static const FName DefaultAnimGraphName("AnimGraph");
							if (OuterGraph->GetFName() != DefaultAnimGraphName)
							{
								ValidateOuterGraph(OuterGraph);

								// Add outer graph to graph stack
								GraphStack.Add(OuterGraph);
								// If outer graph is a linked layer interface, account for it to properly detect nesting
								bWithinLinkedAnimLayer = bWithinLinkedAnimLayer || OuterGraph->InterfaceGuid.IsValid();
							}
						}
					}
					
					ValidateCircularRefAndNesting(Graph, Graphs, GraphStack, bWithinLinkedAnimLayer, MessageLog);
				}
			}
		}
	}
}

void UAnimGraphNode_LinkedAnimLayer::ValidateCircularRefAndNesting(const UEdGraph* CurrentGraph, const TArray<UEdGraph*>& AllGraphs, TArray<const UEdGraph*> GraphStack, bool bWithinLinkedLayerGraph, FCompilerResultsLog& MessageLog)
{
	// Build graph chain string
	auto BuildGraphChainString = [](const TArray<const UEdGraph*>& GraphStack) -> FString
	{
		TStringBuilder<1024> GraphChain;
		bool bFirst = true;
		for (const UEdGraph* Graph : GraphStack)
		{
			if (!bFirst)
			{
				GraphChain << TEXT("->");
			}
			GraphChain << *Graph->GetName();
			bFirst = false;
		}
		return FString(GraphChain);
	};

	// If a graph is already in the stack we have a circular reference
	if (GraphStack.Contains(CurrentGraph))
	{
		// Add the redundant graph so the error is easier to understand
		GraphStack.Add(CurrentGraph);
		MessageLog.Error(*FText::Format(LOCTEXT("CircularLayerReference", "Anim layer node @@ in Graph '{0}' has a circular dependency '{1}'."), FText::FromName(Node.Layer), FText::FromString(BuildGraphChainString(GraphStack))).ToString(), this);
		return;
	}

	GraphStack.Add(CurrentGraph);

	// Find layer nodes and recursively check their graphs 
	TArray<UAnimGraphNode_LinkedAnimLayer*> LayerNodes;
	CurrentGraph->GetNodesOfClass(LayerNodes);
	for (const UAnimGraphNode_LinkedAnimLayer* LayerNode : LayerNodes)
	{
		// Check for linked anim layer nesting
		if (bWithinLinkedLayerGraph && LayerNode->InterfaceGuid.IsValid())
		{
			if (GraphStack.Num() == 1)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("NestedLayer", "Linked anim layer node @@ is part of Animation Layer Graph '{0}', linked layers cannot be nested."), FText::FromName(CurrentGraph->GetFName())).ToString(), LayerNode);
			}
			else
			{
				MessageLog.Error(*FText::Format(LOCTEXT("IndirectlyNestedLayer", "Linked anim layer node @@ is indirectly nested inside another Linked Anim Layer Graph '{0}', linked layers cannot be nested."), FText::FromString(BuildGraphChainString(GraphStack))).ToString(), LayerNode);
			}
		}
		for (const UEdGraph* Graph : AllGraphs)
		{
			if (Graph->GetFName() == LayerNode->Node.Layer)
			{
				ValidateCircularRefAndNesting(Graph, AllGraphs, GraphStack, bWithinLinkedLayerGraph || LayerNode->InterfaceGuid.IsValid(), MessageLog);
			}

		}
	}
};
void UAnimGraphNode_LinkedAnimLayer::GetLinkTarget(UObject*& OutTargetGraph, UBlueprint*& OutTargetBlueprint) const
{
	OutTargetGraph = nullptr;
	OutTargetBlueprint = nullptr;

	auto JumpTargetFromClass = [this](UClass* InClass, UObject*& OutTargetGraph, UBlueprint*& OutTargetBlueprint)
	{
		UAnimBlueprint* TargetAnimBlueprint = InClass ? CastChecked<UAnimBlueprint>(InClass->ClassGeneratedBy) : nullptr;

		while (TargetAnimBlueprint != nullptr)
		{
			// jump to graph in other BP, going up the parent BP hierarchy until we find it
			TArray<UEdGraph*> Graphs;
			TargetAnimBlueprint->GetAllGraphs(Graphs);

			UEdGraph** FoundGraph = Graphs.FindByPredicate([this](UEdGraph* InGraph) { return InGraph->GetFName() == GetLayerName(); });
			if (FoundGraph)
			{
				OutTargetBlueprint = TargetAnimBlueprint;
				OutTargetGraph = *FoundGraph;
				return;
			}
			else
			{
				TargetAnimBlueprint = UAnimBlueprint::GetParentAnimBlueprint(TargetAnimBlueprint);
			}
		}

		// jump to graph in self
		TArray<UEdGraph*> Graphs;
		GetBlueprint()->GetAllGraphs(Graphs);

		UEdGraph** FoundGraph = Graphs.FindByPredicate([this](UEdGraph* InGraph) { return InGraph->GetFName() == GetLayerName(); });
		if (FoundGraph)
		{
			OutTargetBlueprint = nullptr;
			OutTargetGraph = *FoundGraph;
			return;
		}
	};

	// First try a concrete class, if any

	const FAnimNode_LinkedAnimGraph* RuntimeNode = GetLinkedAnimGraphNode();

	if (UObject* TargetInstance = RuntimeNode->GetTargetInstance<UObject>())
	{
		JumpTargetFromClass(TargetInstance->GetClass(), OutTargetGraph, OutTargetBlueprint);
	}
	if (OutTargetGraph == nullptr)
	{
		JumpTargetFromClass(RuntimeNode->InstanceClass, OutTargetGraph, OutTargetBlueprint);
	}
	if (OutTargetGraph == nullptr)
	{
		// then try the interface
		JumpTargetFromClass(*Node.Interface, OutTargetGraph, OutTargetBlueprint);
	}

}


UObject* UAnimGraphNode_LinkedAnimLayer::GetJumpTargetForDoubleClick() const
{
	UObject* TargetGraph;
	UBlueprint* TargetBlueprint;

	GetLinkTarget(TargetGraph, TargetBlueprint);
	return TargetGraph;
}

void UAnimGraphNode_LinkedAnimLayer::JumpToDefinition() const
{

	UObject* TargetGraph;
	UBlueprint* TargetBlueprint;

	GetLinkTarget(TargetGraph, TargetBlueprint);

	if (UAnimationGraph* HyperlinkTarget = Cast<UAnimationGraph>(TargetGraph))
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(HyperlinkTarget);

		if (TargetBlueprint != nullptr)
		{
			const FAnimNode_LinkedAnimGraph* RuntimeNode = GetLinkedAnimGraphNode();

			if (UObject* TargetInstance = RuntimeNode->GetTargetInstance<UObject>())
			{
				TargetBlueprint->SetObjectBeingDebugged(TargetInstance);
			}
		}
	}
	else
	{
		Super::JumpToDefinition();
	}
}

bool UAnimGraphNode_LinkedAnimLayer::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput /*= NULL*/) const
{
	UClass* InterfaceClassToUse = *Node.Interface;

	// Add our interface class. If that changes we need a recompile
	if(InterfaceClassToUse && OptionalOutput)
	{
		OptionalOutput->AddUnique(InterfaceClassToUse);
	}

	bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return InterfaceClassToUse || bSuperResult;
}

void UAnimGraphNode_LinkedAnimLayer::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// We dont allow multi-select here
	if(DetailBuilder.GetSelectedObjects().Num() > 1)
	{
		DetailBuilder.HideCategory(TEXT("Settings"));
		return;
	}

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Settings"));

	// Hide Tag
	TSharedRef<IPropertyHandle> TagHandle = DetailBuilder.GetProperty(TEXT("Node.Tag"), GetClass());
	TagHandle->MarkHiddenByCustomization();

	// Customize Layer
	{
		TSharedRef<IPropertyHandle> LayerHandle = DetailBuilder.GetProperty(TEXT("Node.Layer"), GetClass());
		if(LayerHandle->IsValidHandle())
		{
			LayerHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateUObject(this, &UAnimGraphNode_LinkedAnimLayer::OnLayerChanged, &DetailBuilder));
		}

		LayerHandle->MarkHiddenByCustomization();

		// Check layers available in this BP
		FDetailWidgetRow& LayerWidgetRow = CategoryBuilder.AddCustomRow(LOCTEXT("FilterStringLayer", "Layer"));
		LayerWidgetRow.NameContent()
		[
			LayerHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(150.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Visibility_Lambda([this](){ return HasAvailableLayers() ? EVisibility::Visible : EVisibility::Collapsed; })
				[
					PropertyCustomizationHelpers::MakePropertyComboBox(
						LayerHandle, 
						FOnGetPropertyComboBoxStrings::CreateUObject(this,  &UAnimGraphNode_LinkedAnimLayer::GetLayerNames),
						FOnGetPropertyComboBoxValue::CreateUObject(this,  &UAnimGraphNode_LinkedAnimLayer::GetLayerNameString)
					)
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility_Lambda([this](){ return !HasAvailableLayers() ? EVisibility::Visible : EVisibility::Collapsed; })
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("NoLayersWarning", "No available layers."))
				.ToolTipText(LOCTEXT("NoLayersWarningTooltip", "This Animation Blueprint has no layers to choose from.\nTo add some, either implement an Animation Layer Interface via the Class Settings, or add an animation layer in the My Blueprint tab."))
			]
		];
	}

	GenerateExposedPinsDetails(DetailBuilder);
	UAnimGraphNode_CustomProperty::CustomizeDetails(DetailBuilder);

	// Customize InstanceClass with unique visibility (identical to parent class apart from this)
	{
		TSharedRef<IPropertyHandle> ClassHandle = DetailBuilder.GetProperty(TEXT("Node.InstanceClass"), GetClass());
		ClassHandle->MarkHiddenByCustomization();

		FDetailWidgetRow& ClassWidgetRow = CategoryBuilder.AddCustomRow(LOCTEXT("FilterStringInstanceClass", "Instance Class"));
		ClassWidgetRow.NameContent()
		[
			ClassHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SObjectPropertyEntryBox)
				.Visibility_Lambda([this](){ return HasValidNonSelfLayer() ? EVisibility::Visible : EVisibility::Collapsed; })
				.ObjectPath_UObject(this, &UAnimGraphNode_LinkedAnimLayer::GetCurrentInstanceBlueprintPath)
				.AllowedClass(UAnimBlueprint::StaticClass())
				.NewAssetFactories(TArray<UFactory*>())
				.OnShouldFilterAsset(FOnShouldFilterAsset::CreateUObject(this, &UAnimGraphNode_LinkedAnimLayer::OnShouldFilterInstanceBlueprint))
				.OnObjectChanged(FOnSetObject::CreateUObject(this, &UAnimGraphNode_LinkedAnimLayer::OnSetInstanceBlueprint, &DetailBuilder))
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility_Lambda([this](){ return !HasValidNonSelfLayer() ? EVisibility::Visible : EVisibility::Collapsed; })
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("SelfLayersWarning", "Uses layer in this Blueprint."))
				.ToolTipText(LOCTEXT("SelfLayersWarningTooltip", "This linked anim layer node refers to a layer only in this blueprint, so cannot be overriden by an external blueprint implementation.\nChange to use a layer from an implemented interface to allow this override."))
			]
		];
	}
}

bool UAnimGraphNode_LinkedAnimLayer::OnShouldFilterInstanceBlueprint(const FAssetData& InAssetData) const
{
	if(Super::OnShouldFilterInstanceBlueprint(InAssetData))
	{
		return true;
	}

	if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
	{
		TArray<TSubclassOf<UInterface>> AnimInterfaces;
		for(const FBPInterfaceDescription& InterfaceDesc : CurrentBlueprint->ImplementedInterfaces)
		{
			if(InterfaceDesc.Interface && InterfaceDesc.Interface->IsChildOf<UAnimLayerInterface>())
			{
				if(GetLayerName() == NAME_None || InterfaceDesc.Interface->FindFunctionByName(GetLayerName()))
				{
					AnimInterfaces.Add(InterfaceDesc.Interface);
				}
			}
		}

		// Check interface compatibility
		if(AnimInterfaces.Num() > 0)
		{
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			FAssetData CurrentAssetData = InAssetData;
			bool bMatchesInterface = false;

			do 
			{
				TArray<FString> InterfacePaths;
				FEditorClassUtils::GetImplementedInterfaceClassPathsFromAsset(CurrentAssetData, InterfacePaths);

				for (const FString& InterfacePath : InterfacePaths)
				{
					FTopLevelAssetPath AssetPath(InterfacePath);
					FCoreRedirectObjectName ResolvedInterfaceName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(AssetPath));

					// Verify against all interfaces we currently implement
					for (TSubclassOf<UInterface> AnimInterface : AnimInterfaces)
					{
						bMatchesInterface |= ResolvedInterfaceName.ObjectName == AnimInterface->GetFName();
					}
				}

				// If we didn't find a matching interface, check the parent class
				if (!bMatchesInterface)
				{
					const FString ParentClassFromData = CurrentAssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
					const FString ClassObjectPath = FPackageName::ExportTextPathToObjectPath(ParentClassFromData);
					const FString BlueprintPath = ClassObjectPath.LeftChop(2); // Chop off _C
					CurrentAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(BlueprintPath));
				}
			// Only continue checking if the parent is an anim blueprint
			} while (!bMatchesInterface && (CurrentAssetData.AssetClassPath == UAnimBlueprint::StaticClass()->GetClassPathName()));

			if(!bMatchesInterface)
			{
				return true;
			}
		}
		else
		{
			// No interfaces, so no compatible BPs
			return true;
		}
	}

	return false;
}

FString UAnimGraphNode_LinkedAnimLayer::GetCurrentInstanceBlueprintPath() const
{
	UClass* InterfaceClass = *Node.InstanceClass;

	if(InterfaceClass)
	{
		UBlueprint* ActualBlueprint = UBlueprint::GetBlueprintFromClass(InterfaceClass);

		if(ActualBlueprint)
		{
			return ActualBlueprint->GetPathName();
		}
	}

	return FString();
}

void UAnimGraphNode_LinkedAnimLayer::CreateCustomPins(TArray<UEdGraphPin*>* OldPins)
{
	if(UClass* TargetSkeletonClass = GetTargetSkeletonClass())
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		
		TArray<FOptionalPinFromProperty> OldCustomPinProperties = CustomPinProperties;
		CustomPinProperties.Empty();
		
		// add only sub-input properties
		IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetSkeletonClass);
		for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
		{
			// Check name matches.
			if(AnimBlueprintFunction.Name == Node.GetDynamicLinkFunctionName())
			{
				for(const FAnimBlueprintFunction::FInputPropertyData& PropertyData : AnimBlueprintFunction.InputPropertyData)
				{
					// Use function property here as during compilation (especially compile-on-load) the class property may not be available
					if(FProperty* Property = PropertyData.FunctionProperty)
					{
						const FName PinName = Property->GetFName();
						
						FOptionalPinFromProperty OptionalPin;
						OptionalPin.PropertyName = PinName;
						OptionalPin.PropertyFriendlyName = UEditorEngine::GetFriendlyName(Property);
						OptionalPin.bShowPin = OldCustomPinProperties.ContainsByPredicate([PinName](const FOptionalPinFromProperty& InOptionalPin){ return InOptionalPin.bShowPin && InOptionalPin.PropertyName == PinName; });
						OptionalPin.PropertyTooltip = Property->GetToolTipText();
						OptionalPin.CategoryName = FObjectEditorUtils::GetCategoryFName(Property);
						OptionalPin.bCanToggleVisibility = true;
						OptionalPin.bIsOverrideEnabled = false;

						CustomPinProperties.Add(OptionalPin);

						if(OptionalPin.bShowPin)
						{
							FEdGraphPinType PinType;
							if (K2Schema->ConvertPropertyToPinType(Property, /*out*/ PinType))
							{
								UEdGraphPin* NewPin = CreatePin(EGPD_Input, PinType, PinName);
								NewPin->PinFriendlyName = FText::FromString(OptionalPin.PropertyFriendlyName.IsEmpty() ? PinName.ToString() : OptionalPin.PropertyFriendlyName);
								K2Schema->ConstructBasicPinTooltip(*NewPin, OptionalPin.PropertyTooltip, NewPin->PinToolTip);
								K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(NewPin);
							}
						}
					}
				}
			}
		}
	}
}

FProperty* UAnimGraphNode_LinkedAnimLayer::GetPinProperty(FName InPinName) const
{
	if(UClass* TargetSkeletonClass = GetTargetSkeletonClass())
	{
		// add only sub-input properties
		IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetSkeletonClass);
		for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
		{
			// Check name matches.
			if(AnimBlueprintFunction.Name == Node.GetDynamicLinkFunctionName())
			{
				for(const FAnimBlueprintFunction::FInputPropertyData& PropertyData : AnimBlueprintFunction.InputPropertyData)
				{
					FProperty* Property = PropertyData.FunctionProperty;
					if(Property && Property->GetFName() == InPinName)
					{
						return Property;
					}
				}
			}
		}
	}

	return Super::GetPinProperty(InPinName);
}

void UAnimGraphNode_LinkedAnimLayer::GetLayerNames(TArray<TSharedPtr<FString>>& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems)
{
	// If no interface specified, use this class
	if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
	{
		UClass* TargetClass = *CurrentBlueprint->SkeletonGeneratedClass;
		if(TargetClass)
		{
			IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
			for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
			{
				if(AnimBlueprintFunction.Name != UEdGraphSchema_K2::GN_AnimGraph)
				{
					OutStrings.Add(MakeShared<FString>(AnimBlueprintFunction.Name.ToString()));
					OutToolTips.Add(nullptr);
					OutRestrictedItems.Add(false);
				}
			}
		}
	}
}

FString UAnimGraphNode_LinkedAnimLayer::GetLayerNameString() const
{
	return FunctionReference.GetMemberName().ToString();
}

bool UAnimGraphNode_LinkedAnimLayer::IsStructuralProperty(FProperty* InProperty) const
{
	return Super::IsStructuralProperty(InProperty) ||
		InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimNode_LinkedAnimLayer, Layer);
}

FLinearColor UAnimGraphNode_LinkedAnimLayer::GetDefaultNodeTitleColor() const
{
	if (HasValidNonSelfLayer())
	{
		return LinkedAnimLayerGraphNodeConstants::TitleColorLinkedLayer;		
	}
	else
	{
		return LinkedAnimLayerGraphNodeConstants::TitleColorSelfLayer;
	}
}

UClass* UAnimGraphNode_LinkedAnimLayer::GetTargetSkeletonClass() const
{
	UClass* SuperTargetSkeletonClass = Super::GetTargetSkeletonClass();
	if(SuperTargetSkeletonClass == nullptr)
	{
		// If no concrete class specified, use this class
		if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
		{
			SuperTargetSkeletonClass = *CurrentBlueprint->SkeletonGeneratedClass;
		}
	}
	return SuperTargetSkeletonClass;
}

TSubclassOf<UInterface> UAnimGraphNode_LinkedAnimLayer::GetInterfaceForLayer() const
{
	if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
	{
		// Find layer with this name in interfaces
		for(FBPInterfaceDescription& InterfaceDesc : CurrentBlueprint->ImplementedInterfaces)
		{
			for(UEdGraph* InterfaceGraph : InterfaceDesc.Graphs)
			{
				if(InterfaceGraph->GetFName() == GetLayerName())
				{
					return InterfaceDesc.Interface;
				}
			}
		}
	}

	return nullptr;
}

void UAnimGraphNode_LinkedAnimLayer::UpdateGuidForLayer()
{
	if (!InterfaceGuid.IsValid())
	{
		InterfaceGuid = GetGuidForLayer();
	}
}

void UAnimGraphNode_LinkedAnimLayer::SetLayerName(FName InName)
{
	Node.Layer = InName;

	if(UClass* TargetClass = GetTargetClass())
	{
		FGuid FunctionGuid;
		FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(TargetClass), InName, FunctionGuid);
		FunctionReference.SetExternalMember(InName, TargetClass, FunctionGuid);
	}
	else
	{
		FunctionReference.SetSelfMember(InName);
	}
}

FName UAnimGraphNode_LinkedAnimLayer::GetLayerName() const
{
	ensure(FunctionReference.GetMemberName() == Node.Layer);
	return FunctionReference.GetMemberName();
}

FGuid UAnimGraphNode_LinkedAnimLayer::GetGuidForLayer() const
{
	if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
	{
		// Find layer with this name in interfaces
		for (FBPInterfaceDescription& InterfaceDesc : CurrentBlueprint->ImplementedInterfaces)
		{
			for (UEdGraph* InterfaceGraph : InterfaceDesc.Graphs)
			{
				if (InterfaceGraph->GetFName() == GetLayerName())
				{
					return InterfaceGraph->InterfaceGuid;
				}
			}
		}
	}

	return FGuid();
}

void UAnimGraphNode_LinkedAnimLayer::OnLayerChanged(IDetailLayoutBuilder* DetailBuilder)
{
	OnStructuralPropertyChanged(DetailBuilder);

	UClass* TargetClass = GetTargetClass();
	if(TargetClass)
	{
		FGuid FunctionGuid;
		FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(TargetClass), Node.Layer, FunctionGuid);	
		FunctionReference.SetExternalMember(Node.Layer, TargetClass, FunctionGuid);
	}
	else
	{
		FunctionReference.SetSelfMember(Node.Layer);
	}
	
	// Get the interface for this layer. If null, then we are using a 'self' layer.
	Node.Interface = GetInterfaceForLayer();

	// Update the Guid for conforming
	InterfaceGuid = GetGuidForLayer();

	if(Node.Interface.Get() == nullptr)
	{
		// Self layers cannot have override implementations
		Node.InstanceClass = nullptr;
	}
}

bool UAnimGraphNode_LinkedAnimLayer::HasAvailableLayers() const
{
	if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
	{
		UClass* TargetClass = *CurrentBlueprint->SkeletonGeneratedClass;
		if(TargetClass)
		{
			IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
			for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
			{
				if(AnimBlueprintFunction.Name != UEdGraphSchema_K2::GN_AnimGraph)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool UAnimGraphNode_LinkedAnimLayer::HasValidNonSelfLayer() const
{
	if (UAnimBlueprint* CurrentBlueprint = Cast<UAnimBlueprint>(GetBlueprint()))
	{
		if(Node.Interface.Get())
		{
			for(const FBPInterfaceDescription& InterfaceDesc : CurrentBlueprint->ImplementedInterfaces)
			{
				if(InterfaceDesc.Interface && InterfaceDesc.Interface->IsChildOf<UAnimLayerInterface>())
				{
					if(InterfaceDesc.Interface->FindFunctionByName(GetLayerName()))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void UAnimGraphNode_LinkedAnimLayer::HandleSetObjectBeingDebugged(UObject* InDebugObj)
{
	if(HasValidBlueprint())
	{
		NodeTitleChangedEvent.Broadcast();

		FAnimNode_LinkedAnimLayer* PreviewNode = GetPreviewNode();
		if(PreviewNode)
		{
			PreviewNode->OnInstanceChanged().AddUObject(this, &UAnimGraphNode_LinkedAnimLayer::HandleInstanceChanged);
		}
	}
}

void UAnimGraphNode_LinkedAnimLayer::HandleInstanceChanged()
{
	NodeTitleChangedEvent.Broadcast();
}

void UAnimGraphNode_LinkedAnimLayer::SetupFromLayerId(FName InLayer)
{
	Node.Layer = InLayer;

	// Set to self member first, so we have a valid name in the member reference (otherwise GetInterfaceForLayer will fail)
	// We will override this below
	FunctionReference.SetSelfMember(InLayer);

	// Get the interface for this layer. If null, then we are using a 'self' layer.
	Node.Interface = GetInterfaceForLayer();

	// Update the Guid for conforming
	InterfaceGuid = GetGuidForLayer();

	if(Node.Interface.Get() == nullptr)
	{
		// Self layers cannot have override implementations
		Node.InstanceClass = nullptr;
	}

	// Set up function reference
	UClass* TargetClass = GetTargetClass();
	if(TargetClass)
	{
		FGuid FunctionGuid;
		FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(TargetClass), InLayer, FunctionGuid);
		FunctionReference.SetExternalMember(InLayer, TargetClass, FunctionGuid);
	}
	else
	{
		FunctionReference.SetSelfMember(InLayer);
	}
}

void UAnimGraphNode_LinkedAnimLayer::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Anim graph node base class will allow us to spawn an 'empty' node
	UAnimGraphNode_Base::GetMenuActions(ActionRegistrar);
	
	auto MakeAnimBlueprintAction = [](TSubclassOf<UEdGraphNode> const NodeClass, const FName& InLayerId)
	{
		auto SetNodeLayerId = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FName InLayerId)
		{
			UAnimGraphNode_LinkedAnimLayer* LinkedAnimGraphNode = CastChecked<UAnimGraphNode_LinkedAnimLayer>(NewNode);
			LinkedAnimGraphNode->SetupFromLayerId(InLayerId);
		};
		
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(NodeClass);
		check(NodeSpawner != nullptr);
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(SetNodeLayerId, InLayerId);
		NodeSpawner->DefaultMenuSignature.Category = LOCTEXT("LinkedAnimLayerCategory", "Linked Anim Layers");
		NodeSpawner->DefaultMenuSignature.MenuName = NodeSpawner->DefaultMenuSignature.Tooltip = FText::Format(LOCTEXT("LinkedAnimGraphMenuFormat", "{0} - Linked Anim Layer"), FText::FromName(InLayerId));
		return NodeSpawner;
	};

	if (const UObject* RegistrarTarget = ActionRegistrar.GetActionKeyFilter())
	{
		if (const UAnimBlueprint* TargetAnimBlueprint = Cast<UAnimBlueprint>(RegistrarTarget))
		{
			UClass* TargetClass = *TargetAnimBlueprint->SkeletonGeneratedClass;
			if(TargetClass)
			{
				// Accept interfaces
				if(TargetAnimBlueprint->BlueprintType == BPTYPE_Interface)
				{
					IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
					for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
					{
						if(AnimBlueprintFunction.Name != UEdGraphSchema_K2::GN_AnimGraph)
						{
							if(UFunction* Function = TargetClass->FindFunctionByName(AnimBlueprintFunction.Name))
							{
								if (UBlueprintNodeSpawner* NodeSpawner = MakeAnimBlueprintAction(GetClass(), AnimBlueprintFunction.Name))
								{
									ActionRegistrar.AddBlueprintAction(Function, NodeSpawner);
								}
							}
						}
					}
				}
				else
				{
					// Accept 'self' layers
					IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
					for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
					{
						if(AnimBlueprintFunction.Name != UEdGraphSchema_K2::GN_AnimGraph)
						{
							const bool bIsSelfLayer = [TargetAnimBlueprint, &AnimBlueprintFunction]()
							{
								for(const FBPInterfaceDescription& InterfaceDesc : TargetAnimBlueprint->ImplementedInterfaces)
								{
									for(UEdGraph* InterfaceGraph : InterfaceDesc.Graphs)
									{
										if(InterfaceGraph->GetFName() == AnimBlueprintFunction.Name)
										{
											return false;
										}
									}
								}

								return true;
							}();

							if(bIsSelfLayer)
							{
								if(UFunction* Function = TargetClass->FindFunctionByName(AnimBlueprintFunction.Name))
								{
									if (UBlueprintNodeSpawner* NodeSpawner = MakeAnimBlueprintAction(GetClass(), AnimBlueprintFunction.Name))
									{
										ActionRegistrar.AddBlueprintAction(Function, NodeSpawner);
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

bool UAnimGraphNode_LinkedAnimLayer::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;

	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UBlueprint* Blueprint : FilterContext.Blueprints)
	{
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
		{
			if(UClass* TargetClass = *AnimBlueprint->SkeletonGeneratedClass)
			{
				// Accept only functions contained in this BP
				bool bImplemented = false;
				IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
				for(const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
				{
					if(GetLayerName() == AnimBlueprintFunction.Name)
					{
						bImplemented = true;
						break;
					}
				}

				if(!bImplemented)
				{
					bIsFilteredOut = true;
				}
			}
		}
	}
	
	return bIsFilteredOut;
}

FAnimNode_LinkedAnimGraph* UAnimGraphNode_LinkedAnimLayer::GetLinkedAnimGraphNode()
{
	FAnimNode_LinkedAnimLayer* const RuntimeLinkedAnimGraphNode = GetDebuggedAnimNode<FAnimNode_LinkedAnimLayer>();
	return RuntimeLinkedAnimGraphNode ? RuntimeLinkedAnimGraphNode : &Node;
}

const FAnimNode_LinkedAnimGraph* UAnimGraphNode_LinkedAnimLayer::GetLinkedAnimGraphNode() const
{
	const FAnimNode_LinkedAnimGraph* const RuntimeLinkedAnimGraphNode = GetDebuggedAnimNode<FAnimNode_LinkedAnimLayer>();
	return RuntimeLinkedAnimGraphNode ? RuntimeLinkedAnimGraphNode : &Node;
}

void UAnimGraphNode_LinkedAnimLayer::HandleFunctionReferenceChanged(FName InNewName)
{
	Node.Layer = InNewName;
	ensure(FunctionReference.GetMemberName() == Node.Layer);
}

#undef LOCTEXT_NAMESPACE
