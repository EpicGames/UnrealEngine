// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LinkedInputPose.h"
#include "AnimBlueprintExtension_LinkedInputPose.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "GraphEditorSettings.h"
#include "BlueprintActionFilter.h"
#include "Widgets/Input/SButton.h"
#include "DetailCategoryBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "Styling/AppStyle.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Stats/Stats.h"

#include "Animation/AnimBlueprint.h"
#include "IAnimationBlueprintEditor.h"
#include "AnimationGraphSchema.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "K2Node_CallFunction.h"
#include "Containers/Ticker.h"
#include "Widgets/Input/SCheckBox.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "KismetCompiler.h"
#include "K2Node_VariableGet.h"
#include "AnimBlueprintCompiler.h"
#include "AnimGraphAttributes.h"
#include "AnimGraphNode_LinkedAnimLayer.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LinkedInputPose)

#define LOCTEXT_NAMESPACE "LinkedInputPose"

UAnimGraphNode_LinkedInputPose::UAnimGraphNode_LinkedInputPose()
	: InputPoseIndex(INDEX_NONE)
{
}

void UAnimGraphNode_LinkedInputPose::CreateClassVariablesFromBlueprint(IAnimBlueprintVariableCreationContext& InCreationContext)
{
	IterateFunctionParameters([this, &InCreationContext](const FName& InName, FEdGraphPinType InPinType)
	{
		if(!UAnimationGraphSchema::IsPosePin(InPinType))
		{
			UEdGraphPin* Pin = FindPin(InName, EGPD_Output);
			if(Pin && Pin->LinkedTo.Num() > 0)
			{
				// create properties for 'local' linked input pose pins
				FProperty* NewLinkedInputPoseProperty = InCreationContext.CreateVariable(InName, InPinType);
				if(NewLinkedInputPoseProperty)
				{
					NewLinkedInputPoseProperty->SetPropertyFlags(CPF_BlueprintReadOnly);
				}
			}
		}
	});
}

void UAnimGraphNode_LinkedInputPose::ExpandNode(class FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	IterateFunctionParameters([this, &InCompilerContext](const FName& InName, FEdGraphPinType InPinType)
	{
		if(!UAnimationGraphSchema::IsPosePin(InPinType))
		{
			if(InCompilerContext.bIsFullCompile)
			{
				// Find the property we created in CreateClassVariablesFromBlueprint()
				FProperty* LinkedInputPoseProperty = FindFProperty<FProperty>(InCompilerContext.NewClass, InName);
				if(LinkedInputPoseProperty)
				{
					UEdGraphPin* Pin = FindPin(InName, EGPD_Output);
					if(Pin && Pin->LinkedTo.Num() > 0)
					{
						// Create new node for property access
						UK2Node_VariableGet* VariableGetNode = InCompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(this, GetGraph());
						VariableGetNode->SetFromProperty(LinkedInputPoseProperty, true, LinkedInputPoseProperty->GetOwnerClass());
						VariableGetNode->AllocateDefaultPins();

						// Add pin to generated variable association, used for pin watching
						UEdGraphPin* TrueSourcePin = InCompilerContext.MessageLog.FindSourcePin(Pin);
						if (TrueSourcePin)
						{
							InCompilerContext.NewClass->GetDebugData().RegisterClassPropertyAssociation(TrueSourcePin, LinkedInputPoseProperty);
						}

						// link up to new node - note that this is not a FindPinChecked because if an interface changes without the
						// implementing class being loaded, then its graphs will not be conformed until AFTER the skeleton class
						// has been compiled, so the variable cannot be created. This also doesnt matter, as there wont be anything connected
						// to the pin yet anyways.
						UEdGraphPin* VariablePin = VariableGetNode->FindPin(LinkedInputPoseProperty->GetFName());
						if(VariablePin)
						{
							TArray<UEdGraphPin*> Links = Pin->LinkedTo;
							Pin->BreakAllPinLinks();

							for(UEdGraphPin* LinkPin : Links)
							{
								VariablePin->MakeLinkTo(LinkPin);
							}
						}
					}
				}
			}
		}
	});
}

void UAnimGraphNode_LinkedInputPose::ReconstructLayerNodes(UBlueprint* InBlueprint)
{
	if(InBlueprint)
	{
		TArray<UAnimGraphNode_LinkedAnimLayer*> LinkedAnimLayers;
		FBlueprintEditorUtils::GetAllNodesOfClass<UAnimGraphNode_LinkedAnimLayer>(InBlueprint, LinkedAnimLayers);

		for(UAnimGraphNode_LinkedAnimLayer* LinkedAnimLayer : LinkedAnimLayers)
		{
			// Only reconstruct 'self' nodes in this manner - external layers will be rebuilt via the compilation machinery
			if(LinkedAnimLayer->Node.Interface.Get() == nullptr)
			{
				LinkedAnimLayer->ReconstructNode();
			}
		}
	}
}

void UAnimGraphNode_LinkedInputPose::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property)
	{
		if( PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_LinkedInputPose, Inputs) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_LinkedInputPose, Node.Name) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimBlueprintFunctionPinInfo, Name) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimBlueprintFunctionPinInfo, Type))
		{
			HandleInputPinArrayChanged();
			ReconstructNode();
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
			ReconstructNode();
			ReconstructLayerNodes(GetAnimBlueprint());
		}
	}
}

FLinearColor UAnimGraphNode_LinkedInputPose::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText UAnimGraphNode_LinkedInputPose::GetTooltipText() const
{
	return LOCTEXT("ToolTip", "Inputs to a sub-animation graph from a parent instance.");
}

FText UAnimGraphNode_LinkedInputPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText DefaultTitle = LOCTEXT("Title", "Input Pose");

	if(TitleType != ENodeTitleType::FullTitle)
	{
		return DefaultTitle;
	}
	else
	{
		if(Node.Name != NAME_None)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("NodeTitle"), DefaultTitle);
			Args.Add(TEXT("Name"), FText::FromName(Node.Name));
			return FText::Format(LOCTEXT("TitleListFormatTagged", "{NodeTitle}\n{Name}"), Args);
		}
		else
		{
			return DefaultTitle;
		}
	}
}

FText UAnimGraphNode_LinkedInputPose::GetMenuCategory() const
{
	return LOCTEXT("LinkedAnimGraphCategory", "Animation|Linked Anim Graphs");
}

bool UAnimGraphNode_LinkedInputPose::CanUserDeleteNode() const
{
	// Only allow linked input poses to be deleted if their parent graph is mutable
	// Also allow anim graphs to delete these nodes even theough they are 'read-only'
	return GetGraph()->bAllowDeletion || GetGraph()->GetFName() == UEdGraphSchema_K2::GN_AnimGraph;
}

bool UAnimGraphNode_LinkedInputPose::CanDuplicateNode() const
{
	return false;
}

template <typename Predicate>
static FName CreateUniqueName(const FName& InBaseName, Predicate IsUnique)
{
	FName CurrentName = InBaseName;
	int32 CurrentIndex = 0;

	while (!IsUnique(CurrentName))
	{
		FString PossibleName = InBaseName.ToString() + TEXT("_") + FString::FromInt(CurrentIndex++);
		CurrentName = FName(*PossibleName);
	}

	return CurrentName;
}

void UAnimGraphNode_LinkedInputPose::HandleInputPinArrayChanged()
{
	TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();

	for(UEdGraph* Graph : AnimBlueprint->FunctionGraphs)
	{
		if(Graph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()))
		{
			// Create a unique name for this new linked input pose
			Graph->GetNodesOfClass(LinkedInputPoseNodes);
		}
	}

	for(FAnimBlueprintFunctionPinInfo& Input : Inputs)
	{
		// New names are created empty, so assign a unique name
		if(Input.Name == NAME_None)
		{
			Input.Name = CreateUniqueName(TEXT("InputParam"), [&LinkedInputPoseNodes](FName InName)
			{
				for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode : LinkedInputPoseNodes)
				{
					for(const FAnimBlueprintFunctionPinInfo& Input : LinkedInputPoseNode->Inputs)
					{
						if(Input.Name == InName)
						{
							return false;
						}
					}
				}
				return true;
			});

			if(Input.Type.PinCategory == NAME_None)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, false);
				check(AssetEditor->GetEditorName() == "AnimationBlueprintEditor");
				IAnimationBlueprintEditor* AnimationBlueprintEditor = static_cast<IAnimationBlueprintEditor*>(AssetEditor);
				Input.Type = AnimationBlueprintEditor->GetLastGraphPinTypeUsed();
			}
		}
	}
	
	bool bIsInterface = AnimBlueprint->BlueprintType == BPTYPE_Interface;
	if(bIsInterface)
	{
		UAnimationGraphSchema::AutoArrangeInterfaceGraph(*GetGraph());
	}
}

void UAnimGraphNode_LinkedInputPose::AllocatePinsInternal()
{
	// use member reference if valid
	if (UFunction* Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		CreatePinsFromStubFunction(Function);
	}

	if(IsEditable())
	{
		// use user-defined pins
		CreateUserDefinedPins();
	}
}

void UAnimGraphNode_LinkedInputPose::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	AllocatePinsInternal();
}

void UAnimGraphNode_LinkedInputPose::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	AllocatePinsInternal();

	RestoreSplitPins(OldPins);
}

void UAnimGraphNode_LinkedInputPose::CreateUserDefinedPins()
{
	for(FAnimBlueprintFunctionPinInfo& PinInfo : Inputs)
	{
		UEdGraphPin* NewPin = CreatePin(EEdGraphPinDirection::EGPD_Output, PinInfo.Type, PinInfo.Name);
		NewPin->PinFriendlyName = FText::FromName(PinInfo.Name);
	}
}

void UAnimGraphNode_LinkedInputPose::CreatePinsFromStubFunction(const UFunction* Function)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	IterateFunctionParameters([this, K2Schema, Function](const FName& InName, const FEdGraphPinType& InPinType)
	{
		if(!UAnimationGraphSchema::IsPosePin(InPinType))
		{
			UEdGraphPin* Pin = CreatePin(EGPD_Output, InPinType, InName);
			K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
			
			UK2Node_CallFunction::GeneratePinTooltipFromFunction(*Pin, Function);
		}
	});
}

void UAnimGraphNode_LinkedInputPose::ConformInputPoseName()
{
	IterateFunctionParameters([this](const FName& InName, const FEdGraphPinType& InPinType)
	{
		if(UAnimationGraphSchema::IsPosePin(InPinType))
		{
			Node.Name = InName;
		}
	});
}

bool UAnimGraphNode_LinkedInputPose::ValidateAgainstFunctionReference() const
{
	bool bValid = false;

	IterateFunctionParameters([this, &bValid](const FName& InName, const FEdGraphPinType& InPinType)
	{
		bValid = true;
	});

	return bValid;
}

void UAnimGraphNode_LinkedInputPose::PostPlacedNewNode()
{
	if(IsEditable())
	{
		TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
		UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(GetGraph()->GetOuter());
		for(UEdGraph* Graph : AnimBlueprint->FunctionGraphs)
		{
			if(Graph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()))
			{
				// Create a unique name for this new linked input pose
				Graph->GetNodesOfClass(LinkedInputPoseNodes);
			}
		}

		Node.Name = CreateUniqueName(FAnimNode_LinkedInputPose::DefaultInputPoseName, [this, &LinkedInputPoseNodes](const FName& InNameToCheck)
		{
			for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode : LinkedInputPoseNodes)
			{
				if(LinkedInputPoseNode != this && LinkedInputPoseNode->Node.Name == InNameToCheck)
				{
					return false;
				}
			}

			return true;
		});

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakThis = TWeakObjectPtr<UAnimGraphNode_LinkedInputPose>(this)](float InDeltaTime)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_UAnimGraphNode_LinkedInputPose_PostPlacedNewNode);
			if(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode = WeakThis.Get())
			{
				// refresh the BP editor's details panel in case we are viewing the graph
				if(IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LinkedInputPoseNode->GetAnimBlueprint(), false))
				{
					check(AssetEditor->GetEditorName() == "AnimationBlueprintEditor");
					IAnimationBlueprintEditor* AnimationBlueprintEditor = static_cast<IAnimationBlueprintEditor*>(AssetEditor);
					AnimationBlueprintEditor->RefreshInspector();
				}
			}
			return false;
		}));
	}
}

class SLinkedInputPoseNodeLabelWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLinkedInputPoseNodeLabelWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InNamePropertyHandle, UAnimGraphNode_LinkedInputPose* InLinkedInputPoseNode)
	{
		NamePropertyHandle = InNamePropertyHandle;
		WeakLinkedInputPoseNode = InLinkedInputPoseNode;

		ChildSlot
		[
			SAssignNew(NameTextBox, SEditableTextBox)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &SLinkedInputPoseNodeLabelWidget::HandleGetNameText)
			.OnTextChanged(this, &SLinkedInputPoseNodeLabelWidget::HandleTextChanged)
			.OnTextCommitted(this, &SLinkedInputPoseNodeLabelWidget::HandleTextCommitted)
		];
	}

	FText HandleGetNameText() const
	{
		return FText::FromName(WeakLinkedInputPoseNode->Node.Name);
	}

	bool IsNameValid(const FString& InNewName, FText& OutReason)
	{
		if(InNewName.Len() == 0)
		{
			OutReason = LOCTEXT("ZeroSizeLinkedInputPoseError", "A name must be specified.");
			return false;
		}
		else if(InNewName.Equals(TEXT("None"), ESearchCase::IgnoreCase))
		{
			OutReason = LOCTEXT("LinkedInputPoseInvalidName", "This name is invalid.");
			return false;
		}
		else
		{
			UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(WeakLinkedInputPoseNode->GetGraph()->GetOuter());
			for(UEdGraph* Graph : AnimBlueprint->FunctionGraphs)
			{
				if(Graph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()))
				{
					TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
					Graph->GetNodesOfClass(LinkedInputPoseNodes);

					for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode : LinkedInputPoseNodes)
					{
						if(LinkedInputPoseNode != WeakLinkedInputPoseNode.Get() && LinkedInputPoseNode->Node.Name.ToString().Equals(InNewName, ESearchCase::IgnoreCase))
						{
							OutReason = LOCTEXT("DuplicateLinkedInputPoseError", "This linked input pose name already exists in this blueprint.");
							return false;
						}
					}
				}
			}

			return true;
		}
	}

	void HandleTextChanged(const FText& InNewText)
	{
		const FString NewTextAsString = InNewText.ToString();
	
		FText Reason;
		if(!IsNameValid(NewTextAsString, Reason))
		{
			NameTextBox->SetError(Reason);
		}
		else
		{
			NameTextBox->SetError(FText::GetEmpty());
		}
	}

	void HandleTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
	{
		const FString NewTextAsString = InNewText.ToString();
		FText Reason;
		if(IsNameValid(NewTextAsString, Reason))
		{
			FName NewName = *InNewText.ToString();
			NamePropertyHandle->SetValue(NewName);
		}

		NameTextBox->SetError(FText::GetEmpty());
	}

	TSharedPtr<SEditableTextBox> NameTextBox;
	TSharedPtr<IPropertyHandle> NamePropertyHandle;
	TWeakObjectPtr<UAnimGraphNode_LinkedInputPose> WeakLinkedInputPoseNode;
};

void UAnimGraphNode_LinkedInputPose::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& InputsCategoryBuilder = DetailBuilder.EditCategory("Inputs");

	TArray<TWeakObjectPtr<UObject>> OuterObjects;
	DetailBuilder.GetObjectsBeingCustomized(OuterObjects);
	if(OuterObjects.Num() != 1)
	{
		InputsCategoryBuilder.SetCategoryVisibility(false);
		return;
	}

	// skip if we cant edit this node as it is an interface graph
	UAnimGraphNode_LinkedInputPose* OuterNode = CastChecked<UAnimGraphNode_LinkedInputPose>(OuterObjects[0].Get());
	if(!OuterNode->CanUserDeleteNode())
	{
		FText ReadOnlyWarning = LOCTEXT("ReadOnlyWarning", "This input pose is read-only and cannot be edited");

		InputsCategoryBuilder.SetCategoryVisibility(false);

		IDetailCategoryBuilder& WarningCategoryBuilder = DetailBuilder.EditCategory("InputPose", LOCTEXT("InputPoseCategory", "Input Pose"));
		WarningCategoryBuilder.AddCustomRow(ReadOnlyWarning)
			.WholeRowContent()
			[
				SNew(STextBlock)
				.Text(ReadOnlyWarning)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];

		return;
	}

	TSharedPtr<IPropertyHandle> NamePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_LinkedInputPose, Node.Name), GetClass());
	InputsCategoryBuilder.AddProperty(NamePropertyHandle)
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.CustomWidget()
	.NameContent()
	[
		NamePropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		MakeNameWidget(DetailBuilder)
	];

	UEdGraph* Graph = GetGraph();
	if(Graph && Graph->GetFName() != UEdGraphSchema_K2::GN_AnimGraph)
	{
		InputsCategoryBuilder.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_LinkedInputPose, Inputs), GetClass())
			.ShouldAutoExpand(true);
	}
	else
	{
		TSharedPtr<IPropertyHandle> InputsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_LinkedInputPose, Inputs), GetClass());
		InputsPropertyHandle->MarkHiddenByCustomization();
	}
}

void UAnimGraphNode_LinkedInputPose::OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimGraphNode_LinkedInputPose* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_LinkedInputPose>(this);

	FAnimNode_LinkedInputPose* DestinationNode = reinterpret_cast<FAnimNode_LinkedInputPose*>(InPerNodeContext.GetDestinationPtr());
	DestinationNode->Graph = TrueNode->GetGraph()->GetFName();
}

void UAnimGraphNode_LinkedInputPose::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	Super::GetRequiredExtensions(OutExtensions);
	OutExtensions.Add(UAnimBlueprintExtension_LinkedInputPose::StaticClass());
}

TSharedRef<SWidget> UAnimGraphNode_LinkedInputPose::MakeNameWidget(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> NamePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_LinkedInputPose, Node.Name), GetClass());
	return SNew(SLinkedInputPoseNodeLabelWidget, NamePropertyHandle, this);
}

bool UAnimGraphNode_LinkedInputPose::HasExternalDependencies(TArray<UStruct*>* OptionalOutput) const
{
	const UBlueprint* SourceBlueprint = GetBlueprint();

	UClass* SourceClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());
	bool bResult = (SourceClass != nullptr) && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(SourceClass);
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

int32 UAnimGraphNode_LinkedInputPose::GetNumInputs() const
{
	if (UFunction* Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		// Count the inputs from parameters.
		int32 NumParameters = 0;

		IterateFunctionParameters([&NumParameters](const FName& InName, const FEdGraphPinType& InPinType)
		{
			if(!UAnimationGraphSchema::IsPosePin(InPinType))
			{
				NumParameters++;
			}
		});

		return NumParameters;
	}
	else
	{
		return Inputs.Num();
	}
}

void UAnimGraphNode_LinkedInputPose::PromoteFromInterfaceOverride()
{
	if (UFunction* Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		IterateFunctionParameters([this](const FName& InName, const FEdGraphPinType& InPinType)
		{
			if(!UAnimationGraphSchema::IsPosePin(InPinType))
			{
				Inputs.Emplace(InName, InPinType);	
			}
		});

		// Remove the signature class now, that is not relevant.
		FunctionReference.SetSelfMember(FunctionReference.GetMemberName());
		InputPoseIndex = INDEX_NONE;
	}
}

void UAnimGraphNode_LinkedInputPose::IterateFunctionParameters(TFunctionRef<void(const FName&, const FEdGraphPinType&)> InFunc) const
{
	if (UFunction* Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		// if the generated class is not up to date, use the skeleton's class function to create pins:
		Function = FBlueprintEditorUtils::GetMostUpToDateFunction(Function);

		// We need to find all parameters AFTER the pose we are representing
		int32 CurrentPoseIndex = 0;
		FProperty* PoseParam = nullptr;
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* Param = *PropIt;

			const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);

			if (bIsFunctionInput)
			{
				FEdGraphPinType PinType;
				if(K2Schema->ConvertPropertyToPinType(Param, PinType))
				{
					if(PoseParam == nullptr)
					{
						if(UAnimationGraphSchema::IsPosePin(PinType))
						{
							if(CurrentPoseIndex == InputPoseIndex)
							{
								PoseParam = Param;
								InFunc(Param->GetFName(), PinType);
							}
							CurrentPoseIndex++;
						}
					}
					else
					{
						if(UAnimationGraphSchema::IsPosePin(PinType))
						{
							// Found next pose param, so exit
							break;
						}
						else
						{
							InFunc(Param->GetFName(), PinType);
						}
					}
				}
			}
		}
	}
	else
	{
		// First call pose
		InFunc(Node.Name, UAnimationGraphSchema::MakeLocalSpacePosePin());

		// Then each input
		for(const FAnimBlueprintFunctionPinInfo& PinInfo : Inputs)
		{
			InFunc(PinInfo.Name, PinInfo.Type);
		}
	}
}

bool UAnimGraphNode_LinkedInputPose::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	return Graph->GetFName() == UEdGraphSchema_K2::GN_AnimGraph;
}

void UAnimGraphNode_LinkedInputPose::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	// We have the potential to output ALL registered attributes
	const UAnimGraphAttributes* AnimGraphAttributes = GetDefault<UAnimGraphAttributes>();
	AnimGraphAttributes->ForEachAttribute([&OutAttributes](const FAnimGraphAttributeDesc& InDesc)
	{
		OutAttributes.Add(InDesc.Name);
	});
}

void UAnimGraphNode_LinkedInputPose::AnalyzeLinks(TArrayView<UAnimGraphNode_Base*> InAnimNodes)
{
	Node.bIsOutputLinked = false;

	// Find root nodes
	TArray<UAnimGraphNode_Base*> RootSet;
	for (UAnimGraphNode_Base* InNode : InAnimNodes)
	{
		if (InNode->IsNodeRootSet())
		{
			RootSet.Add(InNode);
		}
	}

	struct FNodeVisitorDownPoseWires
	{
		TSet<UEdGraphNode*> VisitedNodes;
		const UAnimationGraphSchema* Schema;

		FNodeVisitorDownPoseWires()
		{
			Schema = GetDefault<UAnimationGraphSchema>();
		}

		// Traverse root set looking for this node
		bool TraverseNodes(UEdGraphNode* CurrentNode, UAnimGraphNode_LinkedInputPose* SearchedNode)
		{
			VisitedNodes.Add(CurrentNode);

			// Follow every exec output pin
			for (int32 i = 0; i < CurrentNode->Pins.Num(); ++i)
			{
				UEdGraphPin* MyPin = CurrentNode->Pins[i];

				if ((MyPin->Direction == EGPD_Input) && (Schema->IsPosePin(MyPin->PinType)))
				{
					for (int32 j = 0; j < MyPin->LinkedTo.Num(); ++j)
					{
						UEdGraphPin* OtherPin = MyPin->LinkedTo[j];
						UEdGraphNode* OtherNode = OtherPin->GetOwningNode();
						if (!VisitedNodes.Contains(OtherNode))
						{
							// Found ?
							if (OtherNode == SearchedNode || TraverseNodes(OtherNode, SearchedNode))
							{
								return true;
							}
						}
					}
				}
			}
			return false;
		}
	};

	FNodeVisitorDownPoseWires Visitor;

	// Search every root node
	for (UAnimGraphNode_Base* RootNode : RootSet)
	{
		if (Visitor.TraverseNodes(RootNode, this))
		{
			Node.bIsOutputLinked = true;
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
