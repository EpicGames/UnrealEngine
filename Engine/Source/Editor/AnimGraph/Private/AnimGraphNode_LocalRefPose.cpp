// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LocalRefPose.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_LocalRefPose

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LocalRefPose)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_LocalRefPose::UAnimGraphNode_LocalRefPose(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Node.SetRefPoseType(EIT_LocalSpace);
}

FText UAnimGraphNode_LocalRefPose::GetTooltipText() const
{
	return LOCTEXT("UAnimGraphNode_LocalRefPose_Tooltip", "Returns local space reference pose.");
}

FText UAnimGraphNode_LocalRefPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UAnimGraphNode_LocalRefPose_Title", "Local Space Ref Pose");
}

#undef LOCTEXT_NAMESPACE
