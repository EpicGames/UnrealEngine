// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LocalToComponentSpace.h"
#include "AnimationGraphSchema.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_LocalToComponentSpace

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LocalToComponentSpace)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_LocalToComponentSpace::UAnimGraphNode_LocalToComponentSpace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_LocalToComponentSpace::GetNodeTitleColor() const
{
	return FLinearColor(0.7f, 0.7f, 0.7f);
}

FText UAnimGraphNode_LocalToComponentSpace::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_LocalToComponentSpace_Tooltip", "Convert Local Pose to Component Space Pose");
}

FText UAnimGraphNode_LocalToComponentSpace::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_LocalToComponentSpace_Title", "Local To Component");
}

FText UAnimGraphNode_LocalToComponentSpace::GetMenuCategory() const
{
	return LOCTEXT("AnimGraphNode_LocalToComponentSpace_Category", "Animation|Convert Spaces");
}

void UAnimGraphNode_LocalToComponentSpace::CreateOutputPins()
{
	CreatePin(EGPD_Output, UAnimationGraphSchema::PC_Struct, FComponentSpacePoseLink::StaticStruct(), TEXT("ComponentPose"));
}

void UAnimGraphNode_LocalToComponentSpace::PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const
{
	DisplayName.Reset();
}

#undef LOCTEXT_NAMESPACE
