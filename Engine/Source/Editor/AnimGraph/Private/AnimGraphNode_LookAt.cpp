// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_LookAt.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "UObject/AnimPhysObjectVersion.h"
#include "AnimNodeEditModes.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_LookAt

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_LookAt)

#define LOCTEXT_NAMESPACE "AnimGraph_LookAt"

UAnimGraphNode_LookAt::UAnimGraphNode_LookAt(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_LookAt::GetControllerDescription() const
{
	return LOCTEXT("LookAtNode", "Look At");
}

FText UAnimGraphNode_LookAt::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_LookAt_Tooltip", "This node allow a bone to trace or follow another bone");
}

FText UAnimGraphNode_LookAt::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.BoneToModify.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.BoneToModify.BoneName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_LookAt_ListTitle", "{ControllerDescription} - Bone: {BoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_LookAt_Title", "{ControllerDescription}\nBone: {BoneName}"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];		
}

void UAnimGraphNode_LookAt::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const
{
	if(SkelMeshComp)
	{
		if(FAnimNode_LookAt* ActiveNode = GetActiveInstanceNode<FAnimNode_LookAt>(SkelMeshComp->GetAnimInstance()))
		{
			ActiveNode->ConditionalDebugDraw(PDI, SkelMeshComp);
		}
	}
}


void UAnimGraphNode_LookAt::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);

	auto GetAlignVector = [](EAxisOption::Type AxisOption, const FVector& CustomAxis) -> FVector
	{
		switch (AxisOption)
		{
		case EAxisOption::X:
			return FTransform::Identity.GetUnitAxis(EAxis::X);
		case EAxisOption::X_Neg:
			return -FTransform::Identity.GetUnitAxis(EAxis::X);
		case EAxisOption::Y:
			return FTransform::Identity.GetUnitAxis(EAxis::Y);
		case EAxisOption::Y_Neg:
			return -FTransform::Identity.GetUnitAxis(EAxis::Y);
		case EAxisOption::Z:
			return FTransform::Identity.GetUnitAxis(EAxis::Z);
		case EAxisOption::Z_Neg:
			return -FTransform::Identity.GetUnitAxis(EAxis::Z);
		case EAxisOption::Custom:
			return CustomAxis;
		}

		return FVector(1.f, 0.f, 0.f);
	};

	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID)< FAnimPhysObjectVersion::ConvertAnimNodeLookAtAxis)
	{
		Node.LookAt_Axis = FAxis(GetAlignVector(Node.LookAtAxis_DEPRECATED, Node.CustomLookAtAxis_DEPRECATED));
		Node.LookUp_Axis = FAxis(GetAlignVector(Node.LookUpAxis_DEPRECATED, Node.CustomLookUpAxis_DEPRECATED));
		if (Node.LookAtBone_DEPRECATED.BoneName != NAME_None || Node.LookAtSocket_DEPRECATED != NAME_None)
		{
			Node.LookAtLocation = FVector::ZeroVector;
		}
	}
	if (Ar.CustomVer(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::CreateTargetReference)
	{
		if (Node.LookAtSocket_DEPRECATED != NAME_None)
		{
			Node.LookAtTarget.bUseSocket = true;
			Node.LookAtTarget.SocketReference.SocketName = Node.LookAtSocket_DEPRECATED;
		}
		else if (Node.LookAtBone_DEPRECATED.BoneName != NAME_None)
		{
			Node.LookAtTarget.BoneReference.BoneName = Node.LookAtBone_DEPRECATED.BoneName;
		}
	}
}

void UAnimGraphNode_LookAt::GetOnScreenDebugInfo(TArray<FText>& DebugInfo, FAnimNode_Base* RuntimeAnimNode, USkeletalMeshComponent* PreviewSkelMeshComp) const
{
	if (RuntimeAnimNode)
	{
		const FAnimNode_LookAt* LookatRuntimeNode = static_cast<FAnimNode_LookAt*>(RuntimeAnimNode);
		DebugInfo.Add(FText::Format(LOCTEXT("DebugOnScreenBoneName", "Anim Look At (Source:{0})"), FText::FromName(LookatRuntimeNode->BoneToModify.BoneName)));

		if (LookatRuntimeNode->LookAtTarget.HasValidSetup())
		{
			DebugInfo.Add(FText::Format(LOCTEXT("DebugOnScreenLookAtTarget", "	Look At Target (Target:{0})"), FText::FromName(LookatRuntimeNode->LookAtTarget.GetTargetSetup())));
		}
		else
		{
			DebugInfo.Add(FText::Format(LOCTEXT("DebugOnScreenLookAtLocation", "	LookAtLocation: {0}"), FText::FromString(LookatRuntimeNode->LookAtLocation.ToString())));
		}

		DebugInfo.Add(FText::Format(LOCTEXT("DebugOnScreenTargetLocation", "	TargetLocation: {0}"), FText::FromString(LookatRuntimeNode->GetCachedTargetLocation().ToString())));
	}
}

FEditorModeID UAnimGraphNode_LookAt::GetEditorMode() const
{
	return AnimNodeEditModes::LookAt;
}

void UAnimGraphNode_LookAt::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_LookAt* LookAt = static_cast<FAnimNode_LookAt*>(InPreviewNode);

	// copies Pin values from the internal node to get data which are not compiled yet
	LookAt->LookAtLocation = Node.LookAtLocation;
}

void UAnimGraphNode_LookAt::CopyPinDefaultsToNodeData(UEdGraphPin* InPin)
{
	if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_LookAt, LookAtLocation))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_LookAt, LookAtLocation), Node.LookAtLocation);
	}
}

#undef LOCTEXT_NAMESPACE
