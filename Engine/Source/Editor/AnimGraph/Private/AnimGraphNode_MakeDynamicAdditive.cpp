// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_MakeDynamicAdditive.h"

#include "Animation/AnimationSettings.h"
#include "Kismet2/CompilerResultsLog.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_MakeDynamicAdditive

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_MakeDynamicAdditive)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_MakeDynamicAdditive::UAnimGraphNode_MakeDynamicAdditive(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_MakeDynamicAdditive::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.75f);
}

FText UAnimGraphNode_MakeDynamicAdditive::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_MakeDynamicAdditive_Tooltip", "Create a dynamic additive pose (additive pose - base pose)");
}

FText UAnimGraphNode_MakeDynamicAdditive::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_MakeDynamicAdditive_Title", "Make Dynamic Additive");
}

FString UAnimGraphNode_MakeDynamicAdditive::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

void UAnimGraphNode_MakeDynamicAdditive::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}
#undef LOCTEXT_NAMESPACE
