// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_MeshRefPose.h"
#include "AnimationGraphSchema.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_MeshRefPose

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_MeshRefPose)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_MeshRefPose::UAnimGraphNode_MeshRefPose(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UAnimGraphNode_MeshRefPose::GetNodeCategory() const
{
	return TEXT("Animation|Poses");
}

FLinearColor UAnimGraphNode_MeshRefPose::GetNodeTitleColor() const
{
	return FColor(200, 100, 100);
}

FText UAnimGraphNode_MeshRefPose::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_MeshRefPose_Tooltip", "Returns mesh space reference pose.");
}

FText UAnimGraphNode_MeshRefPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_MeshRefPose_Title", "Mesh Space Ref Pose");
}

void UAnimGraphNode_MeshRefPose::CreateOutputPins()
{
	CreatePin(EGPD_Output, UAnimationGraphSchema::PC_Struct, FComponentSpacePoseLink::StaticStruct(), TEXT("ComponentPose"));
}

#undef LOCTEXT_NAMESPACE
