// Copyright Epic Games, Inc. All Rights Reserved.UAnimGraphNode_MirrorPose

#include "AnimGraphNode_Mirror.h"
#include "Animation/AnimNode_Inertialization.h"
#include "Animation/MirrorDataTable.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintActionFilter.h"
#include "BlueprintActionFilter.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Kismet2/CompilerResultsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Mirror)

#define LOCTEXT_NAMESPACE "AnimGraphNode_MirrorPose"


FLinearColor UAnimGraphNode_Mirror::GetNodeTitleColor() const
{
	return FLinearColor(0.7f, 0.7f, 0.7f);
}

FText UAnimGraphNode_Mirror::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Mirror Pose");
}

FText UAnimGraphNode_Mirror::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Node.GetMirrorDataTable())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetName"), FText::FromName(Node.GetMirrorDataTable()->GetFName()));
		return FText::Format(LOCTEXT("MirrorWithAssetNodeTitle", "Mirror with {AssetName}"), Args);
	}
	else
	{
		return LOCTEXT("MirrorNodeTitle", "Mirror");
	}
}

FText UAnimGraphNode_Mirror::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Misc.");
}

void UAnimGraphNode_Mirror::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if (Node.GetBlendTimeOnMirrorStateChange() > 0.0)
	{
		OutAttributes.Add(UE::Anim::IInertializationRequester::Attribute);
	}
}

void UAnimGraphNode_Mirror::PreloadRequiredAssets()
{
	if (Node.GetMirrorDataTable())
	{
		PreloadObject(Node.GetMirrorDataTable());
		PreloadObject(Node.GetMirrorDataTable()->Skeleton);
	}
	Super::PreloadRequiredAssets();
}

void UAnimGraphNode_Mirror::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if (Node.GetMirrorDataTable() == nullptr)
	{
		MessageLog.Error(TEXT("@@ does not have a mirror data table selected.  Please select a table or delete the node."), this);
	}
	else if (ForSkeleton)
	{
		const USkeleton* MirrorTableSkeleton = Node.GetMirrorDataTable()->Skeleton;
		if (MirrorTableSkeleton == nullptr)
		{
			MessageLog.Warning(TEXT("@@ has a mirror data table that has a missing skeleton. Please update the table or create a new table for the skeleton."), this);
		}
	}
	
	BindMirrorDataTableChangedDelegate();
}

void UAnimGraphNode_Mirror::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_Mirror, MirrorDataTable))
	{
		if (Node.GetMirrorDataTable())
		{
			Node.GetMirrorDataTable()->OnDataTableChanged().RemoveAll(this);
		}
	}
}

void  UAnimGraphNode_Mirror::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	BindMirrorDataTableChangedDelegate();
}

void  UAnimGraphNode_Mirror::PostLoad()
{
	Super::PostLoad(); 
	BindMirrorDataTableChangedDelegate();
}

void UAnimGraphNode_Mirror::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_Mirror, MirrorDataTable))
	{
		BindMirrorDataTableChangedDelegate();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_Mirror::BindMirrorDataTableChangedDelegate()
{
	if (Node.GetMirrorDataTable())
	{
		Node.GetMirrorDataTable()->OnDataTableChanged().RemoveAll(this);
		Node.GetMirrorDataTable()->OnDataTableChanged().AddUObject(this, &UAnimGraphNode_Mirror::OnMirrorDataTableChanged);
	}
}

void UAnimGraphNode_Mirror::OnMirrorDataTableChanged()
{
	if (HasValidBlueprint())
	{
		UBlueprint* BP = GetBlueprint();
		BP->Status = BS_Dirty;
	}
}

void UAnimGraphNode_Mirror::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	auto LoadedAssetSetup = [](UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TWeakObjectPtr<UMirrorDataTable> MirrorDataTablePtr)
	{
		UAnimGraphNode_Mirror* MirrorNode = CastChecked<UAnimGraphNode_Mirror>(NewNode);
		MirrorNode->Node.SetMirrorDataTable(MirrorDataTablePtr.Get());
	};

	auto UnloadedAssetSetup = [](UEdGraphNode* NewNode, bool bIsTemplateNode, const FAssetData AssetData)
	{
		UAnimGraphNode_Mirror* MirrorNode = CastChecked<UAnimGraphNode_Mirror>(NewNode);
		if (bIsTemplateNode)
		{
			AssetData.GetTagValue("Skeleton", MirrorNode->UnloadedSkeletonName);
		}
		else
		{
			UMirrorDataTable* MirrorTable = Cast<UMirrorDataTable>(AssetData.GetAsset());
			check(MirrorTable != nullptr);
			MirrorNode->Node.SetMirrorDataTable(MirrorTable);
		}
	};

	const UObject* QueryObject = ActionRegistrar.GetActionKeyFilter();
	if (QueryObject == nullptr)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		// define a filter to help in pulling UMirrrorDataTable asset data from the registry
		FARFilter Filter;
		Filter.ClassPaths.Add(UMirrorDataTable::StaticClass()->GetClassPathName());

		Filter.bRecursiveClasses = true;
		// Find matching assets and add an entry for each one
		TArray<FAssetData> MirrorDataTableList;
		AssetRegistryModule.Get().GetAssets(Filter, /*out*/MirrorDataTableList);

		for (auto AssetIt = MirrorDataTableList.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FAssetData& Asset = *AssetIt;

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			if (Asset.IsAssetLoaded())
			{
				UMirrorDataTable* MirrorDataTable = Cast<UMirrorDataTable>(Asset.GetAsset());
				if (MirrorDataTable)
				{
					NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(LoadedAssetSetup, TWeakObjectPtr<UMirrorDataTable>(MirrorDataTable));
					NodeSpawner->DefaultMenuSignature.MenuName = GetTitleGivenAssetInfo(FText::FromName(MirrorDataTable->GetFName()));
				}
			}
			else
			{
				NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(UnloadedAssetSetup, Asset);
				NodeSpawner->DefaultMenuSignature.MenuName = GetTitleGivenAssetInfo(FText::FromName(Asset.AssetName));
			}
			ActionRegistrar.AddBlueprintAction(Asset, NodeSpawner);
		}
	}
	else if (const UMirrorDataTable* MirrorDataTable = Cast<UMirrorDataTable>(QueryObject))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());

		TWeakObjectPtr<UMirrorDataTable> MirrorDataTablePtr = MakeWeakObjectPtr(const_cast<UMirrorDataTable*>(MirrorDataTable));
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(LoadedAssetSetup, MirrorDataTablePtr);
		NodeSpawner->DefaultMenuSignature.MenuName = GetTitleGivenAssetInfo(FText::FromName(MirrorDataTablePtr->GetFName()));
		NodeSpawner->DefaultMenuSignature.Tooltip = GetTitleGivenAssetInfo(FText::FromString(MirrorDataTablePtr->GetPathName()));

		ActionRegistrar.AddBlueprintAction(QueryObject, NodeSpawner);
	}
	else if (QueryObject == GetClass())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FARFilter Filter;
		Filter.ClassPaths.Add(UMirrorDataTable::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;
		// Find matching assets and add an entry for each one
		TArray<FAssetData> MirrorDataTableList;
		AssetRegistryModule.Get().GetAssets(Filter, /*out*/MirrorDataTableList);

		for (auto AssetIt = MirrorDataTableList.CreateConstIterator(); AssetIt; ++AssetIt)
		{
			const FAssetData& Asset = *AssetIt;
			if (Asset.IsAssetLoaded())
			{
				continue;
			}

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(UnloadedAssetSetup, Asset);

			NodeSpawner->DefaultMenuSignature.MenuName = GetTitleGivenAssetInfo(FText::FromName(Asset.AssetName));
			NodeSpawner->DefaultMenuSignature.Tooltip = GetTitleGivenAssetInfo(FText::FromString(Asset.GetObjectPathString()));
			ActionRegistrar.AddBlueprintAction(Asset, NodeSpawner);
		}
	}

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		// Add a default create option if there are no compatible tables.  This action is filtered out when 
		// HasMirrorDataTableForBlueprints returns true because there is a table that can be used
		UBlueprintNodeSpawner* EmptyNodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		EmptyNodeSpawner->DefaultMenuSignature.MenuName = FText(LOCTEXT("MirrorNodeTitle", "Mirror"));
		ActionRegistrar.AddBlueprintAction(ActionKey, EmptyNodeSpawner);
	}
}

bool UAnimGraphNode_Mirror::HasMirrorDataTableForBlueprints(const TArray<UBlueprint*>& Blueprints) const
{
	// check to see if any mirror data table is available for the blueprint
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FARFilter Filter;
	Filter.ClassPaths.Add(UMirrorDataTable::StaticClass()->GetClassPathName());

	Filter.bRecursiveClasses = true;
	TArray<FAssetData> MirrorDataTableList;
	AssetRegistryModule.Get().GetAssets(Filter, MirrorDataTableList);

	for (UBlueprint* Blueprint : Blueprints)
	{
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
		{
			for (auto AssetIt = MirrorDataTableList.CreateConstIterator(); AssetIt; ++AssetIt)
			{
				const FAssetData& Asset = *AssetIt;
				if (Asset.IsAssetLoaded())
				{
					UMirrorDataTable* MirrorDataTable = Cast<UMirrorDataTable>(Asset.GetAsset());
					if (AnimBlueprint->TargetSkeleton && AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(MirrorDataTable->Skeleton))
					{
						return true; 
					}
				}
				else
				{
					FString TableSkeletonName;
					Asset.GetTagValue("Skeleton", TableSkeletonName);
					if (AnimBlueprint->TargetSkeleton && AnimBlueprint->TargetSkeleton->GetName() == TableSkeletonName)
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

bool UAnimGraphNode_Mirror::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UBlueprint* Blueprint : FilterContext.Blueprints)
	{
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
		{
			UMirrorDataTable* MirrorDataTable = Node.GetMirrorDataTable();
			if (MirrorDataTable)
			{
				if(AnimBlueprint->TargetSkeleton == nullptr || !AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(MirrorDataTable->Skeleton))
				{
					// Mirror Data Table does not use the same skeleton as the Blueprint, cannot use
					bIsFilteredOut = true;
					break;
				}
			}
			else if(!UnloadedSkeletonName.IsEmpty())
			{
				if(AnimBlueprint->TargetSkeleton == nullptr || !AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(UnloadedSkeletonName))
				{
					bIsFilteredOut = true;
					break;
				}
			}
			else
			{
				// Only show the empty "Mirror" option if there is not a possible MirrorDataTable to select
				if (HasMirrorDataTableForBlueprints(FilterContext.Blueprints))
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
		else
		{
			// Not an animation Blueprint, cannot use
			bIsFilteredOut = true;
			break;
		}
	}
	return bIsFilteredOut;
}

FText  UAnimGraphNode_Mirror::GetTitleGivenAssetInfo(const FText& AssetName)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("AssetName"), AssetName);

	return FText::Format(LOCTEXT("MirrorNodeTitle_WithAsset", "Mirror with {AssetName}"), Args);
}

EAnimAssetHandlerType UAnimGraphNode_Mirror::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UMirrorDataTable::StaticClass()))
	{
		return EAnimAssetHandlerType::PrimaryHandler;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}


#undef LOCTEXT_NAMESPACE
