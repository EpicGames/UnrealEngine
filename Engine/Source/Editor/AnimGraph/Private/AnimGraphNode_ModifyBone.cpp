// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ModifyBone.h"
#include "UnrealWidgetFwd.h"
#include "AnimNodeEditModes.h"
#include "Kismet2/CompilerResultsLog.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ModifyBone

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ModifyBone)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_ModifyBone::UAnimGraphNode_ModifyBone(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_ModifyBone::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	// Temporary fix where skeleton is not fully loaded during AnimBP compilation and thus virtual bone name check is invalid UE-39499 (NEED FIX) 
	if (ForSkeleton && !ForSkeleton->HasAnyFlags(RF_NeedPostLoad))
	{
		if (ForSkeleton->GetReferenceSkeleton().FindBoneIndex(Node.BoneToModify.BoneName) == INDEX_NONE)
		{
			if (Node.BoneToModify.BoneName == NAME_None)
			{
				MessageLog.Warning(*LOCTEXT("NoBoneSelectedToModify", "@@ - You must pick a bone to modify").ToString(), this);
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("BoneName"), FText::FromName(Node.BoneToModify.BoneName));

				FText Msg = FText::Format(LOCTEXT("NoBoneFoundToModify", "@@ - Bone {BoneName} not found in Skeleton"), Args);

				MessageLog.Warning(*Msg.ToString(), this);
			}
		}
	}

	if ((Node.TranslationMode == BMM_Ignore) && (Node.RotationMode == BMM_Ignore) && (Node.ScaleMode == BMM_Ignore))
	{
		MessageLog.Warning(*LOCTEXT("NothingToModify", "@@ - No components to modify selected.  Either Rotation, Translation, or Scale should be set to something other than Ignore").ToString(), this);
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

FText UAnimGraphNode_ModifyBone::GetControllerDescription() const
{
	return LOCTEXT("TransformModifyBone", "Transform (Modify) Bone");
}

FText UAnimGraphNode_ModifyBone::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ModifyBone_Tooltip", "The Transform Bone node alters the transform - i.e. Translation, Rotation, or Scale - of the bone");
}

FText UAnimGraphNode_ModifyBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.BoneToModify.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.BoneToModify.BoneName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_ModifyBone_ListTitle", "{ControllerDescription} - Bone: {BoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_ModifyBone_Title", "{ControllerDescription}\nBone: {BoneName}"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

void UAnimGraphNode_ModifyBone::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_ModifyBone* ModifyBone = static_cast<FAnimNode_ModifyBone*>(InPreviewNode);

	// copies Pin values from the internal node to get data which are not compiled yet
	ModifyBone->Translation = Node.Translation;
	ModifyBone->Rotation = Node.Rotation;
	ModifyBone->Scale = Node.Scale;

	// copies Modes
	ModifyBone->TranslationMode = Node.TranslationMode;
	ModifyBone->RotationMode = Node.RotationMode;
	ModifyBone->ScaleMode = Node.ScaleMode;

	// copies Spaces
	ModifyBone->TranslationSpace = Node.TranslationSpace;
	ModifyBone->RotationSpace = Node.RotationSpace;
	ModifyBone->ScaleSpace = Node.ScaleSpace;
}

FEditorModeID UAnimGraphNode_ModifyBone::GetEditorMode() const
{
	return AnimNodeEditModes::ModifyBone;
}

void UAnimGraphNode_ModifyBone::CopyPinDefaultsToNodeData(UEdGraphPin* InPin)
{
	if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Translation))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Translation), Node.Translation);
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Rotation))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Rotation), Node.Rotation);
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Scale))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Scale), Node.Scale);
	}
}

#undef LOCTEXT_NAMESPACE
