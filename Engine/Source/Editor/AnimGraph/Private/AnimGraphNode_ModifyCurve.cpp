// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ModifyCurve.h"
#include "Textures/SlateIcon.h"
#include "GraphEditorActions.h"
#include "ScopedTransaction.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimationGraphSchema.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "PersonaModule.h"
#include "Framework/Commands/UIAction.h"
#include "ToolMenus.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/STextEntryPopup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ModifyCurve)

#define LOCTEXT_NAMESPACE "ModifyCurve"


UAnimGraphNode_ModifyCurve::UAnimGraphNode_ModifyCurve() 
{
}

FText UAnimGraphNode_ModifyCurve::GetMenuCategory() const
{
	return LOCTEXT("AnimGraphNode_ModifyCurve_Category", "Animation|Curves");
}

FText UAnimGraphNode_ModifyCurve::GetTooltipText() const
{
	return GetNodeTitle(ENodeTitleType::ListView);
}

FText UAnimGraphNode_ModifyCurve::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_ModifyCurve_Title", "Modify Curve");
}

void UAnimGraphNode_ModifyCurve::GetAddCurveMenuActions(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NewCurveLabel", "New Curve..."),
		LOCTEXT("NewCurveToolTip", "Adds a new curve to modify"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			FSlateApplication::Get().DismissAllMenus();

			TSharedRef<STextEntryPopup> TextEntry =
				SNew(STextEntryPopup)
				.Label(LOCTEXT("NewCurvePopupLabel", "New Curve Name"))
				.OnTextCommitted_Lambda([this](FText InText, ETextCommit::Type InCommitType)
				{
					FSlateApplication::Get().DismissAllMenus();
					const_cast<UAnimGraphNode_ModifyCurve*>(this)->AddCurvePin(*InText.ToString());
				});

			FSlateApplication& SlateApp = FSlateApplication::Get();
			SlateApp.PushMenu(
				SlateApp.GetInteractiveTopLevelWindows()[0],
				FWidgetPath(),
				TextEntry,
				SlateApp.GetCursorPos(),
				FPopupTransitionEffect::TypeInPopup
				);
		}))
	);

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TSharedRef<SWidget> CurvePicker = PersonaModule.CreateCurvePicker(GetAnimBlueprint()->TargetSkeleton,
		FOnCurvePicked::CreateLambda([this](const FName& InName)
		{
			FSlateApplication::Get().DismissAllMenus();
			const_cast<UAnimGraphNode_ModifyCurve*>(this)->AddCurvePin(InName);
		}),
		FIsCurveNameMarkedForExclusion::CreateLambda([this](const FName& InName)
		{
			return Node.CurveNames.Contains(InName);
		}));

	MenuBuilder.AddWidget(CurvePicker, FText::GetEmpty(), true, false);
}

void UAnimGraphNode_ModifyCurve::GetRemoveCurveMenuActions(FMenuBuilder& MenuBuilder) const
{
	for (FName CurveName : Node.CurveNames)
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateUObject(const_cast<UAnimGraphNode_ModifyCurve*>(this), &UAnimGraphNode_ModifyCurve::RemoveCurvePin, CurveName));
		MenuBuilder.AddMenuEntry(FText::FromName(CurveName), FText::GetEmpty(), FSlateIcon(), Action);
	}
}

void UAnimGraphNode_ModifyCurve::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeModifyCurve", LOCTEXT("ModifyCurve", "Modify Curve"));

		// Clicked pin
		if (Context->Pin != NULL)
		{
			// Get property from pin
			FProperty* AssociatedProperty;
			int32 ArrayIndex;
			GetPinAssociatedProperty(GetFNodeType(), Context->Pin, /*out*/ AssociatedProperty, /*out*/ ArrayIndex);

			if (AssociatedProperty != nullptr)
			{
				FName PinPropertyName = AssociatedProperty->GetFName();

				if (PinPropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_ModifyCurve, CurveValues) && Context->Pin->Direction == EGPD_Input)
				{
					FString PinName = Context->Pin->PinFriendlyName.ToString();
					FUIAction Action = FUIAction(FExecuteAction::CreateUObject(const_cast<UAnimGraphNode_ModifyCurve*>(this), &UAnimGraphNode_ModifyCurve::RemoveCurvePin, FName(*PinName)));
					FText RemovePinLabelText = FText::Format(LOCTEXT("RemoveThisPin", "Remove This Curve Pin: {0}"), FText::FromString(PinName));
					Section.AddMenuEntry("RemoveThisPin", RemovePinLabelText, LOCTEXT("RemoveThisPinTooltip", "Remove this curve pin from this node"), FSlateIcon(), Action);
				}
			}
		}
		
		Section.AddSubMenu(
			"AddCurvePin",
			LOCTEXT("AddCurvePin", "Add Curve Pin"),
			LOCTEXT("AddCurvePinTooltip", "Add a new pin to drive a curve"),
			FNewMenuDelegate::CreateUObject(this, &UAnimGraphNode_ModifyCurve::GetAddCurveMenuActions));

		// If we have curves to remove, create submenu to offer them
		if (Node.CurveNames.Num() > 0)
		{
			Section.AddSubMenu(
				"RemoveCurvePin",
				LOCTEXT("RemoveCurvePin", "Remove Curve Pin"),
				LOCTEXT("RemoveCurvePinTooltip", "Remove a pin driving a curve"),
				FNewMenuDelegate::CreateUObject(this, &UAnimGraphNode_ModifyCurve::GetRemoveCurveMenuActions));
		}
	}
}

void UAnimGraphNode_ModifyCurve::RemoveCurvePin(FName CurveName)
{
	// Make sure we have a curve pin with that name
	int32 CurveIndex = Node.CurveNames.Find(CurveName);
	if (CurveIndex != INDEX_NONE)
	{
		FScopedTransaction Transaction( LOCTEXT("RemoveCurvePinTrans", "Remove Curve Pin") );
		Modify();

		Node.RemoveCurve(CurveIndex);
	
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_ModifyCurve::AddCurvePin(FName CurveName)
{
	// Make sure it doesn't already exist
	int32 CurveIndex = Node.CurveNames.Find(CurveName);
	if (CurveIndex == INDEX_NONE)
	{
		FScopedTransaction Transaction(LOCTEXT("AddCurvePinTrans", "Add Curve Pin"));
		Modify();

		Node.AddCurve(CurveName, 0.f);

		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}


void UAnimGraphNode_ModifyCurve::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	if (SourcePropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_ModifyCurve, CurveValues))
	{
		if (Node.CurveNames.IsValidIndex(ArrayIndex))
		{
			Pin->PinFriendlyName = FText::FromName(Node.CurveNames[ArrayIndex]);
		}
	}
}

#undef LOCTEXT_NAMESPACE
