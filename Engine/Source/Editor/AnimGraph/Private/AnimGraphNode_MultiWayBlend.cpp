// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_MultiWayBlend.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimGraphCommands.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"


/////////////////////////////////////////////////////
// UAnimGraphNode_MultiWayBlend

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_MultiWayBlend)

#define LOCTEXT_NAMESPACE "AnimGraphNode_MultiWayBlend"

UAnimGraphNode_MultiWayBlend::UAnimGraphNode_MultiWayBlend(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UAnimGraphNode_MultiWayBlend::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

FLinearColor UAnimGraphNode_MultiWayBlend::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.75f);
}

FText UAnimGraphNode_MultiWayBlend::GetTooltipText() const
{
	return LOCTEXT("MultiWayBlendTooltip", "Blend multiple poses together by Alpha");
}

FText UAnimGraphNode_MultiWayBlend::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Blend", "Blend Multi");
}

void UAnimGraphNode_MultiWayBlend::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphBlendMulti", LOCTEXT("BlendMultiHeader", "BlendMulti"));
			if (Context->Pin != NULL)
			{
				// we only do this for normal BlendMulti/BlendMulti by enum, BlendMulti by Bool doesn't support add/remove pins
				if (Context->Pin->Direction == EGPD_Input)
				{
					Section.AddMenuEntry(FAnimGraphCommands::Get().RemoveBlendListPin);
				}
			}
			else
			{
				Section.AddMenuEntry(FAnimGraphCommands::Get().AddBlendListPin);
			}
		}
	}
}

void UAnimGraphNode_MultiWayBlend::PostEditUndo()
{
	Super::PostEditUndo();

	Node.UpdateCachedAlphas();
}

void UAnimGraphNode_MultiWayBlend::AddPinToBlendNode()
{
	FScopedTransaction Transaction(LOCTEXT("AddBlendMultiPin", "AddBlendMultiPin"));
	Modify();

	Node.AddPose();
	ReconstructNode();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UAnimGraphNode_MultiWayBlend::RemovePinFromBlendNode(UEdGraphPin* Pin)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveBlendMultiPin", "RemoveBlendMultiPin"));
	Modify();

	FProperty* AssociatedProperty;
	int32 ArrayIndex;
	GetPinAssociatedProperty(GetFNodeType(), Pin, /*out*/ AssociatedProperty, /*out*/ ArrayIndex);

	if (ArrayIndex != INDEX_NONE)
	{
		//@TODO: ANIMREFACTOR: Need to handle moving pins below up correctly
		// setting up removed pins info
		RemovedPinArrayIndex = ArrayIndex;
		Node.RemovePose(ArrayIndex);
		// removes the selected pin and related properties in reconstructNode()
		// @TODO: Considering passing "RemovedPinArrayIndex" to ReconstructNode as the argument
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_MultiWayBlend::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	
	// Make sure we start out with two inputs
	Node.AddPose();
	Node.AddPose();
	ReconstructNode();
}

void UAnimGraphNode_MultiWayBlend::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	// Delete Pins by removed pin info 
	if (RemovedPinArrayIndex != INDEX_NONE)
	{
		RemovePinsFromOldPins(OldPins, RemovedPinArrayIndex);
		// Clears removed pin info to avoid to remove multiple times
		// @TODO : Considering receiving RemovedPinArrayIndex as an argument of ReconstructNode()
		RemovedPinArrayIndex = INDEX_NONE;
	}
}

void UAnimGraphNode_MultiWayBlend::RemovePinsFromOldPins(TArray<UEdGraphPin*>& OldPins, int32 RemovedArrayIndex)
{
	TArray<FString> RemovedPropertyNames;
	TArray<FName> NewPinNames;

	// Store new pin names to compare with old pin names
	for (int32 NewPinIndx = 0; NewPinIndx < Pins.Num(); NewPinIndx++)
	{
		NewPinNames.Add(Pins[NewPinIndx]->PinName);
	}

	// don't know which pins are removed yet so find removed pins comparing NewPins and OldPins
	for (int32 OldPinIdx = 0; OldPinIdx < OldPins.Num(); OldPinIdx++)
	{
		const FName OldPinName = OldPins[OldPinIdx]->PinName;
		if (!NewPinNames.Contains(OldPinName))
		{
			const FString OldPinNameStr = OldPinName.ToString();
			const int32 UnderscoreIndex = OldPinNameStr.Find(TEXT("_"), ESearchCase::CaseSensitive);
			if (UnderscoreIndex != INDEX_NONE)
			{
				FString PropertyName = OldPinNameStr.Left(UnderscoreIndex);
				RemovedPropertyNames.Add(MoveTemp(PropertyName));
			}
		}
	}

	for (int32 PinIdx = 0; PinIdx < OldPins.Num(); PinIdx++)
	{
		// Separate the pin name into property name and index
		const FString OldPinNameStr = OldPins[PinIdx]->PinName.ToString();

		const int32 UnderscoreIndex = OldPinNameStr.Find(TEXT("_"), ESearchCase::CaseSensitive);
		if (UnderscoreIndex != INDEX_NONE)
		{
			const FString PropertyName = OldPinNameStr.Left(UnderscoreIndex);

			if (RemovedPropertyNames.Contains(PropertyName))
			{
				const int32 ArrayIndex = FCString::Atoi(*(OldPinNameStr.Mid(UnderscoreIndex + 1)));
				// if array index is matched, removes pins 
				// and if array index is greater than removed index, decrease index
				if (ArrayIndex == RemovedArrayIndex)
				{
					OldPins[PinIdx]->MarkAsGarbage();
					OldPins.RemoveAt(PinIdx);
					--PinIdx;
				}
				else if (ArrayIndex > RemovedArrayIndex)
				{
					OldPins[PinIdx]->PinName = *FString::Printf(TEXT("%s_%d"), *PropertyName, ArrayIndex - 1);
				}
			}
		}
	}
}
#undef LOCTEXT_NAMESPACE

