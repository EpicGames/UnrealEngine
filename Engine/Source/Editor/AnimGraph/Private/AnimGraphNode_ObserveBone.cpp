// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ObserveBone.h"
#include "Kismet2/CompilerResultsLog.h"
#include "SNodePanel.h"
#include "SGraphNode.h"
#include "KismetNodes/KismetNodeInfoContext.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "AnimNodeEditModes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ObserveBone)

#define LOCTEXT_NAMESPACE "ObserveBone"

/////////////////////////////////////////////////////
// SGraphNodeObserveBone

class SGraphNodeObserveBone : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeObserveBone){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimGraphNode_ObserveBone* InNode)
	{
		GraphNode = InNode;

		SetCursor(EMouseCursor::CardinalCross);

		UpdateGraphNode();
	}

	// SNodePanel::SNode interface
	void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	// End of SNodePanel::SNode interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override
	{
		SGraphNode::UpdateGraphNode();

		// Remove the comment bubble slot
		RemoveSlot(ENodeZone::TopCenter);
	}
	// End of SGraphNode interface

	static FString PrettyVectorToString(const FVector& Vector, const FString& PerComponentPrefix);
};

void SGraphNodeObserveBone::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	const FLinearColor TimelineBubbleColor(0.7f, 0.5f, 0.5f);
	FKismetNodeInfoContext* K2Context = (FKismetNodeInfoContext*)Context;

	// Display the observed bone transform status bubble
 	if (UObject* ActiveObject = K2Context->ActiveObjectBeingDebugged)
 	{
		FProperty* NodeProperty = FKismetDebugUtilities::FindClassPropertyForNode(K2Context->SourceBlueprint, GraphNode);
		if (FStructProperty* StructProperty = CastField<FStructProperty>(NodeProperty))
 		{
 			UClass* ContainingClass = StructProperty->GetTypedOwner<UClass>();
 			if (ActiveObject->IsA(ContainingClass) && (StructProperty->Struct == FAnimNode_ObserveBone::StaticStruct()))
 			{
				FAnimNode_ObserveBone* ObserveBone = StructProperty->ContainerPtrToValuePtr<FAnimNode_ObserveBone>(ActiveObject);

				check(ObserveBone);
				const FString Message = FString::Printf(TEXT("%s\n%s\n%s"),
					*PrettyVectorToString(ObserveBone->Translation, TEXT("T")),
					*PrettyVectorToString(ObserveBone->Rotation.Euler(), TEXT("R")),
					*PrettyVectorToString(ObserveBone->Scale, TEXT("S")));

				new (Popups) FGraphInformationPopupInfo(NULL, TimelineBubbleColor, Message);
			}
 			else
 			{
				const FString ErrorText = FText::Format(LOCTEXT("StaleDebugDataFmt", "Stale debug data\nProperty is on {0}\nDebugging a {1}"), FText::FromString(ContainingClass->GetName()), FText::FromString(ActiveObject->GetClass()->GetName())).ToString();
				new (Popups) FGraphInformationPopupInfo(NULL, TimelineBubbleColor, ErrorText);
 			}
 		}
	}

	SGraphNode::GetNodeInfoPopups(Context, Popups);
}

FString SGraphNodeObserveBone::PrettyVectorToString(const FVector& Vector, const FString& PerComponentPrefix)
{
	const FString ReturnString = FString::Printf(TEXT("%sX=%.2f, %sY=%.2f, %sZ=%.2f"), *PerComponentPrefix, Vector.X, *PerComponentPrefix, Vector.Y, *PerComponentPrefix, Vector.Z);
	return ReturnString;
}

/////////////////////////////////////////////////////
// UAnimGraphNode_ObserveBone

TSharedPtr<SGraphNode> UAnimGraphNode_ObserveBone::CreateVisualWidget()
{
	return SNew(SGraphNodeObserveBone, this);
}

UAnimGraphNode_ObserveBone::UAnimGraphNode_ObserveBone(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_ObserveBone::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	if (ForSkeleton && ForSkeleton->GetReferenceSkeleton().FindBoneIndex(Node.BoneToObserve.BoneName) == INDEX_NONE)
	{
		MessageLog.Warning(*LOCTEXT("NoBoneToObserve", "@@ - You must pick a bone to observe").ToString(), this);
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

FText UAnimGraphNode_ObserveBone::GetControllerDescription() const
{
	return LOCTEXT("AnimGraphNode_ObserveBone", "Observe Bone");
}

FText UAnimGraphNode_ObserveBone::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ObserveBone_Tooltip", "Observes a bone for debugging purposes");
}

FText UAnimGraphNode_ObserveBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (((TitleType == ENodeTitleType::ListView) || (TitleType == ENodeTitleType::MenuTitle)) && (Node.BoneToObserve.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.BoneToObserve.BoneName));

		return FText::Format(LOCTEXT("AnimGraphNode_ObserveBone_Title", "{ControllerDescription}: {BoneName}"), Args);
	}
}

FLinearColor UAnimGraphNode_ObserveBone::GetNodeTitleColor() const
{
	return FLinearColor(0.7f, 0.7f, 0.7f);
}

void UAnimGraphNode_ObserveBone::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, Alpha))
	{
		Pin->bHidden = true;
	}
}

void UAnimGraphNode_ObserveBone::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_ObserveBone, Node), GetClass());
	DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, Alpha)));
	DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, AlphaScaleBias)));
}

FEditorModeID UAnimGraphNode_ObserveBone::GetEditorMode() const
{
	return AnimNodeEditModes::ObserveBone;
}

#undef LOCTEXT_NAMESPACE
