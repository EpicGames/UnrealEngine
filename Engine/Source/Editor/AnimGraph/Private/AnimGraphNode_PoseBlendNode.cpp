// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_PoseBlendNode.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"

#include "AnimGraphCommands.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintActionFilter.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_PoseBlendNode)

#define LOCTEXT_NAMESPACE "PoseBlendNode"

// Action to add a pose asset blend node to the graph
struct FNewPoseBlendNodeAction : public FEdGraphSchemaAction_K2NewNode
{
protected:
	FAssetData AssetInfo;
public:
	FNewPoseBlendNodeAction(const FAssetData& InAssetInfo, FText Title)
		: FEdGraphSchemaAction_K2NewNode(LOCTEXT("PoseAsset", "PoseAssets"), Title, LOCTEXT("EvalCurvesToMakePose", "Evaluates curves to produce a pose from pose asset"), 0, FText::FromString(InAssetInfo.GetObjectPathString()))
	{
		AssetInfo = InAssetInfo;

		UAnimGraphNode_PoseBlendNode* Template = NewObject<UAnimGraphNode_PoseBlendNode>();
		NodeTemplate = Template;
	}

	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override
	{
		UAnimGraphNode_PoseBlendNode* SpawnedNode = CastChecked<UAnimGraphNode_PoseBlendNode>(FEdGraphSchemaAction_K2NewNode::PerformAction(ParentGraph, FromPin, Location, bSelectNewNode));
		SpawnedNode->Node.PoseAsset = Cast<UPoseAsset>(AssetInfo.GetAsset());

		return SpawnedNode;
	}
};
/////////////////////////////////////////////////////
// UAnimGraphNode_PoseBlendNode

UAnimGraphNode_PoseBlendNode::UAnimGraphNode_PoseBlendNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_PoseBlendNode::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const
{
	if(Node.PoseAsset)
	{
		HandleAnimReferenceCollection(Node.PoseAsset, AnimationAssets);
	}
}

void UAnimGraphNode_PoseBlendNode::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(Node.PoseAsset, AnimAssetReplacementMap);
}

FText UAnimGraphNode_PoseBlendNode::GetTooltipText() const
{
	// FText::Format() is slow, so we utilize the cached list title
	return GetNodeTitle(ENodeTitleType::ListView);
}

FText UAnimGraphNode_PoseBlendNode::GetNodeTitleForPoseAsset(ENodeTitleType::Type TitleType, UPoseAsset* InPoseAsset) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("PoseAssetName"), FText::FromString(InPoseAsset->GetName()));

	return FText::Format(LOCTEXT("PoseByName_Title", "{PoseAssetName}"), Args);
}

void UAnimGraphNode_PoseBlendNode::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FText UAnimGraphNode_PoseBlendNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Node.PoseAsset == nullptr)
	{
		// we may have a valid variable connected or default pin value
		UEdGraphPin* PosePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseBlendNode, PoseAsset));
		if (PosePin && PosePin->LinkedTo.Num() > 0)
		{
			return LOCTEXT("PoseByName_TitleVariable", "Pose");
		}
		else if (PosePin && PosePin->DefaultObject != nullptr)
		{
			return GetNodeTitleForPoseAsset(TitleType, CastChecked<UPoseAsset>(PosePin->DefaultObject));
		}
		else
		{
			return LOCTEXT("PoseByName_TitleNONE", "Pose (None)");
		}
	}
	else
	{
		return GetNodeTitleForPoseAsset(TitleType, Node.PoseAsset);
	}
}

void UAnimGraphNode_PoseBlendNode::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UPoseAsset::StaticClass() },
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return GetTitleGivenAssetInfo(FText::FromName(InAssetData.AssetName));
			}
			else
			{
				return LOCTEXT("PlayerDesc", "Evaluate Pose");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return GetTitleGivenAssetInfo(FText::FromName(InAssetData.AssetName));
			}
			else
			{
				return LOCTEXT("PlayerDescTooltip", "Evaluate Pose Asset");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_AssetPlayerBase::SetupNewNode(InNewNode, bInIsTemplateNode, InAssetData);
		});	
}

FText UAnimGraphNode_PoseBlendNode::GetTitleGivenAssetInfo(const FText& AssetName)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("AssetName"), AssetName);

	return FText::Format(LOCTEXT("PoseAssetNodeTitle", "Evaluate Pose {AssetName}"), Args);
}

FText UAnimGraphNode_PoseBlendNode::GetMenuCategory() const
{
	return LOCTEXT("PoseAssetCategory_Label", "Animation|Poses");
}

bool UAnimGraphNode_PoseBlendNode::DoesSupportTimeForTransitionGetter() const
{
	return false;
}

void UAnimGraphNode_PoseBlendNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to single frame
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodePoseBlender", LOCTEXT("PoseBlenderHeading", "Pose Blender"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToPoseByName);
		}
	}
}

EAnimAssetHandlerType UAnimGraphNode_PoseBlendNode::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UPoseAsset::StaticClass()))
	{
		return EAnimAssetHandlerType::PrimaryHandler;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

#undef LOCTEXT_NAMESPACE
