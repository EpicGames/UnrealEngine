// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_PoseByName.h"
#include "ToolMenus.h"

#include "Kismet2/CompilerResultsLog.h"
#include "AnimGraphCommands.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_PoseByName

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_PoseByName)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_PoseByName::UAnimGraphNode_PoseByName(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_PoseByName::PreloadRequiredAssets()
{
	PreloadRequiredAssetsHelper(Node.PoseAsset, FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseHandler, PoseAsset)));

	Super::PreloadRequiredAssets();
}


void UAnimGraphNode_PoseByName::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const
{
	if(Node.PoseAsset)
	{
		HandleAnimReferenceCollection(Node.PoseAsset, AnimationAssets);
	}
}

void UAnimGraphNode_PoseByName::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(Node.PoseAsset, AnimAssetReplacementMap);
}

FText UAnimGraphNode_PoseByName::GetTooltipText() const
{
	// FText::Format() is slow, so we utilize the cached list title
	return GetNodeTitle(ENodeTitleType::ListView);
}

FText UAnimGraphNode_PoseByName::GetMenuCategory() const
{
	return LOCTEXT("PoseAssetCategory_Label", "Animation|Poses");
}

FText UAnimGraphNode_PoseByName::GetNodeTitleForPoseAsset(ENodeTitleType::Type TitleType, UPoseAsset* InPoseAsset) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("PoseAssetName"), FText::FromString(InPoseAsset->GetName()));
	Args.Add(TEXT("PoseName"), FText::FromString(Node.PoseName.ToString()));

	// FText::Format() is slow, so we cache this to save on performance
	CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("PoseByName_Title", "{PoseAssetName} : {PoseName}"), Args), this);

	return CachedNodeTitle;
}

FText UAnimGraphNode_PoseByName::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Node.PoseAsset == nullptr)
	{
		// we may have a valid variable connected or default pin value
		UEdGraphPin* PosePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseByName, PoseAsset));
		if (PosePin && PosePin->LinkedTo.Num() > 0)
		{
			return LOCTEXT("PoseByName_TitleVariable", "Pose");
		}
		else if (PosePin && PosePin->DefaultObject != nullptr)
		{
			return GetNodeTitleForPoseAsset(TitleType, CastChecked<UPoseAsset>(PosePin->DefaultObject));
		}
		else
		{
			return LOCTEXT("PoseByName_TitleNONE", "Pose (None)");
		}
	}
	// @TODO: don't know enough about this node type to comfortably assert that
	//        the CacheName won't change after the node has spawned... until
	//        then, we'll leave this optimization off
	else //if (CachedNodeTitle.IsOutOfDate(this))
	{
		return GetNodeTitleForPoseAsset(TitleType, Node.PoseAsset);
	}
}

// void UAnimGraphNode_PoseByName::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
// {
// 	// Intentionally empty; you can drop down a regular sequence player and convert into a sequence evaluator in the right-click menu.
// }

void UAnimGraphNode_PoseByName::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UPoseAsset* PoseAsset =  Cast<UPoseAsset>(Asset))
	{
		Node.PoseAsset = PoseAsset;
	}
}

void UAnimGraphNode_PoseByName::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	UPoseAsset* PoseAssetToCheck = Node.PoseAsset;
	UEdGraphPin* PoseAssetPin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseByName, PoseAsset));
	if (PoseAssetPin != nullptr && PoseAssetToCheck == nullptr)
	{
		PoseAssetToCheck = Cast<UPoseAsset>(PoseAssetPin->DefaultObject);
	}

	if (PoseAssetToCheck == nullptr)
	{
		// we may have a connected node
		if (PoseAssetPin == nullptr || PoseAssetPin->LinkedTo.Num() == 0)
		{
			MessageLog.Error(TEXT("@@ references an unknown pose asset"), this);
		}
	}
	else if (ForSkeleton)
	{
		USkeleton* PoseAssetSkeleton = PoseAssetToCheck->GetSkeleton();
		if (PoseAssetSkeleton == nullptr)
		{
			MessageLog.Error(TEXT("@@ references pose asset that uses a missing skeleton @@"), this, PoseAssetSkeleton);
		}
	}
}

bool UAnimGraphNode_PoseByName::DoesSupportTimeForTransitionGetter() const
{
	return false;
}

UAnimationAsset* UAnimGraphNode_PoseByName::GetAnimationAsset() const 
{
	UPoseAsset* PoseAsset = Node.PoseAsset;
	UEdGraphPin* PoseAssetPin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseByName, PoseAsset));
	if (PoseAssetPin != nullptr && PoseAsset == nullptr)
	{
		PoseAsset = Cast<UPoseAsset>(PoseAssetPin->DefaultObject);
	}

	return PoseAsset;
}

void UAnimGraphNode_PoseByName::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to single frame
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodePoseByName", LOCTEXT("PoseByNameHeading", "Pose By Name"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToPoseBlender);
		}
	}
}

void UAnimGraphNode_PoseByName::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Intentionally empty; you can drop down a regular pose blend node and convert into a poseasset by name in the right-click menu.
}

EAnimAssetHandlerType UAnimGraphNode_PoseByName::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UPoseAsset::StaticClass()))
	{
		return EAnimAssetHandlerType::Supported;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

#undef LOCTEXT_NAMESPACE
