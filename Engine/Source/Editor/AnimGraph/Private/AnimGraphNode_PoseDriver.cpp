// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_PoseDriver.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimNodeEditModes.h"
#include "RBF/RBFSolver.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_PoseDriver)

#define LOCTEXT_NAMESPACE "PoseDriver"

struct FPoseDriverCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		// Add RBFData
		AddRBFData = 1,
		// Add multi-bone input support
		MultiBoneInput = 2,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FPoseDriverCustomVersion() {}
};

const FGuid FPoseDriverCustomVersion::GUID(0xAFE08691, 0x3A0D4952, 0xB673673B, 0x7CF22D1E);
FCustomVersionRegistration GRegisterPoseDriverCustomVersion(FPoseDriverCustomVersion::GUID, FPoseDriverCustomVersion::LatestVersion, TEXT("PoseDriverVer"));


//////////////////////////////////////////////////////////////////////////

UAnimGraphNode_PoseDriver::UAnimGraphNode_PoseDriver(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer)
{
	SelectedTargetIndex = INDEX_NONE;
	AxisLength = 20.0f;
	bDrawDebugCones = true;
	ConeSubdivision = 32;
}

FText UAnimGraphNode_PoseDriver::GetTooltipText() const
{
	return LOCTEXT("UAnimGraphNode_PoseDriver_ToolTip", "Drive parameters base on a bones distance from a set of defined poses.");
}

FText UAnimGraphNode_PoseDriver::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	static const FText DescriptionAll = LOCTEXT("PoseDriver", "Pose Driver");
	static const FText DescriptionSolo = LOCTEXT("PoseDriverSolo", "Pose Driver [Solo]");

	const FText& Description = Node.SoloTargetIndex == INDEX_NONE ? DescriptionAll : DescriptionSolo;

	const FName FirstSourceBone = (Node.SourceBones.Num() > 0) ? Node.SourceBones[0].BoneName : NAME_None;
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (FirstSourceBone == NAME_None))
	{
		return Description;
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), Description);
		const int32 TotalNumBones = Node.SourceBones.Num();
		Args.Add(TEXT("FirstSourceBone"), FText::FromName(FirstSourceBone));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_PoseDriver_ListTitle", "{ControllerDescription} - Source: {FirstSourceBone}"), Args), this);
		}
		else
		{
			if (TotalNumBones > 1)
			{
				Args.Add(TEXT("MoreJoint"), FText::AsNumber(TotalNumBones - 1));
				CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_PoseDriver_TitleWithBones", "{ControllerDescription}\nSource: {FirstSourceBone} and {MoreJoint} more"), Args), this);
			}
			else
			{
				CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_PoseDriver_Title", "{ControllerDescription}\nSource: {FirstSourceBone}"), Args), this);
			}
		}
	}
	return CachedNodeTitles[TitleType];
}

FText UAnimGraphNode_PoseDriver::GetMenuCategory() const
{
	return LOCTEXT("PoseAssetCategory_Label", "Animation|Poses");
}


FLinearColor UAnimGraphNode_PoseDriver::GetNodeBodyTintColor() const
{
	return Node.SoloTargetIndex == INDEX_NONE ?
		Super::GetNodeBodyTintColor() :
		FLinearColor::Green;
}

void UAnimGraphNode_PoseDriver::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	if (Node.SourceBones.Num() == 0)
	{
		MessageLog.Warning(*LOCTEXT("PoseDriver_NoSourceBone", "@@ - You must specify at least one Source Bone").ToString(), this);
	}

	FName MissingBoneName = NAME_None;
	if (ForSkeleton)
	{
		for (const FBoneReference& BoneRef : Node.SourceBones)
		{
			if (ForSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneRef.BoneName) == INDEX_NONE)
			{
				MissingBoneName = BoneRef.BoneName;
				break;
			}
		}
	}

	if (MissingBoneName != NAME_None)
	{
		MessageLog.Warning(*LOCTEXT("SourceBoneNotFound", "@@ - Entry in SourceBones not found").ToString(), this);
	}
	
	TArray<FRBFTarget> RBFTargets;
	Node.GetRBFTargets(RBFTargets, nullptr);
	TArray<int> InvalidTargets;
	if (!FRBFSolver::ValidateTargets(Node.RBFParams, RBFTargets, InvalidTargets))
	{
		for (int TargetIdx : InvalidTargets)
		{
			MessageLog.Error(*LOCTEXT("PoseDriver_InvalidTarget", "@@ - '@@' is an invalid or duplicate target.").ToString(),
				this, GetData(Node.PoseTargets[TargetIdx].DrivenName.ToString()));
		}
	}

	if (Node.SoloTargetIndex != INDEX_NONE)
	{
		MessageLog.Warning(*LOCTEXT("PoseDriver_SoloEnabled", "@@ - Solo enabled on target '@@'").ToString(),
			this, GetData(Node.PoseTargets[Node.SoloTargetIndex].DrivenName.ToString()));
	}

	// Note: UAnimGraphNode_PoseHandler::ValidateAnimNodeDuringCompilation checks if PoseAsset is valid, 
	// This check is only necessary when using DrivePoses
	if (Node.DriveOutput == EPoseDriverOutput::DrivePoses)
	{
		Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
	}
	else
	{
		UAnimGraphNode_AssetPlayerBase::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
	}
}

FEditorModeID UAnimGraphNode_PoseDriver::GetEditorMode() const
{
	return AnimNodeEditModes::PoseDriver;
}

EAnimAssetHandlerType UAnimGraphNode_PoseDriver::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UPoseAsset::StaticClass()))
	{
		return EAnimAssetHandlerType::Supported;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

void UAnimGraphNode_PoseDriver::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FPoseDriverCustomVersion::GUID) < FPoseDriverCustomVersion::MultiBoneInput)
	{
		if (Node.SourceBone_DEPRECATED.BoneName != NAME_None)
		{
			Node.SourceBones.Add(Node.SourceBone_DEPRECATED);
		}
	}

	if (GetLinkerCustomVersion(FPoseDriverCustomVersion::GUID) < FPoseDriverCustomVersion::AddRBFData)
	{
		// since this is postload, sometimes pose asset post load isn't finished yet
		// we mmake sure it finishes since this needs post info
		if (Node.PoseAsset)
		{
			Node.PoseAsset->ConditionalPostLoad();
		}

		// Convert distance method
		if (Node.Type_DEPRECATED == EPoseDriverType::SwingAndTwist)
		{
			Node.DriveSource = EPoseDriverSource::Rotation;
			Node.RBFParams.DistanceMethod = ERBFDistanceMethod::Quaternion;
		}
		else if (Node.Type_DEPRECATED == EPoseDriverType::SwingOnly)
		{
			Node.DriveSource = EPoseDriverSource::Rotation;
			Node.RBFParams.DistanceMethod = ERBFDistanceMethod::SwingAngle;
		}
		else
		{
			Node.DriveSource = EPoseDriverSource::Translation;
			Node.RBFParams.DistanceMethod = ERBFDistanceMethod::Euclidean;
		}

		// Copy twist axis
		Node.RBFParams.TwistAxis = Node.TwistAxis_DEPRECATED;

		// Copy target data from pose asset
		CopyTargetsFromPoseAsset();

		// Set per-target scales
		float MaxDistance;
		AutoSetTargetScales(MaxDistance);

		// Set radiust to be max distance, and apply old overall radius scaling
		Node.RBFParams.Radius = MaxDistance * Node.RadialScaling_DEPRECATED;

		// Recompile if required to propagate changes to AnimInstance Class
		UAnimBlueprint* AnimBP = GetAnimBlueprint();
		if (AnimBP)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(AnimBP);
		}
	}
}


void UAnimGraphNode_PoseDriver::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FPoseDriverCustomVersion::GUID);
}

void UAnimGraphNode_PoseDriver::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) {
	FAnimNode_PoseDriver* PreviewPoseDriver = static_cast<FAnimNode_PoseDriver*>(InPreviewNode);

	PreviewPoseDriver->RBFParams.SolverType = Node.RBFParams.SolverType;
	PreviewPoseDriver->RBFParams.Radius = Node.RBFParams.Radius;
	PreviewPoseDriver->RBFParams.bAutomaticRadius = Node.RBFParams.bAutomaticRadius;
	PreviewPoseDriver->RBFParams.Function = Node.RBFParams.Function;
	PreviewPoseDriver->RBFParams.DistanceMethod = Node.RBFParams.DistanceMethod;
	PreviewPoseDriver->RBFParams.TwistAxis = Node.RBFParams.TwistAxis;
	PreviewPoseDriver->RBFParams.WeightThreshold = Node.RBFParams.WeightThreshold;
	PreviewPoseDriver->RBFParams.NormalizeMethod = Node.RBFParams.NormalizeMethod;
	PreviewPoseDriver->RBFParams.MedianReference = Node.RBFParams.MedianReference;
	PreviewPoseDriver->RBFParams.MedianMin = Node.RBFParams.MedianMin;
	PreviewPoseDriver->RBFParams.MedianMax = Node.RBFParams.MedianMax;
	PreviewPoseDriver->DriveOutput = Node.DriveOutput;
	PreviewPoseDriver->DriveSource = Node.DriveSource;
	PreviewPoseDriver->PoseTargets = Node.PoseTargets;
	PreviewPoseDriver->bCachedDrivenIDsAreDirty = true;
	PreviewPoseDriver->SoloTargetIndex = Node.SoloTargetIndex;
	PreviewPoseDriver->bSoloDrivenOnly = Node.bSoloDrivenOnly;
}

FAnimNode_PoseDriver* UAnimGraphNode_PoseDriver::GetPreviewPoseDriverNode() const
{
	FAnimNode_PoseDriver* PreviewNode = nullptr;
	USkeletalMeshComponent* Component = nullptr;

	// look for a valid component in the object being debugged,
	// we might be set to something other than the preview.
	UObject* ObjectBeingDebugged = GetAnimBlueprint()->GetObjectBeingDebugged();
	if (ObjectBeingDebugged)
	{
		UAnimInstance* InstanceBeingDebugged = Cast<UAnimInstance>(ObjectBeingDebugged);
		if (InstanceBeingDebugged)
		{
			Component = InstanceBeingDebugged->GetSkelMeshComponent();
		}
	}

	// Fall back to the LastPreviewComponent
	if (Component == nullptr)
	{
		Component = LastPreviewComponent;
	}
	if (Component != nullptr && Component->GetAnimInstance() != nullptr)
	{
		PreviewNode = static_cast<FAnimNode_PoseDriver*>(FindDebugAnimNode(Component));
	}

	return PreviewNode;
}

void UAnimGraphNode_PoseDriver::CopyTargetsFromPoseAsset()
{
	UPoseAsset* PoseAsset = Node.PoseAsset;

	// store the previous targets for re-use
	TMap< FName, FPoseDriverTarget > PreviousTargets;
	for (int32 i = 0; i < Node.PoseTargets.Num(); i++)
	{
		PreviousTargets.Add(Node.PoseTargets[i].DrivenName, Node.PoseTargets[i]);
	}

	if (PoseAsset && PoseAsset->GetSkeleton()) // Use PoseAsset here not CurrentPoseAsset, because we want to be able to run this on on nodes that have not been init'd yet
	{
		Node.PoseTargets.Empty();

		// For each pose we create a target
		const TArray<FName>& PoseNames = PoseAsset->GetPoseFNames();
		for (int32 PoseIdx = 0; PoseIdx < PoseAsset->GetNumPoses(); PoseIdx++)
		{
			FPoseDriverTarget PoseTarget;
			PoseTarget.DrivenName = PoseNames[PoseIdx];

			// Create entry for each bone
			for (const FBoneReference& SourceBoneRef : Node.SourceBones)
			{
				FTransform SourceBoneTransform = FTransform::Identity;

				// Don't want to create target for base pose in additive case
				bool bIsBasePose = (PoseAsset->IsValidAdditive() && PoseIdx == PoseAsset->GetBasePoseIndex());
				if (!bIsBasePose)
				{
					// Get transforms from pose (this also converts from additive if necessary)
					TArray<FTransform> PoseTransforms;
					if (PoseAsset->GetFullPose(PoseIdx, PoseTransforms))
					{
						// If eval'ing in different space (and that space is valid)
						if (Node.EvalSpaceBone.BoneName != NAME_None)
						{
							FTransform SourceCompSpace = PoseAsset->GetComponentSpaceTransform(SourceBoneRef.BoneName, PoseTransforms);
							FTransform EvalCompSpace = PoseAsset->GetComponentSpaceTransform(Node.EvalSpaceBone.BoneName, PoseTransforms);

							SourceBoneTransform = SourceCompSpace.GetRelativeTransform(EvalCompSpace);
						}
						else
						{
							// Check we have a track for the source bone
							int32 SourceTrackIndex = PoseAsset->GetTrackIndexByName(SourceBoneRef.BoneName);
							if (SourceTrackIndex != INDEX_NONE)
							{
								SourceBoneTransform = PoseTransforms[SourceTrackIndex];
							}
						}
					}
				}

				// If we got a valid transform, add a pose target now
				FPoseDriverTransform PoseTransform;
				PoseTransform.TargetTranslation = SourceBoneTransform.GetTranslation();
				PoseTransform.TargetRotation = SourceBoneTransform.Rotator();

				PoseTarget.BoneTransforms.Add(PoseTransform);
			}

			// re-apply the same setting in case we have seen this target before
			const FPoseDriverTarget* PreviousTarget = PreviousTargets.Find(PoseTarget.DrivenName);
			if (PreviousTarget)
			{
				PoseTarget.TargetScale = PreviousTarget->TargetScale;
				PoseTarget.bApplyCustomCurve = PreviousTarget->bApplyCustomCurve;
				PoseTarget.CustomCurve = PreviousTarget->CustomCurve;
				PoseTarget.DistanceMethod = PreviousTarget->DistanceMethod;
				PoseTarget.FunctionType = PreviousTarget->FunctionType;
				PoseTarget.bIsHidden = PreviousTarget->bIsHidden;
			}

			Node.PoseTargets.Add(PoseTarget);
		}

		Node.bCachedDrivenIDsAreDirty = true;
	}
}

void UAnimGraphNode_PoseDriver::SetSourceBones(const TArray<FName>& BoneNames)
{
	Node.SourceBones.Empty(BoneNames.Num());
	for (const FName& BoneName : BoneNames)
	{
		Node.SourceBones.Add(BoneName);
	}
}

void UAnimGraphNode_PoseDriver::GetSourceBoneNames(TArray<FName>& BoneNames)
{
	for (const FBoneReference& SourceBone : Node.SourceBones)
	{
		BoneNames.Add(SourceBone.BoneName);
	}
}

void UAnimGraphNode_PoseDriver::SetDrivingBones(const TArray<FName>& BoneNames)
{
	Node.OnlyDriveBones.Empty(BoneNames.Num());
	for (const FName& BoneName : BoneNames)
	{
		Node.OnlyDriveBones.Add(BoneName);
	}
}

void UAnimGraphNode_PoseDriver::GetDrivingBoneNames(TArray<FName>& BoneNames)
{
	for (const FBoneReference& SourceBone : Node.OnlyDriveBones)
	{
		BoneNames.Add(SourceBone.BoneName);
	}
}

void UAnimGraphNode_PoseDriver::SetRBFParameters(FRBFParams Parameters)
{
	Node.RBFParams = Parameters;
}

FRBFParams& UAnimGraphNode_PoseDriver::GetRBFParameters()
{
	return Node.RBFParams;
}

void UAnimGraphNode_PoseDriver::SetPoseDriverSource(EPoseDriverSource DriverSource)
{
	Node.DriveSource = DriverSource;	
}

EPoseDriverSource& UAnimGraphNode_PoseDriver::GetPoseDriverSource()
{
	return Node.DriveSource;
}

void UAnimGraphNode_PoseDriver::SetPoseDriverOutput(EPoseDriverOutput DriverOutput)
{
	Node.DriveOutput = DriverOutput;
}

EPoseDriverOutput& UAnimGraphNode_PoseDriver::GetPoseDriverOutput()
{
	return Node.DriveOutput;
}

void UAnimGraphNode_PoseDriver::AddNewTarget()
{
	FPoseDriverTarget& NewTarget = Node.PoseTargets[Node.PoseTargets.Add(FPoseDriverTarget())];

	// Create entry for each bone
	NewTarget.BoneTransforms.AddDefaulted(Node.SourceBones.Num());
}


void UAnimGraphNode_PoseDriver::ReserveTargetTransforms()
{
	// reallocate transforms array in each target
	for (FPoseDriverTarget& PoseTarget : Node.PoseTargets)
	{
		PoseTarget.BoneTransforms.SetNum(Node.SourceBones.Num());
	}
}

FLinearColor UAnimGraphNode_PoseDriver::GetColorFromWeight(float InWeight)
{
	return FMath::Lerp(FLinearColor::Blue.Desaturate(0.5), FLinearColor::Red, InWeight);
}

void UAnimGraphNode_PoseDriver::AutoSetTargetScales(float& OutMaxDistance)
{
	if (LastPreviewComponent && LastPreviewComponent->AnimScriptInstance)
	{
		const FBoneContainer& RequiredBones = LastPreviewComponent->AnimScriptInstance->GetRequiredBones();

		TArray<FRBFTarget> RBFTargets;
		Node.GetRBFTargets(RBFTargets, &RequiredBones);

		// Find distances from targets to nearest neighbours
		TArray<float> Distances;
		bool bSuccess = FRBFSolver::FindTargetNeighbourDistances(Node.RBFParams, RBFTargets, Distances);
		if (bSuccess)
		{
			// Find overall largest distance 
			OutMaxDistance = KINDA_SMALL_NUMBER; // ensure result > 0
			for (float Distance : Distances)
			{
				OutMaxDistance = FMath::Max(OutMaxDistance, Distance);
			}

			// Set scales so largest distance is 1.0, and others are less than that
			for (int32 TargetIdx = 0; TargetIdx < Node.PoseTargets.Num(); TargetIdx++)
			{
				FPoseDriverTarget& PoseTarget = Node.PoseTargets[TargetIdx];
				PoseTarget.TargetScale = Distances[TargetIdx] / OutMaxDistance;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
