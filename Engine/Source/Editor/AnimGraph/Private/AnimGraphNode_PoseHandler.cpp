// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_PoseHandler.h"
#include "Animation/PoseAsset.h"
#include "AnimNodes/AnimNode_PoseHandler.h"
#include "Kismet2/CompilerResultsLog.h"
#include "IAnimBlueprintNodeOverrideAssetsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_PoseHandler)

UAnimGraphNode_PoseHandler::UAnimGraphNode_PoseHandler(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer)
{

}

void UAnimGraphNode_PoseHandler::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
	
	ValidateAnimNodeDuringCompilationHelper(ForSkeleton, MessageLog, GetPoseHandlerNode()->PoseAsset, UPoseAsset::StaticClass(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseHandler, PoseAsset)), GET_MEMBER_NAME_CHECKED(FAnimNode_PoseHandler, PoseAsset));
}

void UAnimGraphNode_PoseHandler::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(Asset))
	{
		GetPoseHandlerNode()->PoseAsset = PoseAsset;
	}
}

void UAnimGraphNode_PoseHandler::OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const
{
	if(InContext.GetAssets().Num() > 0)
	{
		if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(InContext.GetAssets()[0]))
		{
			FAnimNode_PoseHandler& AnimNode = InContext.GetAnimNode<FAnimNode_PoseHandler>();
			AnimNode.SetPoseAsset(PoseAsset);
		}
	}
}

void UAnimGraphNode_PoseHandler::PreloadRequiredAssets()
{
	PreloadRequiredAssetsHelper(GetPoseHandlerNode()->PoseAsset, FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseHandler, PoseAsset)));

	Super::PreloadRequiredAssets();
}

UAnimationAsset* UAnimGraphNode_PoseHandler::GetAnimationAsset() const
{
	UPoseAsset* PoseAsset = GetPoseHandlerNode()->PoseAsset;
	UEdGraphPin* PoseAssetPin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_PoseHandler, PoseAsset));
	if (PoseAssetPin != nullptr && PoseAsset == nullptr)
	{
		PoseAsset = Cast<UPoseAsset>(PoseAssetPin->DefaultObject);
	}

	return PoseAsset;
}

TSubclassOf<UAnimationAsset> UAnimGraphNode_PoseHandler::GetAnimationAssetClass() const
{
	return UPoseAsset::StaticClass();
}
