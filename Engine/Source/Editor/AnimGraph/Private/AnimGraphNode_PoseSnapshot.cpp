// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_PoseSnapshot.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_LocalCachePose

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_PoseSnapshot)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_PoseSnapshot::UAnimGraphNode_PoseSnapshot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_PoseSnapshot::GetTooltipText() const
{
	return LOCTEXT("UAnimGraphNode_PoseSnapshot_Tooltip", "Returns local space pose snapshot.");
}

FText UAnimGraphNode_PoseSnapshot::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UAnimGraphNode_PoseSnapshot_Title", "Pose Snapshot");
}

FText UAnimGraphNode_PoseSnapshot::GetMenuCategory() const
{
	return LOCTEXT("UAnimGraphNode_PoseSnapshot_Category", "Animation|Poses");
}

#undef LOCTEXT_NAMESPACE
