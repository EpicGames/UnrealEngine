// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RandomPlayer.h"

#include "EditorCategoryUtils.h"
#include "Animation/AnimAttributes.h"
#include "Animation/AnimRootMotionProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RandomPlayer)

#define LOCTEXT_NAMESPACE "AnimGraphNode_RandomPlayer"

FLinearColor UAnimGraphNode_RandomPlayer::GetNodeTitleColor() const
{
	return FLinearColor(0.10f, 0.60f, 0.12f);
}

FText UAnimGraphNode_RandomPlayer::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Plays sequences picked from a provided list in random orders.");
}

FText UAnimGraphNode_RandomPlayer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Random Sequence Player");
}

FText UAnimGraphNode_RandomPlayer::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Sequences");
}

void UAnimGraphNode_RandomPlayer::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
	OutAttributes.Add(UE::Anim::FAttributes::Attributes);

	if (UE::Anim::IAnimRootMotionProvider::Get())
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_RandomPlayer::PreloadRequiredAssets()
{
	for (const FRandomPlayerSequenceEntry& Entry : Node.Entries)
	{
		PreloadObject(Entry.Sequence);
	}

	Super::PreloadRequiredAssets();
}

#undef LOCTEXT_NAMESPACE
