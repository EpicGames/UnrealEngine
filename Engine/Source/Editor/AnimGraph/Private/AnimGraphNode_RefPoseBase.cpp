// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RefPoseBase.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_RefPoseBase

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RefPoseBase)

UAnimGraphNode_RefPoseBase::UAnimGraphNode_RefPoseBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UAnimGraphNode_RefPoseBase::GetNodeCategory() const
{
	return TEXT("Animation|Poses");
}

FLinearColor UAnimGraphNode_RefPoseBase::GetNodeTitleColor() const
{
	if ( Node.GetRefPoseType() == EIT_Additive )
	{
		return FLinearColor(0.10f, 0.60f, 0.12f);
	}
	else
	{
		return FColor(200, 100, 100);
	}
}
