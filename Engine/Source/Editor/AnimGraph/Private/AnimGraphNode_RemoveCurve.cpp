// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RemoveCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RemoveCurve)

#define LOCTEXT_NAMESPACE "RemoveCurve"

UAnimGraphNode_RemoveCurve::UAnimGraphNode_RemoveCurve()
{
}

FText UAnimGraphNode_RemoveCurve::GetMenuCategory() const
{
	return LOCTEXT("AnimGraphNode_RemoveCurve_Category", "Animation|Curves");
}

FText UAnimGraphNode_RemoveCurve::GetTooltipText() const
{	
	return LOCTEXT("AnimGraphNode_RemoveCurve_Tooltip", "Removes animation curves from the pose input");
}

FText UAnimGraphNode_RemoveCurve::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_RemoveCurve_Title", "Remove Curve");
}

#undef LOCTEXT_NAMESPACE