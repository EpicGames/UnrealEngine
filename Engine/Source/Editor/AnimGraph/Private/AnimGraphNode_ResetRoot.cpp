// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ResetRoot.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ResetRoot

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ResetRoot)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_ResetRoot::UAnimGraphNode_ResetRoot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_ResetRoot::GetControllerDescription() const
{
	return LOCTEXT("ResetRoot", "Reset Root Transform");
}

FText UAnimGraphNode_ResetRoot::GetTooltipText() const
{
	return LOCTEXT("UAnimGraphNode_ResetRoot_Tooltip", "Reset Root Transform, but maintain Children Transforms in Component Space.");
}

FText UAnimGraphNode_ResetRoot::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

void UAnimGraphNode_ResetRoot::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}


#undef LOCTEXT_NAMESPACE
