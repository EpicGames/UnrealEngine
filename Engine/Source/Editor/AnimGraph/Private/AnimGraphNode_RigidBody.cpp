// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RigidBody.h"
#include "AnimNodeEditModes.h"
#include "Features/IModularFeatures.h"
#include "Kismet2/CompilerResultsLog.h"
#include "BoneControllers/AnimNode_RigidBody.h"
#include "EditorModeManager.h"
#include "IPhysicsAssetRenderInterface.h"
#include "IAnimNodeEditMode.h"

// Details includes
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_RigidBody

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RigidBody)

#define LOCTEXT_NAMESPACE "RigidBody"

UAnimGraphNode_RigidBody::UAnimGraphNode_RigidBody(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_RigidBody::GetControllerDescription() const
{
	return LOCTEXT("AnimGraphNode_RigidBody_ControllerDescription", "Rigid body simulation for physics asset");
}

FText UAnimGraphNode_RigidBody::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_RigidBody_Tooltip", "This simulates based on the skeletal mesh component's physics asset");
}

FText UAnimGraphNode_RigidBody::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("AnimGraphNode_RigidBody_NodeTitle", "RigidBody"));
}

FString UAnimGraphNode_RigidBody::GetNodeCategory() const
{
	return TEXT("Animation|Dynamics");
}

void UAnimGraphNode_RigidBody::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

void UAnimGraphNode_RigidBody::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp, const bool bIsSelected, const bool bIsPoseWatchEnabled) const
{
	if (const FAnimNode_RigidBody* const RuntimeRigidBodyNode = GetDebuggedAnimNode<FAnimNode_RigidBody>())
	{
		if (UPhysicsAsset* const PhysicsAsset = RuntimeRigidBodyNode->GetPhysicsAsset())
		{
			IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());

			// Draw Bodies.
			if (bIsSelected || (bIsPoseWatchEnabled && PoseWatchElementBodies.IsValid() && PoseWatchElementBodies->GetIsVisible()))
			{
				FColor PrimitiveColorOverride = FColor::Transparent;

				// Get primitive color from pose watch component.
				if (!bIsSelected)
				{
					PrimitiveColorOverride = PoseWatchElementBodies->GetColor();
					PrimitiveColorOverride.A = 255;
				}

				PhysicsAssetRenderInterface.DebugDrawBodies(PreviewSkelMeshComp, PhysicsAsset, PDI, PrimitiveColorOverride);
			}

			// Draw Constraints.
			if (bIsSelected || (bIsPoseWatchEnabled && PoseWatchElementConstraints.IsValid() && PoseWatchElementConstraints->GetIsVisible()))
			{
				PhysicsAssetRenderInterface.DebugDrawConstraints(PreviewSkelMeshComp, PhysicsAsset, PDI);
			}
		}
	}
}

void UAnimGraphNode_RigidBody::OnPoseWatchChanged(const bool IsPoseWatchEnabled, TObjectPtr<UPoseWatch> InPoseWatch, FEditorModeTools& InModeTools, FAnimNode_Base* InRuntimeNode)
{
	Super::OnPoseWatchChanged(IsPoseWatchEnabled, InPoseWatch, InModeTools, InRuntimeNode);

	UPoseWatch* const PoseWatch = InPoseWatch.Get();

	if (PoseWatch)
	{
		// A new pose watch has been created for this node - add node specific pose watch components.
		PoseWatchElementBodies = InPoseWatch.Get()->FindOrAddElement(FText(LOCTEXT("PoseWatchElementLabel_RigidBody_PhysicsBodies", "Physics Bodies")), TEXT("PhysicsAssetEditor.Tree.Body"));
		PoseWatchElementConstraints = InPoseWatch.Get()->FindOrAddElement(FText(LOCTEXT("PoseWatchElementLabel__RigidBody_PhysicsConstraints", "Physics Constraints")), TEXT("PhysicsAssetEditor.Tree.Constraint"));

		check(PoseWatchElementConstraints.IsValid()); // Expect to find a valid component;
		if (PoseWatchElementConstraints.IsValid())
		{
			PoseWatchElementConstraints->SetHasColor(false);
		}
	}
}

void UAnimGraphNode_RigidBody::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	IDetailCategoryBuilder& ViewportCategory = DetailBuilder.EditCategory(TEXT("Debug Visualization"));
	FDetailWidgetRow& WidgetRow = ViewportCategory.AddCustomRow(LOCTEXT("ToggleDebugVisualizationButtonRow", "DebugVisualization"));
	FAnimNode_RigidBody* const RigidBodyNode = static_cast<FAnimNode_RigidBody*>(GetDebuggedAnimNode());

	WidgetRow
		[
			SNew(SHorizontalBox)
			// Show/Hide Bodies button.
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this](){ this->ToggleBodyVisibility(); return FReply::Handled(); })
				.ButtonColorAndOpacity_Lambda([this](){ return (AreAnyBodiesHidden()) ? FAppStyle::Get().GetSlateColor("Colors.AccentRed") : FAppStyle::Get().GetSlateColor("Colors.AccentGreen"); })
				.Content()
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return (AreAnyBodiesHidden()) ? LOCTEXT("ShowAllBodiesButtonText", "Show All Bodies") : LOCTEXT("HideAllBodiesButtonText", "Hide All Bodies"); })
					.ToolTipText(LOCTEXT("ToggleBodyVisibilityButtonToolTip", "Toggle debug visualization of all physics bodies"))
				]
			]
			// Show/Hide Constraints button.
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([this](){ this->ToggleConstraintVisibility(); return FReply::Handled(); })
				.ButtonColorAndOpacity_Lambda([this]() { return (AreAnyConstraintsHidden()) ? FAppStyle::Get().GetSlateColor("Colors.AccentRed") : FAppStyle::Get().GetSlateColor("Colors.AccentGreen"); })
				.Content()
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return (AreAnyConstraintsHidden()) ? LOCTEXT("ShowAllConstraintsButtonText", "Show All Constraints") : LOCTEXT("HideAllConstraintsButtonText", "Hide All Constraints"); })
					.ToolTipText(LOCTEXT("ToggleConstraintVisibilityButtonToolTip", "Toggle debug visualization of all physics constriants"))
				]
			]
		];

	// Add per node check box for allowing bone transform scale propagation. This check box is hidden unless the the feature is enabled globally via CVar.
	{
		IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Settings"));
		TSharedRef<IPropertyHandle> AllowPoseScalePropagationHandle = DetailBuilder.GetProperty(TEXT("Node.bAllowPoseScalePropagation"), GetClass());

		TAttribute<EVisibility> VisiblityAttr = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UAnimGraphNode_RigidBody::ShouldShowPoseScalePropagationSettings));

		SettingsCategory.AddCustomRow(AllowPoseScalePropagationHandle->GetPropertyDisplayName(), true)
			.NameContent()
			[
				AllowPoseScalePropagationHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				AllowPoseScalePropagationHandle->CreatePropertyValueWidget()
			]
			.Visibility(VisiblityAttr);

		AllowPoseScalePropagationHandle->MarkHiddenByCustomization();
	}
}

void UAnimGraphNode_RigidBody::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());
	PhysicsAssetRenderInterface.SaveConfig();
}

void UAnimGraphNode_RigidBody::ToggleBodyVisibility()
{
	FAnimNode_RigidBody* const RigidBodyNode = static_cast<FAnimNode_RigidBody*>(GetDebuggedAnimNode());
	
	IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());

	if (RigidBodyNode)
	{
		PhysicsAssetRenderInterface.ToggleShowAllBodies(RigidBodyNode->GetPhysicsAsset());
	}
}

void UAnimGraphNode_RigidBody::ToggleConstraintVisibility()
{
	FAnimNode_RigidBody* const RigidBodyNode = static_cast<FAnimNode_RigidBody*>(GetDebuggedAnimNode());
	IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());

	if (RigidBodyNode)
	{
		PhysicsAssetRenderInterface.ToggleShowAllConstraints(RigidBodyNode->GetPhysicsAsset());
	}
}

bool UAnimGraphNode_RigidBody::AreAnyBodiesHidden() const
{
	FAnimNode_RigidBody* const RigidBodyNode = static_cast<FAnimNode_RigidBody*>(GetDebuggedAnimNode());
	IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());

	if (RigidBodyNode)
	{
		return PhysicsAssetRenderInterface.AreAnyBodiesHidden(RigidBodyNode->GetPhysicsAsset());
	}

	return false;
}

bool UAnimGraphNode_RigidBody::AreAnyConstraintsHidden() const
{
	FAnimNode_RigidBody* const RigidBodyNode = static_cast<FAnimNode_RigidBody*>(GetDebuggedAnimNode());
	IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());

	if (RigidBodyNode)
	{
		return PhysicsAssetRenderInterface.AreAnyConstraintsHidden(RigidBodyNode->GetPhysicsAsset());
	}

	return false;
}

EVisibility UAnimGraphNode_RigidBody::ShouldShowPoseScalePropagationSettings()
{
	return RigidBodyNodeIsPoseScalePropagationEnabledGlobally() ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
