// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_Root.h"
#include "GraphEditorSettings.h"
#include "AnimBlueprintCompiler.h"
#include "IAnimBlueprintCompilationContext.h"
#include "IAnimBlueprintCopyTermDefaultsContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

/////////////////////////////////////////////////////
// FPoseLinkMappingRecord

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Root)

void FPoseLinkMappingRecord::PatchLinkIndex(uint8* DestinationPtr, int32 LinkID, int32 SourceLinkID) const
{
	checkSlow(IsValid());

	DestinationPtr = ChildProperty->ContainerPtrToValuePtr<uint8>(DestinationPtr);
	
	if (ChildPropertyIndex != INDEX_NONE)
	{
		FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(ChildProperty);

		FScriptArrayHelper ArrayHelper(ArrayProperty, DestinationPtr);
		check(ArrayHelper.IsValidIndex(ChildPropertyIndex));

		DestinationPtr = ArrayHelper.GetRawPtr(ChildPropertyIndex);
	}

	// Check to guard against accidental infinite loops
	check((LinkID == INDEX_NONE) || (LinkID != SourceLinkID));

	// Patch the pose link
	FPoseLinkBase& PoseLink = *((FPoseLinkBase*)DestinationPtr);
	PoseLink.LinkID = LinkID;
	PoseLink.SourceLinkID = SourceLinkID;
}

/////////////////////////////////////////////////////
// UAnimGraphNode_Root

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_Root::UAnimGraphNode_Root(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_Root::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	const int32 CustomAnimVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && CustomAnimVersion < FFortniteMainBranchObjectVersion::AnimNodeRootDefaultGroupChange)
	{
#if WITH_EDITORONLY_DATA
		Node.LayerGroup = Node.Group_DEPRECATED;
#endif
	}
}

FLinearColor UAnimGraphNode_Root::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText UAnimGraphNode_Root::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText DefaultTitle = LOCTEXT("AnimGraphNodeRoot_Title", "Output Pose");

	if(TitleType != ENodeTitleType::FullTitle)
	{
		return DefaultTitle;
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), DefaultTitle);
		Args.Add(TEXT("Name"), FText::FromString(GetOuter()->GetName()));

		return FText::Format(LOCTEXT("AnimGraphNodeRoot_TitleNamed", "{NodeTitle}\n{Name}"), Args);
	}
}

FText UAnimGraphNode_Root::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNodeRoot_Tooltip", "Wire the final animation pose for this graph into this node");
}

bool UAnimGraphNode_Root::IsPoseWatchable() const
{
	return false;
}

bool UAnimGraphNode_Root::IsSinkNode() const
{
	return true;
}

void UAnimGraphNode_Root::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Intentionally empty. This node is auto-generated when a new graph is created.
}

FString UAnimGraphNode_Root::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/AnimationStateMachine");
}

void UAnimGraphNode_Root::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimGraphNode_Root* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_Root>(this);

	Node.SetName(TrueNode->GetGraph()->GetFName());
}

void UAnimGraphNode_Root::OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimGraphNode_Root* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_Root>(this);

	FAnimNode_Root* DestinationNode = reinterpret_cast<FAnimNode_Root*>(InPerNodeContext.GetDestinationPtr());
	DestinationNode->SetName(TrueNode->GetGraph()->GetFName());
}

#undef LOCTEXT_NAMESPACE
