// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RotateRootBone.h"


/////////////////////////////////////////////////////
// UAnimGraphNode_RotateRootBone

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RotateRootBone)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_RotateRootBone::UAnimGraphNode_RotateRootBone(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_RotateRootBone::GetNodeTitleColor() const
{
	return FLinearColor(0.7f, 0.7f, 0.7f);
}

FText UAnimGraphNode_RotateRootBone::GetTooltipText() const
{
	return LOCTEXT("RotateRootBone", "Rotate Root Bone");
}

FText UAnimGraphNode_RotateRootBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("RotateRootBone", "Rotate Root Bone");
}

FText UAnimGraphNode_RotateRootBone::GetMenuCategory() const
{
	return LOCTEXT("RotateRootBoneCateogory", "Animation|Misc.");
}

void UAnimGraphNode_RotateRootBone::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotateRootBone, Pitch))
	{
		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.PitchScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotateRootBone, Yaw))
	{
		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.YawScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_RotateRootBone::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#undef LOCTEXT_NAMESPACE
