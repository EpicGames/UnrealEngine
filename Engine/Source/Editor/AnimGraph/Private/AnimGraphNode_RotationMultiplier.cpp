// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RotationMultiplier.h"
/////////////////////////////////////////////////////
// UAnimGraphNode_RotationMultiplier

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RotationMultiplier)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_RotationMultiplier::UAnimGraphNode_RotationMultiplier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_RotationMultiplier::GetControllerDescription() const
{
	return LOCTEXT("ApplyPercentageOfRotation", "Apply a Percentage of Rotation");
}

FText UAnimGraphNode_RotationMultiplier::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_RotationMultiplier_Tooltip", "The Apply a Percentage of Rotation control drives the Rotation of a target bone at some specified percentage of the Rotation of another bone within the Skeleton.");
}

FText UAnimGraphNode_RotationMultiplier::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.TargetBone.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.TargetBone.BoneName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_ModifyBone_ListTitle", "{ControllerDescription} - Bone: {BoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_ModifyBone_Title", "{ControllerDescription}\nBone: {BoneName}"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

#undef LOCTEXT_NAMESPACE
