// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RotationOffsetBlendSpace.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "ToolMenus.h"
#include "GraphEditorActions.h"
#include "Kismet2/CompilerResultsLog.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimGraphCommands.h"
#include "BlueprintNodeTemplateCache.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAnimBlueprintNodeOverrideAssetsContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_RotationOffsetBlendSpace

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RotationOffsetBlendSpace)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_RotationOffsetBlendSpace"

UAnimGraphNode_RotationOffsetBlendSpace::UAnimGraphNode_RotationOffsetBlendSpace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_RotationOffsetBlendSpace::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.AlphaBoolBlend.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FText UAnimGraphNode_RotationOffsetBlendSpace::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UBlendSpace* BlendSpaceToCheck = Node.GetBlendSpace();
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, BlendSpace));
	if (BlendSpacePin != nullptr && BlendSpaceToCheck == nullptr)
	{
		BlendSpaceToCheck = Cast<UBlendSpace>(BlendSpacePin->DefaultObject);
	}

	if (BlendSpaceToCheck == nullptr)
	{
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			return LOCTEXT("RotationOffsetBlend_NONE_ListTitle", "AimOffset Player '(None)'");
		}
		else
		{
			return LOCTEXT("RotationOffsetBlend_NONE_Title", "(None)\nAimOffset Player");
		}
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		const FText BlendSpaceName = FText::FromString(BlendSpaceToCheck->GetName());

		FFormatNamedArguments Args;
		Args.Add(TEXT("BlendSpaceName"), BlendSpaceName);

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AimOffsetListTitle", "AimOffset Player '{BlendSpaceName}'"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AimOffsetFullTitle", "{BlendSpaceName}\nAimOffset Player"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

void UAnimGraphNode_RotationOffsetBlendSpace::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UAimOffsetBlendSpace::StaticClass(), UAimOffsetBlendSpace1D::StaticClass() },
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescFormat", "AimOffset Player '{0}'"), FText::FromName(InAssetData.AssetName));
			}
			else
			{
				return LOCTEXT("MenuDesc", "AimOffset Player");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescTooltipFormat", "AimOffset Player\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
			}
			else
			{
				return LOCTEXT("MenuDescTooltip", "AimOffset Player");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_AssetPlayerBase::SetupNewNode(InNewNode, bInIsTemplateNode, InAssetData);
		});
}

FBlueprintNodeSignature UAnimGraphNode_RotationOffsetBlendSpace::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddSubObject(Node.GetBlendSpace());

	return NodeSignature;
}

void UAnimGraphNode_RotationOffsetBlendSpace::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(Asset))
	{
		Node.SetBlendSpace(BlendSpace);
	}
}

void UAnimGraphNode_RotationOffsetBlendSpace::OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const
{
	if(InContext.GetAssets().Num() > 0)
	{
		if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(InContext.GetAssets()[0]))
		{
			FAnimNode_BlendSpacePlayer& AnimNode = InContext.GetAnimNode<FAnimNode_BlendSpacePlayer>();
			AnimNode.SetBlendSpace(BlendSpace);
		}
	}
}

void UAnimGraphNode_RotationOffsetBlendSpace::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	ValidateAnimNodeDuringCompilationHelper(ForSkeleton, MessageLog, Node.GetBlendSpace(), UBlendSpace::StaticClass(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, BlendSpace)), GET_MEMBER_NAME_CHECKED(FAnimNode_RotationOffsetBlendSpace, BlendSpace));

	UBlendSpace* BlendSpaceToCheck = Node.GetBlendSpace();
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, BlendSpace));
	if (BlendSpacePin != nullptr && BlendSpaceToCheck == nullptr)
	{
		BlendSpaceToCheck = Cast<UBlendSpace>(BlendSpacePin->DefaultObject);
	}

	if (BlendSpaceToCheck)
	{
		if (Cast<UAimOffsetBlendSpace>(BlendSpaceToCheck) == NULL &&
			Cast<UAimOffsetBlendSpace1D>(BlendSpaceToCheck) == NULL)
		{
			MessageLog.Error(TEXT("@@ references an invalid blend space (one that is not an aim offset)"), this);
		}
	}

	if (UAnimationSettings::Get()->bEnablePerformanceLog)
	{
		if (Node.LODThreshold < 0)
		{
			MessageLog.Warning(TEXT("@@ contains no LOD Threshold."), this);
		}
	}
}

void UAnimGraphNode_RotationOffsetBlendSpace::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to single frame
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeBlendSpacePlayer", LOCTEXT("BlendSpaceHeading", "Blend Space"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().OpenRelatedAsset);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToAimOffsetLookAt);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToAimOffsetGraph);
		}
	}
}

void UAnimGraphNode_RotationOffsetBlendSpace::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const
{
	if(Node.GetBlendSpace())
	{
		HandleAnimReferenceCollection(Node.BlendSpace, AnimationAssets);
	}
}

void UAnimGraphNode_RotationOffsetBlendSpace::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(Node.BlendSpace, AnimAssetReplacementMap);
}



EAnimAssetHandlerType UAnimGraphNode_RotationOffsetBlendSpace::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UBlendSpace::StaticClass()) && IsAimOffsetBlendSpace(AssetClass))
	{
		return EAnimAssetHandlerType::PrimaryHandler;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

void UAnimGraphNode_RotationOffsetBlendSpace::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, Alpha))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Float);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBias.GetFriendlyName(Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName));
		}
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, bAlphaBoolEnabled))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Bool);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaCurveName))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Curve);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_RotationOffsetBlendSpace::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaScaleBias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
	{
		ReconstructNode();
	}

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaInputType))
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeAlphaInputType", "Change Alpha Input Type"));
		Modify();

		// Break links to pins going away
		for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Pins[PinIndex];
			if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, Alpha))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Float)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, bAlphaBoolEnabled))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaCurveName))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
		}

		ReconstructNode();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_RotationOffsetBlendSpace::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RotationOffsetBlendSpace, bAlphaBoolEnabled)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaBoolBlend)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Float)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RotationOffsetBlendSpace, Alpha)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaScaleBias)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaCurveName)));
	}

	if ((Node.AlphaInputType != EAnimAlphaInputType::Float)
		&& (Node.AlphaInputType != EAnimAlphaInputType::Curve))
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RotationOffsetBlendSpace, AlphaScaleBiasClamp)));
	}
}

UAnimationAsset* UAnimGraphNode_RotationOffsetBlendSpace::GetAnimationAsset() const 
{
	UBlendSpace* BlendSpace = Node.GetBlendSpace();
	UEdGraphPin* BlendSpacePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RotationOffsetBlendSpace, BlendSpace));
	if (BlendSpacePin != nullptr && BlendSpace == nullptr)
	{
		BlendSpace = Cast<UBlendSpace>(BlendSpacePin->DefaultObject);
	}

	return BlendSpace;
}

#undef LOCTEXT_NAMESPACE
