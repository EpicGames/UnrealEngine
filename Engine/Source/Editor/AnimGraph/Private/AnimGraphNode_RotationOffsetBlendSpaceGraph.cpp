// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RotationOffsetBlendSpaceGraph.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "AnimGraphNodeAlphaOptions.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "DetailLayoutBuilder.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RotationOffsetBlendSpaceGraph)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_RotationOffsetBlendSpaceGraph"

FText UAnimGraphNode_RotationOffsetBlendSpaceGraph::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if(BlendSpaceGraph || BlendSpace)
	{
		const FText BlendSpaceName = FText::FromString(BlendSpaceGraph ? GetBlendSpaceGraphName() : GetBlendSpaceName());

		if(TitleType == ENodeTitleType::EditableTitle)
		{
			return BlendSpaceName;
		}
		else if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("BlendSpaceName"), BlendSpaceName);
			return FText::Format(LOCTEXT("AimOffsetListTitle", "AimOffset '{BlendSpaceName}'"), Args);
		}
		else
		{
			FFormatNamedArguments TitleArgs;
			TitleArgs.Add(TEXT("BlendSpaceName"), BlendSpaceName);
			FText Title = FText::Format(LOCTEXT("AimOffsetFullTitle", "{BlendSpaceName}\nAimOffset"), TitleArgs);

			if ((TitleType == ENodeTitleType::FullTitle) && (Node.GetGroupName() != NAME_None))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Title"), Title);
				Args.Add(TEXT("SyncGroupName"), FText::FromName(Node.GetGroupName()));
				Title = FText::Format(LOCTEXT("AimOffsetNodeGroupSubtitle", "{Title}\nSync group {SyncGroupName}"), Args);
			}

			return Title;
		}
	}
	else if(BlendSpaceClass.Get())
	{
		return BlendSpaceClass.Get()->GetDisplayNameText();
	}
	else
	{
		// Template node, so use the tooltip generated in GetMenuActions
		return FText::GetEmpty();
	}
}

void UAnimGraphNode_RotationOffsetBlendSpaceGraph::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	UAnimGraphNode_AssetPlayerBase::GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UAimOffsetBlendSpace::StaticClass(), UAimOffsetBlendSpace1D::StaticClass() },
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescFormat", "AimOffset '{0}'"), FText::FromName(InAssetData.AssetName));
			}
			else if(InClass != nullptr)
			{
				return InClass->GetDisplayNameText();
			}
			else
			{
				return LOCTEXT("MenuDesc", "AimOffset");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				return FText::Format(LOCTEXT("MenuDescTooltipFormat", "AimOffset\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
			}
			else if(InClass != nullptr)
			{
				return InClass->GetDisplayNameText();
			}
			else
			{
				return LOCTEXT("MenuDescTooltip", "AimOffset");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_RotationOffsetBlendSpaceGraph* GraphNode = CastChecked<UAnimGraphNode_RotationOffsetBlendSpaceGraph>(InNewNode);
			GraphNode->SetupFromAsset(InAssetData, bInIsTemplateNode);
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, TSubclassOf<UObject> InClass)
		{
			UAnimGraphNode_RotationOffsetBlendSpaceGraph* GraphNode = CastChecked<UAnimGraphNode_RotationOffsetBlendSpaceGraph>(InNewNode);
			GraphNode->SetupFromClass(TSubclassOf<UBlendSpace>(InClass.Get()), bInIsTemplateNode);
		});
}

void UAnimGraphNode_RotationOffsetBlendSpaceGraph::BakeDataDuringCompilation(FCompilerResultsLog& MessageLog)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	AnimBlueprint->FindOrAddGroup(Node.GetGroupName());
}

void UAnimGraphNode_RotationOffsetBlendSpaceGraph::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	FAnimGraphNodeAlphaOptions::HandleCustomizePinData(Node, Pin);
}

void UAnimGraphNode_RotationOffsetBlendSpaceGraph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FAnimGraphNodeAlphaOptions::HandlePostEditChangeProperty(Node, this, PropertyChangedEvent);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_RotationOffsetBlendSpaceGraph::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	Super::CustomizeDetails(InDetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = InDetailBuilder.GetProperty(TEXT("Node"), GetClass());

	FAnimGraphNodeAlphaOptions::HandleCustomizeDetails(Node, NodeHandle, InDetailBuilder);
}

#undef LOCTEXT_NAMESPACE
