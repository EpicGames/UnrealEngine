// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_SaveCachedPose.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "AnimBlueprintExtension_CachedPose.h"

/////////////////////////////////////////////////////
// FCachedPoseNameValidator

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_SaveCachedPose)

class FCachedPoseNameValidator : public FStringSetNameValidator
{
public:
	FCachedPoseNameValidator(const UBlueprint* InBlueprint, const UAnimGraphNode_SaveCachedPose* CurrentNode, const FString& InExistingName)
		: FStringSetNameValidator(InExistingName)
	{
		TArray<const UAnimGraphNode_SaveCachedPose*> Nodes;
		FBlueprintEditorUtils::GetAllNodesOfClass<const UAnimGraphNode_SaveCachedPose>(InBlueprint, Nodes);

		for (const UAnimGraphNode_SaveCachedPose* Node : Nodes)
		{
			if (Node != CurrentNode)
			{
				Names.Add(Node->CacheName);
			}
		}
	}
};

/////////////////////////////////////////////////////
// UAnimGraphNode_ComponentToLocalSpace

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_SaveCachedPose::UAnimGraphNode_SaveCachedPose(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
}

FText UAnimGraphNode_SaveCachedPose::GetTooltipText() const
{
	return LOCTEXT("SaveCachedPose_Tooltip", "Denotes an animation tree that can be referenced elsewhere in the blueprint, which will be evaluated at most once per frame and then cached.");
}

FText UAnimGraphNode_SaveCachedPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromString(CacheName);
	}
	else if ((TitleType == ENodeTitleType::MenuTitle) && CacheName.IsEmpty())
	{
		return LOCTEXT("NewSaveCachedPose", "New Save cached pose...");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), FText::FromString(CacheName));
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("AnimGraphNode_SaveCachedPose_Title", "Save cached pose '{NodeTitle}'"), Args), this);
	}
	return CachedNodeTitle;
}

FString UAnimGraphNode_SaveCachedPose::GetNodeCategory() const
{
	return TEXT("Animation|Cached Poses");
}

void UAnimGraphNode_SaveCachedPose::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode)
	{
		UAnimGraphNode_SaveCachedPose* CachedPoseNode = CastChecked<UAnimGraphNode_SaveCachedPose>(NewNode);
		// we use an empty CacheName in GetNodeTitle() to relay the proper menu title
		if (!bIsTemplateNode)
		{
			// @TODO: is the idea that this name is unique? what if Rand() hit twice? why not MakeUniqueObjectName()?			
			CachedPoseNode->CacheName = TEXT("SavedPose") + FString::FromInt(FMath::Rand());
		}
	};

	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

bool UAnimGraphNode_SaveCachedPose::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	//EGraphType GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
	//bool const bIsNotStateMachine = (GraphType != GT_StateMachine);

	bool const bIsNotStateMachine = TargetGraph->GetOuter()->IsA(UAnimBlueprint::StaticClass());
	return bIsNotStateMachine && Super::IsCompatibleWithGraph(TargetGraph);
}

void UAnimGraphNode_SaveCachedPose::OnRenameNode(const FString& NewName)
{
	CacheName = NewName;
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

TSharedPtr<class INameValidatorInterface> UAnimGraphNode_SaveCachedPose::MakeNameValidator() const
{
	return MakeShared<FCachedPoseNameValidator>(GetBlueprint(), this, CacheName);
}

void UAnimGraphNode_SaveCachedPose::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_SaveCachedPose, CacheName))
	{
	 	if (MakeUniqueCacheName())
	 	{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_SaveCachedPose::PostPasteNode()
{
	Super::PostPasteNode();
	if (MakeUniqueCacheName())
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_SaveCachedPose::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	OutExtensions.Add(UAnimBlueprintExtension_CachedPose::StaticClass());
}

bool UAnimGraphNode_SaveCachedPose::MakeUniqueCacheName()
{
	bool bHasChanged = false;
	// Disallow empty names
	if (CacheName.IsEmpty())
	{
		CacheName = TEXT("CachedPose");
		bHasChanged = true;
	}
	// Check for duplicate names among other SaveCachedPose nodes
	FCachedPoseNameValidator Validator(GetBlueprint(), this, "");
	if (Validator.IsValid(CacheName,true) != EValidatorResult::Ok)
	{
		Validator.FindValidString(CacheName);
		bHasChanged = true;
	}
	return bHasChanged;
}

#undef LOCTEXT_NAMESPACE
