// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ScaleChainLength.h"
#include "Animation/AnimInstance.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ScaleChainLength

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ScaleChainLength)

#define LOCTEXT_NAMESPACE "AnimGraph_ScaleChainLength"

UAnimGraphNode_ScaleChainLength::UAnimGraphNode_ScaleChainLength(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_ScaleChainLength::GetNodeTitleColor() const
{
	return FLinearColor(0.7f, 0.7f, 0.7f);
}

FText UAnimGraphNode_ScaleChainLength::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ScaleChainLength_Tooltip", "Scales a bone chain based on distance from StartBone to TargetLocation");
}

FText UAnimGraphNode_ScaleChainLength::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_ScaleChainLength", "Scale Chain Length");
}

FString UAnimGraphNode_ScaleChainLength::GetNodeCategory() const
{
	return TEXT("Animation|Misc.");
}

#undef LOCTEXT_NAMESPACE
