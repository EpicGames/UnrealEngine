// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_SequenceEvaluator.h"
#include "ToolMenus.h"

#include "Kismet2/CompilerResultsLog.h"
#include "AnimGraphCommands.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "AnimGraphNode_SequenceEvaluator.h"

#include "AnimGraphNodeBinding.h"
#include "BlueprintNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "IAnimBlueprintNodeOverrideAssetsContext.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeTemplateCache.h"
#include "Animation/AnimRootMotionProvider.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_SequenceEvaluator

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_SequenceEvaluator)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_SequenceEvaluator"

UAnimGraphNode_SequenceEvaluator::UAnimGraphNode_SequenceEvaluator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_SequenceEvaluator::PreloadRequiredAssets()
{
	PreloadRequiredAssetsHelper(Node.GetSequence(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequenceEvaluator, Sequence)));

	Super::PreloadRequiredAssets();
}

void UAnimGraphNode_SequenceEvaluator::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	AnimBlueprint->FindOrAddGroup(Node.GetGroupName());
}

void UAnimGraphNode_SequenceEvaluator::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const
{
	if(Node.GetSequence())
	{
		HandleAnimReferenceCollection(Node.Sequence, AnimationAssets);
	}
}

void UAnimGraphNode_SequenceEvaluator::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(Node.Sequence, AnimAssetReplacementMap);
}

FText UAnimGraphNode_SequenceEvaluator::GetMenuCategory() const
{
	return LOCTEXT("MenuCategory", "Animation|Sequences");
}

FText UAnimGraphNode_SequenceEvaluator::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraphPin* SequencePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequenceEvaluator, Sequence));
	return GetNodeTitleHelper(TitleType, SequencePin, Node.ShouldUseExplicitFrame() ? LOCTEXT("PlayerDescByFrame", "Sequence Evaluator (by frame)") :  LOCTEXT("PlayerDescByTime", "Sequence Evaluator (by time)"));
}

FSlateIcon UAnimGraphNode_SequenceEvaluator::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimSequence");
}

void UAnimGraphNode_SequenceEvaluator::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UAnimSequence::StaticClass() },
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				const FString TagValue = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
				if(const bool bKnownToBeAdditive = (!TagValue.IsEmpty() && !TagValue.Equals(TEXT("AAT_None"))))
				{
					return FText::Format(LOCTEXT("MenuDescFormatAdditive", "Evaluate '{0}' (additive)"), FText::FromName(InAssetData.AssetName));
				}
				else
				{
					return FText::Format(LOCTEXT("MenuDescFormat", "Evaluate '{0}'"), FText::FromName(InAssetData.AssetName));
				}
			}
			else
			{
				return LOCTEXT("MenuDesc", "Sequence Evaluator");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				const FString TagValue = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
				if(const bool bKnownToBeAdditive = (!TagValue.IsEmpty() && !TagValue.Equals(TEXT("AAT_None"))))
				{
					return FText::Format(LOCTEXT("MenuDescTooltipFormat_EvaluateAdditive", "Evaluate (additive)\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
				}
				else
				{
					return FText::Format(LOCTEXT("MenuDescTooltipFormat_Evaluate", "Evaluate\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
				}
			}
			else
			{
				return LOCTEXT("MenuDescTooltip", "Sequence Evaluator");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_AssetPlayerBase::SetupNewNode(InNewNode, bInIsTemplateNode, InAssetData);
		});
}

void UAnimGraphNode_SequenceEvaluator::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UAnimSequenceBase* Seq =  Cast<UAnimSequence>(Asset))
	{
		Node.SetSequence(Seq);
	}
}

void UAnimGraphNode_SequenceEvaluator::CopySettingsFromAnimationAsset(UAnimationAsset* Asset)
{
	if (UAnimSequenceBase* Seq = Cast<UAnimSequence>(Asset))
	{
		Node.SetShouldLoop(Seq->bLoop);
	}
}

void UAnimGraphNode_SequenceEvaluator::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if( PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNode_SequenceEvaluator, bUseExplicitFrame))
	{
		UEdGraphPin** Pin = nullptr;
		if (Node.bUseExplicitFrame)
		{
			Pin = Pins.FindByPredicate([](const UEdGraphPin* InPin)
			{
				return InPin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequenceEvaluator, ExplicitTime);
			});
		}
		else
		{
			Pin = Pins.FindByPredicate([](const UEdGraphPin* InPin)
			{
				return InPin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequenceEvaluator, ExplicitFrame);
			});			
		}

		if (Pin)
		{			
			Modify();
			(*Pin)->BreakAllPinLinks();
			RemoveBindings((*Pin)->PinName);
		}
	}
}

void UAnimGraphNode_SequenceEvaluator::OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const
{
	if(InContext.GetAssets().Num() > 0)
	{
		if (UAnimSequenceBase* Sequence = Cast<UAnimSequenceBase>(InContext.GetAssets()[0]))
		{
			FAnimNode_SequenceEvaluator& AnimNode = InContext.GetAnimNode<FAnimNode_SequenceEvaluator>();
			AnimNode.SetSequence(Sequence);
		}
	}
}

void UAnimGraphNode_SequenceEvaluator::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	ValidateAnimNodeDuringCompilationHelper(ForSkeleton, MessageLog, Node.GetSequence(), UAnimSequenceBase::StaticClass(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequenceEvaluator, Sequence)), GET_MEMBER_NAME_CHECKED(FAnimNode_SequenceEvaluator, Sequence));
}

void UAnimGraphNode_SequenceEvaluator::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to a regular sequence player
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeSequenceEvaluator", LOCTEXT("SequenceEvaluatorHeading", "Sequence Evaluator"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().OpenRelatedAsset);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToSeqPlayer);
		}
	}
}

bool UAnimGraphNode_SequenceEvaluator::DoesSupportTimeForTransitionGetter() const
{
	return true;
}

UAnimationAsset* UAnimGraphNode_SequenceEvaluator::GetAnimationAsset() const 
{
	UAnimSequenceBase* Sequence = Node.GetSequence();
	UEdGraphPin* SequencePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequenceEvaluator, Sequence));
	if (SequencePin != nullptr && Sequence == nullptr)
	{
		Sequence = Cast<UAnimSequenceBase>(SequencePin->DefaultObject);
	}

	return Sequence;
}

const TCHAR* UAnimGraphNode_SequenceEvaluator::GetTimePropertyName() const 
{
	return TEXT("ExplicitTime");
}

UScriptStruct* UAnimGraphNode_SequenceEvaluator::GetTimePropertyStruct() const 
{
	return FAnimNode_SequenceEvaluator::StaticStruct();
}

EAnimAssetHandlerType UAnimGraphNode_SequenceEvaluator::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UAnimSequence::StaticClass()) || AssetClass->IsChildOf(UAnimComposite::StaticClass()))
	{
		return EAnimAssetHandlerType::Supported;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

void UAnimGraphNode_SequenceEvaluator::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	Super::GetOutputLinkAttributes(OutAttributes);

	if (UE::Anim::IAnimRootMotionProvider::Get())
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

#undef LOCTEXT_NAMESPACE
