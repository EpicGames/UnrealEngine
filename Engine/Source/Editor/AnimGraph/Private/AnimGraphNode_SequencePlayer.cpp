// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_SequencePlayer.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"

#include "Kismet2/CompilerResultsLog.h"
#include "AnimGraphCommands.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintActionFilter.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "DetailLayoutBuilder.h"
#include "EditorCategoryUtils.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimPoseSearchProvider.h"
#include "Animation/AnimRootMotionProvider.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "IAnimBlueprintNodeOverrideAssetsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_SequencePlayer)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_SequencePlayer"

/////////////////////////////////////////////////////
// UAnimGraphNode_SequencePlayer

UAnimGraphNode_SequencePlayer::UAnimGraphNode_SequencePlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_SequencePlayer::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if(Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AnimNodeConstantDataRefactorPhase0)
	{
		Node.PlayRateScaleBiasClampConstants.CopyFromLegacy(Node.PlayRateScaleBiasClamp_DEPRECATED);
	}
}

void UAnimGraphNode_SequencePlayer::PreloadRequiredAssets()
{
	PreloadRequiredAssetsHelper(Node.GetSequence(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequencePlayer, Sequence)));

	Super::PreloadRequiredAssets();
}

FLinearColor UAnimGraphNode_SequencePlayer::GetNodeTitleColor() const
{
	UAnimSequenceBase* Sequence = Node.GetSequence();
	if ((Sequence != NULL) && Sequence->IsValidAdditive())
	{
		return FLinearColor(0.10f, 0.60f, 0.12f);
	}
	else
	{
		return FColor(200, 100, 100);
	}
}

FSlateIcon UAnimGraphNode_SequencePlayer::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimSequence");
}

FText UAnimGraphNode_SequencePlayer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraphPin* SequencePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequencePlayer, Sequence));
	return GetNodeTitleHelper(TitleType, SequencePin, LOCTEXT("PlayerDesc", "Sequence Player"),
		[](UAnimationAsset* InAsset)
		{
			UAnimSequenceBase* SequenceBase = CastChecked<UAnimSequenceBase>(InAsset);
			const bool bAdditive = SequenceBase->IsValidAdditive();
			return bAdditive ? LOCTEXT("AdditivePostFix", "(additive)") : FText::GetEmpty();
		});
}

FText UAnimGraphNode_SequencePlayer::GetMenuCategory() const
{
	return LOCTEXT("MenuCategory", "Animation|Sequences");
}

void UAnimGraphNode_SequencePlayer::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	GetMenuActionsHelper(
		InActionRegistrar,
		GetClass(),
		{ UAnimSequence::StaticClass(), UAnimComposite::StaticClass() },
		{ },
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				const FString TagValue = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
				if(const bool bKnownToBeAdditive = (!TagValue.IsEmpty() && !TagValue.Equals(TEXT("AAT_None"))))
				{
					return FText::Format(LOCTEXT("MenuDescFormat_PlayAdditive", "Play '{0}' (additive)"), FText::FromName(InAssetData.AssetName));
				}
				else
				{
					return FText::Format(LOCTEXT("MenuDescFormat_Play1", "Play '{0}'"), FText::FromName(InAssetData.AssetName));
				}
			}
			else
			{
				return LOCTEXT("PlayerDesc", "Sequence Player");
			}
		},
		[](const FAssetData& InAssetData, UClass* InClass)
		{
			if(InAssetData.IsValid())
			{
				const FString TagValue = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
				if(const bool bKnownToBeAdditive = (!TagValue.IsEmpty() && !TagValue.Equals(TEXT("AAT_None"))))
				{
					return FText::Format(LOCTEXT("MenuDescTooltipFormat_PlayAdded", "Play (additive)\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
				}
				else
				{
					return FText::Format(LOCTEXT("MenuDescTooltipFormat_Play", "Play\n'{0}'"), FText::FromString(InAssetData.GetObjectPathString()));
				}
			}
			else
			{
				return LOCTEXT("PlayerDescTooltip", "Sequence Player");
			}
		},
		[](UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData)
		{
			UAnimGraphNode_AssetPlayerBase::SetupNewNode(InNewNode, bInIsTemplateNode, InAssetData);
		});	
}

EAnimAssetHandlerType UAnimGraphNode_SequencePlayer::SupportsAssetClass(const UClass* AssetClass) const
{
	if (AssetClass->IsChildOf(UAnimSequence::StaticClass()) || AssetClass->IsChildOf(UAnimComposite::StaticClass()))
	{
		return EAnimAssetHandlerType::PrimaryHandler;
	}
	else
	{
		return EAnimAssetHandlerType::NotSupported;
	}
}

void UAnimGraphNode_SequencePlayer::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	Super::GetOutputLinkAttributes(OutAttributes);

	if (UE::Anim::IAnimRootMotionProvider::Get())
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_SequencePlayer::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	ValidateAnimNodeDuringCompilationHelper(ForSkeleton, MessageLog, Node.GetSequence(), UAnimSequenceBase::StaticClass(), FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequencePlayer, Sequence)), GET_MEMBER_NAME_CHECKED(FAnimNode_SequencePlayer, Sequence));
}

void UAnimGraphNode_SequencePlayer::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->bIsDebugging)
	{
		// add an option to convert to single frame
		{
			FToolMenuSection& Section = Menu->AddSection("AnimGraphNodeSequencePlayer", LOCTEXT("SequencePlayerHeading", "Sequence Player"));
			Section.AddMenuEntry(FAnimGraphCommands::Get().OpenRelatedAsset);
			Section.AddMenuEntry(FAnimGraphCommands::Get().ConvertToSeqEvaluator);
		}
	}
}

void UAnimGraphNode_SequencePlayer::SetAnimationAsset(UAnimationAsset* Asset)
{
	if (UAnimSequenceBase* Seq = Cast<UAnimSequenceBase>(Asset))
	{
		Node.SetSequence(Seq);
	}
}

void UAnimGraphNode_SequencePlayer::CopySettingsFromAnimationAsset(UAnimationAsset* Asset)
{
	if (UAnimSequenceBase* Seq = Cast<UAnimSequenceBase>(Asset))
	{
		Node.SetLoopAnimation(Seq->bLoop);
	}
}

void UAnimGraphNode_SequencePlayer::OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const
{
	if(InContext.GetAssets().Num() > 0)
	{
		if (UAnimSequenceBase* Sequence = Cast<UAnimSequenceBase>(InContext.GetAssets()[0]))
		{
			FAnimNode_SequencePlayer& AnimNode = InContext.GetAnimNode<FAnimNode_SequencePlayer>();
			AnimNode.SetSequence(Sequence);
		}
	}
}

void UAnimGraphNode_SequencePlayer::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	AnimBlueprint->FindOrAddGroup(Node.GetGroupName());
}

void UAnimGraphNode_SequencePlayer::GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const
{
	if(Node.GetSequence())
	{
		HandleAnimReferenceCollection(Node.Sequence, AnimationAssets);
	}
}

void UAnimGraphNode_SequencePlayer::ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(Node.Sequence, AnimAssetReplacementMap);
}

bool UAnimGraphNode_SequencePlayer::DoesSupportTimeForTransitionGetter() const
{
	return true;
}

UAnimationAsset* UAnimGraphNode_SequencePlayer::GetAnimationAsset() const 
{
	UAnimSequenceBase* Sequence = Node.GetSequence();
	UEdGraphPin* SequencePin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequencePlayer, Sequence));
	if (SequencePin != nullptr && Sequence == nullptr)
	{
		Sequence = Cast<UAnimSequenceBase>(SequencePin->DefaultObject);
	}

	return Sequence;
}

const TCHAR* UAnimGraphNode_SequencePlayer::GetTimePropertyName() const 
{
	return TEXT("InternalTimeAccumulator");
}

UScriptStruct* UAnimGraphNode_SequencePlayer::GetTimePropertyStruct() const 
{
	return FAnimNode_SequencePlayer::StaticStruct();
}

void UAnimGraphNode_SequencePlayer::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	if (!UE::Anim::IPoseSearchProvider::IsAvailable())
	{
		DetailBuilder.HideCategory(TEXT("PoseMatching"));
	}
}

void UAnimGraphNode_SequencePlayer::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequencePlayer, PlayRate))
	{
		if (!Pin->bHidden)
		{
			// Draw value for PlayRateBasis if the pin is not exposed
			UEdGraphPin* PlayRateBasisPin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequencePlayer, PlayRateBasis));
			if (!PlayRateBasisPin || PlayRateBasisPin->bHidden)
			{
				if (Node.GetPlayRateBasis() != 1.f)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("PinFriendlyName"), Pin->PinFriendlyName);
					Args.Add(TEXT("PlayRateBasis"), FText::AsNumber(Node.GetPlayRateBasis()));
					Pin->PinFriendlyName = FText::Format(LOCTEXT("FAnimNode_SequencePlayer_PlayRateBasis_Value", "({PinFriendlyName} / {PlayRateBasis})"), Args);
				}
			}
			else // PlayRateBasisPin is visible
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("PinFriendlyName"), Pin->PinFriendlyName);
				Pin->PinFriendlyName = FText::Format(LOCTEXT("FAnimNode_SequencePlayer_PlayRateBasis_Name", "({PinFriendlyName} / PlayRateBasis)"), Args);
			}

			Pin->PinFriendlyName = Node.GetPlayRateScaleBiasClampConstants().GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_SequencePlayer::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SequencePlayer, PlayRateBasis))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, bMapRange))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, Scale))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, Bias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, bClampResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, ClampMin))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, ClampMax))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, bInterpResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, InterpSpeedIncreasing))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClampConstants, InterpSpeedDecreasing)))
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#undef LOCTEXT_NAMESPACE
