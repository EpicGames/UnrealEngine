// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_SkeletalControlBase.h"
#include "UnrealWidgetFwd.h"
#include "AnimationGraphSchema.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_SkeletalControlBase

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_SkeletalControlBase)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_SkeletalControlBase::UAnimGraphNode_SkeletalControlBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UAnimGraphNode_SkeletalControlBase::GetWidgetCoordinateSystem(const USkeletalMeshComponent* SkelComp)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GetWidgetMode(SkelComp) == UE::Widget::WM_Scale)
	{
		return COORD_Local;
	}
	else
	{
		return COORD_World;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// returns int32 instead of EWidgetMode because of compiling issue on Mac
int32 UAnimGraphNode_SkeletalControlBase::GetWidgetMode(const USkeletalMeshComponent* SkelComp)
{
	return  (int32)UE::Widget::EWidgetMode::WM_None;
}

int32 UAnimGraphNode_SkeletalControlBase::ChangeToNextWidgetMode(const USkeletalMeshComponent* SkelComp, int32 CurWidgetMode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetWidgetMode(SkelComp);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FName UAnimGraphNode_SkeletalControlBase::FindSelectedBone()
{
	return NAME_None;
}

void UAnimGraphNode_SkeletalControlBase::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			FAnimNode_SkeletalControlBase* SkeletalControlBase = static_cast<FAnimNode_SkeletalControlBase*>(GetFNode());
			SkeletalControlBase->AlphaBoolBlend.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FLinearColor UAnimGraphNode_SkeletalControlBase::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.10f);
}

FString UAnimGraphNode_SkeletalControlBase::GetNodeCategory() const
{
	return TEXT("Animation|Skeletal Controls");
}

FText UAnimGraphNode_SkeletalControlBase::GetControllerDescription() const
{
	return LOCTEXT("ImplementMe", "Implement me");
}

FText UAnimGraphNode_SkeletalControlBase::GetTooltipText() const
{
	return GetControllerDescription();
}

void UAnimGraphNode_SkeletalControlBase::CreateOutputPins()
{
	CreatePin(EGPD_Output, UAnimationGraphSchema::PC_Struct, FComponentSpacePoseLink::StaticStruct(), TEXT("Pose"));
}


void UAnimGraphNode_SkeletalControlBase::ConvertToComponentSpaceTransform(const USkeletalMeshComponent* SkelComp, const FTransform & InTransform, FTransform & OutCSTransform, int32 BoneIndex, EBoneControlSpace Space) const
{
	USkeleton * Skeleton = SkelComp->GetSkeletalMeshAsset()->GetSkeleton();

	switch (Space)
	{
	case BCS_WorldSpace:
	{
		OutCSTransform = InTransform;
		OutCSTransform.SetToRelativeTransform(SkelComp->GetComponentTransform());
	}
		break;

	case BCS_ComponentSpace:
	{
		// Component Space, no change.
		OutCSTransform = InTransform;
	}
		break;

	case BCS_ParentBoneSpace:
		if (BoneIndex != INDEX_NONE)
		{
			const int32 ParentIndex = Skeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)																																																																																																																															
			{
				const int32 MeshParentIndex = Skeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkelComp->GetSkeletalMeshAsset(), ParentIndex);
				if (MeshParentIndex != INDEX_NONE)
				{
					const FTransform ParentTM = SkelComp->GetBoneTransform(MeshParentIndex);
					OutCSTransform = InTransform * ParentTM;
				}
				else
				{
					OutCSTransform = InTransform;
				}
			}
		}
		break;

	case BCS_BoneSpace:
		if (BoneIndex != INDEX_NONE)
		{
			const int32 MeshBoneIndex = Skeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkelComp->GetSkeletalMeshAsset(), BoneIndex);
			if (MeshBoneIndex != INDEX_NONE)
			{
				const FTransform BoneTM = SkelComp->GetBoneTransform(MeshBoneIndex);
				OutCSTransform = InTransform * BoneTM;
			}			
			else
			{
				OutCSTransform = InTransform;
			}
		}
		break;

	default:
		if (SkelComp->GetSkeletalMeshAsset())
		{
			UE_LOGF(LogAnimation, Warning, "ConvertToComponentSpaceTransform: Unknown BoneSpace %d  for Mesh: %ls", (uint8)Space, *SkelComp->GetSkeletalMeshAsset()->GetFName().ToString());
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "ConvertToComponentSpaceTransform: Unknown BoneSpace %d  for Skeleton: %ls", (uint8)Space, *Skeleton->GetFName().ToString());
		}
		break;
	}
}


FVector UAnimGraphNode_SkeletalControlBase::ConvertCSVectorToBoneSpace(const USkeletalMeshComponent* SkelComp, FVector& InCSVector, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space)
{
	FVector OutVector = InCSVector;

	if (MeshBases.GetPose().IsValid())
	{
		const FMeshPoseBoneIndex MeshBoneIndex(SkelComp->GetBoneIndex(BoneName));
		const FCompactPoseBoneIndex BoneIndex = MeshBases.GetPose().GetBoneContainer().MakeCompactPoseIndex(MeshBoneIndex);

		switch (Space)
		{
			// World Space, no change in preview window
		case BCS_WorldSpace:
		case BCS_ComponentSpace:
			// Component Space, no change.
			break;

		case BCS_ParentBoneSpace:
		{
			const FCompactPoseBoneIndex ParentIndex = MeshBases.GetPose().GetParentBoneIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(ParentIndex);
				OutVector = ParentTM.InverseTransformVector(InCSVector);
			}
		}
			break;

		case BCS_BoneSpace:
		{
			const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(BoneIndex);
			OutVector = BoneTM.InverseTransformVector(InCSVector);
		}
			break;
		}
	}

	return OutVector;
}

FQuat UAnimGraphNode_SkeletalControlBase::ConvertCSRotationToBoneSpace(const USkeletalMeshComponent* SkelComp, FRotator& InCSRotator, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space)
{
	FQuat OutQuat = FQuat::Identity;

	if (MeshBases.GetPose().IsValid())
	{
		const FMeshPoseBoneIndex MeshBoneIndex(SkelComp->GetBoneIndex(BoneName));
		const FCompactPoseBoneIndex BoneIndex = MeshBases.GetPose().GetBoneContainer().MakeCompactPoseIndex(MeshBoneIndex);

		FVector RotAxis;
		float RotAngle;
		InCSRotator.Quaternion().ToAxisAndAngle(RotAxis, RotAngle);

		switch (Space)
		{
			// World Space, no change in preview window
		case BCS_WorldSpace:
		case BCS_ComponentSpace:
			// Component Space, no change.
			OutQuat = InCSRotator.Quaternion();
			break;

		case BCS_ParentBoneSpace:
		{
			const FCompactPoseBoneIndex ParentIndex = MeshBases.GetPose().GetParentBoneIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(ParentIndex);
				FTransform InverseParentTM = ParentTM.Inverse();
				//Calculate the new delta rotation
				FVector4 BoneSpaceAxis = InverseParentTM.TransformVector(RotAxis);
				FQuat DeltaQuat(BoneSpaceAxis, RotAngle);
				DeltaQuat.Normalize();
				OutQuat = DeltaQuat;
			}
		}
			break;

		case BCS_BoneSpace:
		{
			const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(BoneIndex);
			FTransform InverseBoneTM = BoneTM.Inverse();
			FVector4 BoneSpaceAxis = InverseBoneTM.TransformVector(RotAxis);
			//Calculate the new delta rotation
			FQuat DeltaQuat(BoneSpaceAxis, RotAngle);
			DeltaQuat.Normalize();
			OutQuat = DeltaQuat;
		}
			break;
		}
	}

	return OutQuat;
}

FVector UAnimGraphNode_SkeletalControlBase::ConvertWidgetLocation(const USkeletalMeshComponent* SkelComp, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const FVector& Location, const EBoneControlSpace Space)
{
	FVector WidgetLoc = FVector::ZeroVector;

	if (MeshBases.GetPose().IsValid())
	{
		USkeleton * Skeleton = SkelComp->GetSkeletalMeshAsset()->GetSkeleton();
		const FMeshPoseBoneIndex MeshBoneIndex(SkelComp->GetBoneIndex(BoneName));
		const FCompactPoseBoneIndex CompactBoneIndex = MeshBases.GetPose().GetBoneContainer().MakeCompactPoseIndex(MeshBoneIndex);

		switch (Space)
		{
			// GetComponentTransform() must be Identity in preview window so same as ComponentSpace
		case BCS_WorldSpace:
		case BCS_ComponentSpace:
		{
			// Component Space, no change.
			WidgetLoc = Location;
		}
			break;

		case BCS_ParentBoneSpace:

			if (CompactBoneIndex != INDEX_NONE)
			{
				const FCompactPoseBoneIndex CompactParentIndex = MeshBases.GetPose().GetParentBoneIndex(CompactBoneIndex);
				if (CompactParentIndex != INDEX_NONE)
				{
					const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(CompactParentIndex);
					WidgetLoc = ParentTM.TransformPosition(Location);
				}
			}
			break;

		case BCS_BoneSpace:

			if (CompactBoneIndex != INDEX_NONE)
			{
				const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(CompactBoneIndex);
				WidgetLoc = BoneTM.TransformPosition(Location);
			}
		}
	}

	return WidgetLoc;
}

void UAnimGraphNode_SkeletalControlBase::GetDefaultValue(const FName UpdateDefaultValueName, FVector& OutVec)
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinName == UpdateDefaultValueName)
		{
			if (GetSchema()->IsCurrentPinDefaultValid(Pin).IsEmpty())
			{
				FString DefaultString = Pin->GetDefaultAsString();

				// Existing nodes (from older versions) might have an empty default value string; in that case we just fall through and return the zero vector below (which is the default value in that case).
				if(!DefaultString.IsEmpty())
				{
					TArray<FString> ResultString;

					//Parse string to split its contents separated by ','
					DefaultString.TrimStartAndEndInline();
					DefaultString.ParseIntoArray(ResultString, TEXT(","), true);

					check(ResultString.Num() == 3);

					OutVec.Set(
						FCString::Atof(*ResultString[0]),
						FCString::Atof(*ResultString[1]),
						FCString::Atof(*ResultString[2])
						);
					return;
				}
			}
		}
	}
	OutVec = FVector::ZeroVector;
}

void UAnimGraphNode_SkeletalControlBase::SetDefaultValue(const FName UpdateDefaultValueName, const FVector& Value)
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinName == UpdateDefaultValueName)
		{
			if (GetSchema()->IsCurrentPinDefaultValid(Pin).IsEmpty())
			{
				FString Str = FString::Printf(TEXT("%.3f,%.3f,%.3f"), Value.X, Value.Y, Value.Z);
				if (Pin->DefaultValue != Str)
				{
					PreEditChange(nullptr);
					GetSchema()->TrySetDefaultValue(*Pin, Str);
					PostEditChange();
					break;
				}
			}
		}
	}
}

bool UAnimGraphNode_SkeletalControlBase::IsPinShown(const FName PinName) const
{
	for (const FOptionalPinFromProperty& Pin : ShowPinForProperties)
	{
		if (Pin.PropertyName == PinName)
		{
			return Pin.bShowPin;
		}
	}
	return false;
}

void UAnimGraphNode_SkeletalControlBase::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, Alpha))
	{
		Pin->bHidden = (GetNode()->AlphaInputType != EAnimAlphaInputType::Float);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = GetNode()->AlphaScaleBias.GetFriendlyName(GetNode()->AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName));
		}
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, bAlphaBoolEnabled))
	{
		Pin->bHidden = (GetNode()->AlphaInputType != EAnimAlphaInputType::Bool);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, AlphaCurveName))
	{
		Pin->bHidden = (GetNode()->AlphaInputType != EAnimAlphaInputType::Curve);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = GetNode()->AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_SkeletalControlBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, AlphaScaleBias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
	{
		ReconstructNode();
	}

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, AlphaInputType))
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeAlphaInputType", "Change Alpha Input Type"));
		Modify();

		const FAnimNode_SkeletalControlBase* SkelControlNode = GetNode();

		// Break links to pins going away
		for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Pins[PinIndex];
			if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, Alpha))
			{
				if (GetNode()->AlphaInputType != EAnimAlphaInputType::Float)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, bAlphaBoolEnabled))
			{
				if (GetNode()->AlphaInputType != EAnimAlphaInputType::Bool)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_SkeletalControlBase, AlphaCurveName))
			{
				if (GetNode()->AlphaInputType != EAnimAlphaInputType::Curve)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
		}

		ReconstructNode();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UAnimGraphNode_SkeletalControlBase::ShowVisualWarning() const
{
	const FAnimNode_SkeletalControlBase* DebuggedNode = GetDebuggedNode();

	const bool bHasError = DebuggedNode ? DebuggedNode->HasValidationVisualWarnings() : false;

	return bHasError;
}

FText UAnimGraphNode_SkeletalControlBase::GetVisualWarningTooltipText() const
{
	FAnimNode_SkeletalControlBase* DebuggedNode = GetDebuggedNode();

	return DebuggedNode ? DebuggedNode->GetValidationVisualWarningMessage() : FText::GetEmpty();
}

void UAnimGraphNode_SkeletalControlBase::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	const FAnimNode_SkeletalControlBase* SkelControlNode = GetNode();
	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (SkelControlNode->AlphaInputType != EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, bAlphaBoolEnabled)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, AlphaBoolBlend)));
	}

	if (SkelControlNode->AlphaInputType != EAnimAlphaInputType::Float)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, Alpha)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, AlphaScaleBias)));
	}

	if (SkelControlNode->AlphaInputType != EAnimAlphaInputType::Curve)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, AlphaCurveName)));
	}

	if ((SkelControlNode->AlphaInputType != EAnimAlphaInputType::Float) 
		&& (SkelControlNode->AlphaInputType != EAnimAlphaInputType::Curve))
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_SkeletalControlBase, AlphaScaleBiasClamp)));
	}
}

void UAnimGraphNode_SkeletalControlBase::ValidateAnimNodePostCompile(FCompilerResultsLog& MessageLog, UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex)
{
	if (UAnimationSettings::Get()->bEnablePerformanceLog)
	{
		const FAnimNode_SkeletalControlBase* Node = GetNode();
		if (Node && Node->LODThreshold < 0)
		{
			MessageLog.Warning(TEXT("@@ contains no LOD Threshold."), this);
		}
	}
}

FAnimNode_SkeletalControlBase* UAnimGraphNode_SkeletalControlBase::GetDebuggedNode() const
{
	if (const UObject* ObjectBeingDebugged = GetAnimBlueprint()->GetObjectBeingDebugged())
	{
		if (const UAnimInstance* InstanceBeingDebugged = Cast<const UAnimInstance>(ObjectBeingDebugged))
		{
			USkeletalMeshComponent* Component = InstanceBeingDebugged->GetSkelMeshComponent();
			if (Component != nullptr && Component->GetAnimInstance() != nullptr)
			{
				return static_cast<FAnimNode_SkeletalControlBase*>(FindDebugAnimNode(Component));
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
