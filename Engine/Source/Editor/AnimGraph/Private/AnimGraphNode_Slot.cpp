// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_Slot.h"
#include "IAnimGraph_SequencerMixerTargetConnector.h"
#include "Features/IModularFeatures.h"


/////////////////////////////////////////////////////
// UAnimGraphNode_Slot

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Slot)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_Slot::UAnimGraphNode_Slot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_Slot::GetNodeTitleColor() const
{
	return FLinearColor(0.7f, 0.7f, 0.7f);
}

FText UAnimGraphNode_Slot::GetTooltipText() const
{
	return LOCTEXT("AnimSlotNode_Tooltip", "Plays animation from code using AnimMontage");
}

FText UAnimGraphNode_Slot::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Node.SlotName == NAME_None || !HasValidBlueprint() )
	{
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			return LOCTEXT("SlotNodeListTitle_NoName", "Slot '(No slot name)'");
		}
		else
		{
			return LOCTEXT("SlotNodeTitle_NoName", "(No slot name)\nSlot");
		}
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
		FName GroupName = (AnimBlueprint->TargetSkeleton) ? AnimBlueprint->TargetSkeleton->GetSlotGroupName(Node.SlotName) : FAnimSlotGroup::DefaultGroupName;

		FFormatNamedArguments Args;
		Args.Add(TEXT("SlotName"), FText::FromName(Node.SlotName));
		Args.Add(TEXT("GroupName"), FText::FromName(GroupName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("SlotNodeListTitle", "Slot '{SlotName}'"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("SlotNodeTitle", "Slot '{SlotName}'\nGroup '{GroupName}'"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

FString UAnimGraphNode_Slot::GetNodeCategory() const
{
	return TEXT("Animation|Montage");
}

void UAnimGraphNode_Slot::BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	if (!GIsCookerLoadingPackage && !IsRunningCookCommandlet() && AnimBlueprint->TargetSkeleton) // Don't modify skeleton during cook
	{
		AnimBlueprint->TargetSkeleton->RegisterSlotNode(Node.SlotName);
	}
}

void UAnimGraphNode_Slot::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	Super::GetRequiredExtensions(OutExtensions);
	
	if (const IAnimGraph_SequencerMixerTargetConnector* MixerTargetConnector = IAnimGraph_SequencerMixerTargetConnector::Get())
	{
		MixerTargetConnector->GetSequencerMixerTargetRequiredExtensions(OutExtensions);
	}
}

#undef LOCTEXT_NAMESPACE
