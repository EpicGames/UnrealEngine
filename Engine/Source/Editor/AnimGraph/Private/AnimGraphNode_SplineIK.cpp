// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_SplineIK.h"
#include "AnimNodeEditModes.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/Skeleton.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_SplineIK)

#define LOCTEXT_NAMESPACE "A3Nodes"

void UAnimGraphNode_SplineIK::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.TwistBlend.SetBlendOption(EAlphaBlendOption::Linear);
		}
	}
	
	Super::Serialize(Ar);
}

UAnimGraphNode_SplineIK::UAnimGraphNode_SplineIK(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_SplineIK::GetControllerDescription() const
{
	return LOCTEXT("SplineIK", "Spline IK");
}

FText UAnimGraphNode_SplineIK::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_SplineIK_Tooltip", "The Spline IK control constrains a chain of bones to a spline.");
}

FText UAnimGraphNode_SplineIK::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Node.StartBone.BoneName == NAME_None || Node.EndBone.BoneName == NAME_None)
	{
		return GetControllerDescription();
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("StartBoneName"), FText::FromName(Node.StartBone.BoneName));
		Args.Add(TEXT("EndBoneName"), FText::FromName(Node.EndBone.BoneName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_SplineIK_ListTitle", "{ControllerDescription} - {StartBoneName} - {EndBoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_SplineIK_Title", "{ControllerDescription}\nChain: {StartBoneName} - {EndBoneName}"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

FEditorModeID UAnimGraphNode_SplineIK::GetEditorMode() const
{
	return AnimNodeEditModes::SplineIK;
}

void UAnimGraphNode_SplineIK::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FBoneReference, BoneName) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_SplineIK, bAutoCalculateSpline) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_SplineIK, PointCount) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_SplineIK, BoneAxis))
	{
		USkeleton* Skeleton = GetAnimBlueprint()->TargetSkeleton;
		Node.GatherBoneReferences(Skeleton->GetReferenceSkeleton());

		ReconstructNode();
	}
}

#undef LOCTEXT_NAMESPACE
