// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_SpringBone.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_SpringBone

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_SpringBone)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_SpringBone::UAnimGraphNode_SpringBone(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_SpringBone::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_REPLACE_SPRING_NOZ_PROPERTY)
	{
		//Fix up z translation from old NoZSpring property
		Node.bTranslateZ = !Node.bNoZSpring_DEPRECATED;
	}
}

FText UAnimGraphNode_SpringBone::GetControllerDescription() const
{
	return LOCTEXT("SpringController", "Spring controller");
}

FText UAnimGraphNode_SpringBone::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_SpringBone_Tooltip", "The Spring Controller applies a spring solver that can be used to limit how far a bone can stretch from its reference pose position and apply a force in the opposite direction.");
}

FText UAnimGraphNode_SpringBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.SpringBone.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if(!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.SpringBone.BoneName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_SpringBone_ListTitle", "{ControllerDescription} - Bone: {BoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_SpringBone_Title", "{ControllerDescription}\nBone: {BoneName}"), Args), this);
		}
	}

	return CachedNodeTitles[TitleType];
}

#undef LOCTEXT_NAMESPACE
