// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_StateMachine.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_StateMachine

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_StateMachine)

UAnimGraphNode_StateMachine::UAnimGraphNode_StateMachine(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
}
