// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_StateMachineBase.h"
#include "EdGraph/EdGraph.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "Kismet2/Kismet2NameValidators.h"

#include "AnimationGraph.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationStateMachineSchema.h"
#include "AnimGraphNode_StateMachine.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "K2Node_AnimGetter.h"
#include "AnimBlueprintCompiler.h"
#include "AnimStateNode.h"
#include "AnimStateTransitionNode.h"
#include "AnimGraphNode_TransitionResult.h"
#include "AnimGraphNode_StateResult.h"
#include "AnimStateEntryNode.h"
#include "AnimationStateGraph.h"
#include "AnimationTransitionGraph.h"
#include "AnimationCustomTransitionGraph.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "AnimStateConduitNode.h"
#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "AnimGraphNode_RandomPlayer.h"
#include "AnimGraphNode_TransitionPoseEvaluator.h"
#include "AnimGraphNode_CustomTransitionResult.h"
#include "AnimBlueprintExtension_StateMachine.h"
#include "IAnimBlueprintGeneratedClassCompiledData.h"
#include "IAnimBlueprintCompilationContext.h"
#include "Animation/AnimNode_Inertialization.h"
#include "AnimStateAliasNode.h"
#include "EdGraphUtilities.h"

/////////////////////////////////////////////////////
// FAnimStateMachineNodeNameValidator

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_StateMachineBase)

class FAnimStateMachineNodeNameValidator : public FStringSetNameValidator
{
public:
	FAnimStateMachineNodeNameValidator(const UAnimGraphNode_StateMachineBase* InStateMachineNode)
		: FStringSetNameValidator(FString())
	{
		Names.Add(TEXT("None"));

		TArray<UAnimGraphNode_StateMachineBase*> Nodes;

		UAnimationGraph* StateMachine = CastChecked<UAnimationGraph>(InStateMachineNode->GetOuter());
		StateMachine->GetNodesOfClassEx<UAnimGraphNode_StateMachine, UAnimGraphNode_StateMachineBase>(Nodes);

		for (auto Node : Nodes)
		{
			if (Node != InStateMachineNode)
			{
				Names.Add(Node->GetStateMachineName());
			}
		}

		// Include the name of animation layers
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(StateMachine);

		if (Blueprint)
		{
			UClass* TargetClass = *Blueprint->SkeletonGeneratedClass;
			if (TargetClass)
			{
				IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
				for (const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
				{
					if (AnimBlueprintFunction.Name != UEdGraphSchema_K2::GN_AnimGraph)
					{
						Names.Add(AnimBlueprintFunction.Name.ToString());
					}
				}
			}
		}
	}
};

/////////////////////////////////////////////////////
// UAnimGraphNode_StateMachineBase

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_StateMachineBase::UAnimGraphNode_StateMachineBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_StateMachineBase::GetNodeTitleColor() const
{
	return FLinearColor(0.8f, 0.8f, 0.8f);
}

FText UAnimGraphNode_StateMachineBase::GetTooltipText() const
{
	return LOCTEXT("StateMachineTooltip", "Animation State Machine");
}

FText UAnimGraphNode_StateMachineBase::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::MenuTitle || TitleType == ENodeTitleType::ListView) && (EditorStateMachineGraph == nullptr))
	{
		return LOCTEXT("AddNewStateMachine", "State Machine");
	}
	else if (EditorStateMachineGraph == nullptr)
	{
		if (TitleType == ENodeTitleType::FullTitle)
		{
			return LOCTEXT("NullStateMachineFullTitle", "Error: No Graph\nState Machine");
		}
		else
		{
			return LOCTEXT("ErrorNoGraph", "Error: No Graph");
		}
	}
	else if (TitleType == ENodeTitleType::FullTitle)
	{
		if (CachedFullTitle.IsOutOfDate(this))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Title"), FText::FromName(EditorStateMachineGraph->GetFName()));
			// FText::Format() is slow, so we cache this to save on performance
			CachedFullTitle.SetCachedText(FText::Format(LOCTEXT("StateMachineFullTitle", "{Title}\nState Machine"), Args), this);
		}
		return CachedFullTitle;
	}

	return FText::FromName(EditorStateMachineGraph->GetFName());
}

FText UAnimGraphNode_StateMachineBase::GetMenuCategory() const
{
	return LOCTEXT("StateMachineCategory", "Animation|State Machines");
}

void UAnimGraphNode_StateMachineBase::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	// Create a new animation graph
	check(EditorStateMachineGraph == NULL);
	EditorStateMachineGraph = CastChecked<UAnimationStateMachineGraph>(FBlueprintEditorUtils::CreateNewGraph(this, NAME_None, UAnimationStateMachineGraph::StaticClass(), UAnimationStateMachineSchema::StaticClass()));
	check(EditorStateMachineGraph);
	EditorStateMachineGraph->OwnerAnimGraphNode = this;

	// Find an interesting name
	TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
	FBlueprintEditorUtils::RenameGraphWithSuggestion(EditorStateMachineGraph, NameValidator, TEXT("New State Machine"));

	// Initialize the anim graph
	const UEdGraphSchema* Schema = EditorStateMachineGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*EditorStateMachineGraph);

	// Add the new graph as a child of our parent graph
	UEdGraph* ParentGraph = GetGraph();
	
	if(ParentGraph->SubGraphs.Find(EditorStateMachineGraph) == INDEX_NONE)
	{
		ParentGraph->Modify();
		ParentGraph->SubGraphs.Add(EditorStateMachineGraph);
	}
}

UObject* UAnimGraphNode_StateMachineBase::GetJumpTargetForDoubleClick() const
{
	// Open the state machine graph
	return EditorStateMachineGraph;
}

void UAnimGraphNode_StateMachineBase::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(HyperlinkTarget);
	}
}

void UAnimGraphNode_StateMachineBase::DestroyNode()
{
	UEdGraph* GraphToRemove = EditorStateMachineGraph;

	EditorStateMachineGraph = NULL;
	Super::DestroyNode();

	if (GraphToRemove)
	{
		UBlueprint* Blueprint = GetBlueprint();
		GraphToRemove->Modify();
		FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphToRemove, EGraphRemoveFlags::Recompile);
	}
}

void UAnimGraphNode_StateMachineBase::PostPasteNode()
{
	Super::PostPasteNode();

	if(EditorStateMachineGraph)
	{
		// Add the new graph as a child of our parent graph
		UEdGraph* ParentGraph = GetGraph();

		if(ParentGraph->SubGraphs.Find(EditorStateMachineGraph) == INDEX_NONE)
		{
			ParentGraph->SubGraphs.Add(EditorStateMachineGraph);
		}

		for (UEdGraphNode* GraphNode : EditorStateMachineGraph->Nodes)
		{
			GraphNode->CreateNewGuid();
			GraphNode->PostPasteNode();
			GraphNode->ReconstructNode();
		}

		// Find an interesting name
		TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
		FBlueprintEditorUtils::RenameGraphWithSuggestion(EditorStateMachineGraph, NameValidator, EditorStateMachineGraph->GetName());

		//restore transactional flag that is lost during copy/paste process
		EditorStateMachineGraph->SetFlags(RF_Transactional);
	}
}

FString UAnimGraphNode_StateMachineBase::GetStateMachineName()
{
	return (EditorStateMachineGraph != NULL) ? *(EditorStateMachineGraph->GetName()) : TEXT("(null)");
}

TSharedPtr<class INameValidatorInterface> UAnimGraphNode_StateMachineBase::MakeNameValidator() const
{
	return MakeShareable(new FAnimStateMachineNodeNameValidator(this));
}

FString UAnimGraphNode_StateMachineBase::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/AnimationStateMachine");
}

void UAnimGraphNode_StateMachineBase::OnRenameNode(const FString& NewName)
{
	FBlueprintEditorUtils::RenameGraph(EditorStateMachineGraph, NewName);
}

TArray<UEdGraph*> UAnimGraphNode_StateMachineBase::GetSubGraphs() const
{
	return TArray<UEdGraph*>( { EditorStateMachineGraph } );
}

void UAnimGraphNode_StateMachineBase::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	struct FMachineCreator
	{
	public:

		// A transition node with a state alias as source may have multiple transitions belonging to the same node. The referenced aliased state to differentiates them during compilation. 
		using FUniqueTransition = TPair<UAnimStateTransitionNode*, UAnimStateNodeBase*>;

		int32 MachineIndex;
		TMap<UAnimStateNodeBase*, int32> StateIndexTable;
		TMultiMap<int32, int32> StateIndexToStateAliasNodeIndices;
		TArray<const UAnimStateAliasNode*> StateAliasNodes;
		TMap<FUniqueTransition, int32> TransitionIndexTable;
		UAnimGraphNode_StateMachineBase* StateMachineInstance;
		TArray<FBakedAnimationStateMachine>& BakedMachines;
		IAnimBlueprintGeneratedClassCompiledData& CompiledData;
		IAnimBlueprintCompilationContext& CompilationContext;

	public:
		FMachineCreator(UAnimGraphNode_StateMachineBase* InStateMachineInstance, int32 InMachineIndex, TArray<FBakedAnimationStateMachine>& InBakedMachines, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& InCompiledData)
			: MachineIndex(InMachineIndex)
			, StateMachineInstance(InStateMachineInstance)
			, BakedMachines(InBakedMachines)
			, CompiledData(InCompiledData)
			, CompilationContext(InCompilationContext)
		{
			FStateMachineDebugData& MachineInfo = GetMachineSpecificDebugData();
			MachineInfo.MachineIndex = MachineIndex;
			MachineInfo.MachineInstanceNode = CompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_StateMachineBase>(InStateMachineInstance);

			StateMachineInstance->GetNode().StateMachineIndexInClass = MachineIndex;

			FBakedAnimationStateMachine& BakedMachine = GetMachine();
			BakedMachine.MachineName = StateMachineInstance->EditorStateMachineGraph->GetFName();
			BakedMachine.InitialState = INDEX_NONE;
		}

		FBakedAnimationStateMachine& GetMachine()
		{
			return BakedMachines[MachineIndex];
		}

		FStateMachineDebugData& GetMachineSpecificDebugData()
		{
			UAnimationStateMachineGraph* SourceGraph = CompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimationStateMachineGraph>(StateMachineInstance->EditorStateMachineGraph);
			return CompiledData.GetAnimBlueprintDebugData().StateMachineDebugData.FindOrAdd(SourceGraph);
		}

		int32 FindState(UAnimStateNodeBase* StateNode)
		{
			if (int32* pResult = StateIndexTable.Find(StateNode))
			{
				return *pResult;
			}
			
			return INDEX_NONE;
		}

		int32 FindOrAddState(UAnimStateNodeBase* StateNode)
		{
			if (int32* pResult = StateIndexTable.Find(StateNode))
			{
				return *pResult;
			}
			else
			{
				FBakedAnimationStateMachine& BakedMachine = GetMachine();

				const int32 StateIndex = BakedMachine.States.Num();
				StateIndexTable.Add(StateNode, StateIndex);
				new (BakedMachine.States) FBakedAnimationState();

				UAnimStateNodeBase* SourceNode = CompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimStateNodeBase>(StateNode);
				GetMachineSpecificDebugData().NodeToStateIndex.Add(SourceNode, StateIndex);
				GetMachineSpecificDebugData().StateIndexToNode.Add(StateIndex, SourceNode);
				if (UAnimStateNode* SourceStateNode = Cast<UAnimStateNode>(SourceNode))
				{
					CompiledData.GetAnimBlueprintDebugData().StateGraphToNodeMap.Add(SourceStateNode->BoundGraph, SourceStateNode);
				}

				return StateIndex;
			}
		}

		void AddStateAliasTransitionMapping(UAnimStateAliasNode* AliasNode, const FStateMachineDebugData::FStateAliasTransitionStateIndexPair& TransitionStateIndexPair)
		{
			UAnimStateAliasNode* SourceNode = CompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimStateAliasNode>(AliasNode);
			GetMachineSpecificDebugData().StateAliasNodeToTransitionStatePairs.Add(SourceNode, TransitionStateIndexPair);
		}

		int32 FindOrAddTransition(FUniqueTransition UniqueTransition)
		{
			if (int32* pResult = TransitionIndexTable.Find(UniqueTransition))
			{
				return *pResult;
			}
			else
			{
				FBakedAnimationStateMachine& BakedMachine = GetMachine();

				const int32 TransitionIndex = BakedMachine.Transitions.Num();
				TransitionIndexTable.Add(UniqueTransition, TransitionIndex);
				new (BakedMachine.Transitions) FAnimationTransitionBetweenStates();

				UAnimStateTransitionNode* TransitionNode = UniqueTransition.Get<0>();
				UAnimStateTransitionNode* SourceTransitionNode = CompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimStateTransitionNode>(TransitionNode);
				GetMachineSpecificDebugData().NodeToTransitionIndex.Add(SourceTransitionNode, TransitionIndex);
				CompiledData.GetAnimBlueprintDebugData().TransitionGraphToNodeMap.Add(SourceTransitionNode->BoundGraph, SourceTransitionNode);

				if (SourceTransitionNode->CustomTransitionGraph != NULL)
				{
					CompiledData.GetAnimBlueprintDebugData().TransitionBlendGraphToNodeMap.Add(SourceTransitionNode->CustomTransitionGraph, SourceTransitionNode);
				}

				return TransitionIndex;
			}
		}

		void Validate()
		{
			FBakedAnimationStateMachine& BakedMachine = GetMachine();

			// Make sure there is a valid entry point
			if (BakedMachine.InitialState == INDEX_NONE)
			{
				CompilationContext.GetMessageLog().Warning(*LOCTEXT("NoEntryNode", "There was no entry state connection in @@").ToString(), StateMachineInstance);
				BakedMachine.InitialState = 0;
			}
			else
			{
				// Make sure the entry node is a state and not a conduit
				if (BakedMachine.States[BakedMachine.InitialState].bIsAConduit && !StateMachineInstance->GetNode().bAllowConduitEntryStates)
				{
					UEdGraphNode* StateNode = GetMachineSpecificDebugData().FindNodeFromStateIndex(BakedMachine.InitialState);
					CompilationContext.GetMessageLog().Error(*LOCTEXT("BadStateEntryNode", 
					"A conduit (@@) cannot be used as the entry node for a state machine. To enable this, check the 'Allow conduit entry states' checkbox for StateMachine. Warning, if a valid entry state cannot be found at runtime then this will generate a reference pose!"
					).ToString(), StateNode);
				}
			}
		}
	};
	
	UAnimBlueprintExtension_StateMachine* Extension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_StateMachine>(GetAnimBlueprint());
	check(Extension);

	if (EditorStateMachineGraph == NULL)
	{
		InCompilationContext.GetMessageLog().Error(*LOCTEXT("BadStateMachineNoGraph", "@@ does not have a corresponding graph").ToString(), this);
		return;
	}

	TMap<UAnimGraphNode_TransitionResult*, int32> AlreadyMergedTransitionList;

	TArray<FBakedAnimationStateMachine>& BakedStateMachines = OutCompiledData.GetBakedStateMachines();
	const int32 MachineIndex = BakedStateMachines.AddDefaulted();
	FMachineCreator Oven(this, MachineIndex, BakedStateMachines, InCompilationContext, OutCompiledData);

	// Map of states that contain a single player node (from state root node index to associated sequence player)
	TMap<int32, UObject*> SimplePlayerStatesMap;

	// Process all the states/transitions
	for (auto StateNodeIt = EditorStateMachineGraph->Nodes.CreateIterator(); StateNodeIt; ++StateNodeIt)
	{
		UEdGraphNode* Node = *StateNodeIt;

		if (UAnimStateNodeBase* StateNodeBase = Cast<UAnimStateNodeBase>(Node))
		{
			StateNodeBase->ValidateNodeDuringCompilation(InCompilationContext.GetMessageLog());
		}

		if (UAnimStateEntryNode* EntryNode = Cast<UAnimStateEntryNode>(Node))
		{
			// Handle the state graph entry
			FBakedAnimationStateMachine& BakedMachine = Oven.GetMachine();
			if (BakedMachine.InitialState != INDEX_NONE)
			{
				InCompilationContext.GetMessageLog().Error(*LOCTEXT("TooManyStateMachineEntryNodes", "Found an extra entry node @@").ToString(), EntryNode);
			}
			else if (UAnimStateAliasNode* AliasState = Cast<UAnimStateAliasNode>(EntryNode->GetOutputNode()))
			{
				InCompilationContext.GetMessageLog().Error(*LOCTEXT("AliasAsEntryState", "An alias (@@) cannot be used as the entry node for a state machine").ToString(), AliasState);
			}
			else if (UAnimStateNodeBase* StartState = Cast<UAnimStateNodeBase>(EntryNode->GetOutputNode()))
			{
				BakedMachine.InitialState = Oven.FindOrAddState(StartState);
			}
			else
			{
				InCompilationContext.GetMessageLog().Warning(*LOCTEXT("NoConnection", "Entry node @@ is not connected to state").ToString(), EntryNode);
			}
		}
		else if (UAnimStateNode* StateNode = Cast<UAnimStateNode>(Node))
		{
			const int32 StateIndex = Oven.FindOrAddState(StateNode);
			FBakedAnimationState& BakedState = Oven.GetMachine().States[StateIndex];

			if (StateNode->BoundGraph != NULL)
			{
				BakedState.StateName = StateNode->BoundGraph->GetFName();
				BakedState.StartNotify = OutCompiledData.FindOrAddNotify(StateNode->StateEntered);
				BakedState.EndNotify = OutCompiledData.FindOrAddNotify(StateNode->StateLeft);
				BakedState.FullyBlendedNotify = OutCompiledData.FindOrAddNotify(StateNode->StateFullyBlended);
				BakedState.bIsAConduit = false;
				BakedState.bAlwaysResetOnEntry = StateNode->bAlwaysResetOnEntry;

				// Process the inner graph of this state
				if (UAnimGraphNode_StateResult* AnimGraphResultNode = CastChecked<UAnimationStateGraph>(StateNode->BoundGraph)->GetResultNode())
				{
					InCompilationContext.ValidateGraphIsWellFormed(StateNode->BoundGraph);

					AnimGraphResultNode->Node.SetStateIndex(StateIndex);

					BakedState.StateRootNodeIndex = Extension->ExpandGraphAndProcessNodes(StateNode->BoundGraph, AnimGraphResultNode, InCompilationContext, OutCompiledData);

					// See if the state consists of a single sequence player node, and remember the index if so
					for (UEdGraphPin* TestPin : AnimGraphResultNode->Pins)
					{
						if ((TestPin->Direction == EGPD_Input) && (TestPin->LinkedTo.Num() == 1))
						{
							if (UAnimGraphNode_SequencePlayer* SequencePlayer = Cast<UAnimGraphNode_SequencePlayer>(TestPin->LinkedTo[0]->GetOwningNode()))
							{
								SimplePlayerStatesMap.Add(BakedState.StateRootNodeIndex, InCompilationContext.GetMessageLog().FindSourceObject(SequencePlayer));
							}
						}
					}
				}
				else
				{
					BakedState.StateRootNodeIndex = INDEX_NONE;
					InCompilationContext.GetMessageLog().Error(*LOCTEXT("StateWithNoResult", "@@ has no result node").ToString(), StateNode);
				}
			}
			else
			{
				BakedState.StateName = NAME_None;
				InCompilationContext.GetMessageLog().Error(*LOCTEXT("StateWithBadGraph", "@@ has no bound graph").ToString(), StateNode);
			}

			// If this check fires, then something in the machine has changed causing the states array to not
			// be a separate allocation, and a state machine inside of this one caused stuff to shift around
			checkSlow(&BakedState == &(Oven.GetMachine().States[StateIndex]));
		}
		else if (UAnimStateConduitNode* ConduitNode = Cast<UAnimStateConduitNode>(Node))
		{
			const int32 StateIndex = Oven.FindOrAddState(ConduitNode);
			FBakedAnimationState& BakedState = Oven.GetMachine().States[StateIndex];

			BakedState.StateName = ConduitNode->BoundGraph ? ConduitNode->BoundGraph->GetFName() : TEXT("OLD CONDUIT");
			BakedState.bIsAConduit = true;
			
			if (ConduitNode->BoundGraph != NULL)
			{
				if (UAnimGraphNode_TransitionResult* EntryRuleResultNode = CastChecked<UAnimationTransitionGraph>(ConduitNode->BoundGraph)->GetResultNode())
				{
					BakedState.EntryRuleNodeIndex = Extension->ExpandGraphAndProcessNodes(ConduitNode->BoundGraph, EntryRuleResultNode, InCompilationContext, OutCompiledData);
				}
			}

			// If this check fires, then something in the machine has changed causing the states array to not
			// be a separate allocation, and a state machine inside of this one caused stuff to shift around
			checkSlow(&BakedState == &(Oven.GetMachine().States[StateIndex]));
		}
		else if (UAnimStateAliasNode* StateAliasNode = Cast<UAnimStateAliasNode>(Node))
		{
			Oven.StateAliasNodes.Add(StateAliasNode);
		}
	}

	int32 NumStateAlias = Oven.StateAliasNodes.Num();
	for (auto AliasIndex = 0; AliasIndex < NumStateAlias; ++AliasIndex)
	{
		const UAnimStateAliasNode* StateAliasNode = Oven.StateAliasNodes[AliasIndex];

		auto MapAlias = [&](const UAnimStateNodeBase* StateNode)
		{
			if (StateNode)
			{
				if (int32* StateIndexPtr = Oven.StateIndexTable.Find(StateNode))
				{
					Oven.StateIndexToStateAliasNodeIndices.Add(*StateIndexPtr, AliasIndex);
				}
			}
		};

		if (StateAliasNode->bGlobalAlias)
		{
			for (auto StateNodeIt = Oven.StateIndexTable.CreateConstIterator(); StateNodeIt; ++StateNodeIt)
			{
				MapAlias(StateNodeIt->Key);
			}
		}
		else
		{
			for (auto StateNodeWeakIt = StateAliasNode->GetAliasedStates().CreateConstIterator(); StateNodeWeakIt; ++StateNodeWeakIt)
			{
				MapAlias(StateNodeWeakIt->Get());
			}
		}
	}

	// Process transitions after all the states because getters within custom graphs may want to
	// reference back to other states, which are only valid if they have already been baked
	for (auto StateNodeIt = Oven.StateIndexTable.CreateIterator(); StateNodeIt; ++StateNodeIt)
	{
		UAnimStateNodeBase* StateNode = StateNodeIt.Key();
		const int32 StateIndex = StateNodeIt.Value();

		FBakedAnimationState& BakedState = Oven.GetMachine().States[StateIndex];

		// Add indices to all player and layer nodes
		TArray<UEdGraph*> GraphsToCheck;
		TArray<UEdGraph*> SubGraphs = StateNode->GetSubGraphs();
		GraphsToCheck.Append(SubGraphs);
		for(UEdGraph* SubGraph : SubGraphs)
		{
			SubGraph->GetAllChildrenGraphs(GraphsToCheck);
		}

		TArray<UAnimGraphNode_LinkedAnimGraphBase*> LinkedAnimGraphNodes;
		TArray<UAnimGraphNode_AssetPlayerBase*> AssetPlayerNodes;
		TArray<UAnimGraphNode_RandomPlayer*> AssetRandomPlayerPlayerNodes;
		for (UEdGraph* ChildGraph : GraphsToCheck)
		{
			ChildGraph->GetNodesOfClass(AssetPlayerNodes);
			ChildGraph->GetNodesOfClass(AssetRandomPlayerPlayerNodes);
			ChildGraph->GetNodesOfClass(LinkedAnimGraphNodes);
		}

		for (UAnimGraphNode_AssetPlayerBase* Node : AssetPlayerNodes)
		{
			if (int32* IndexPtr = OutCompiledData.GetAnimBlueprintDebugData().NodeGuidToIndexMap.Find(Node->NodeGuid))
			{
				BakedState.PlayerNodeIndices.Add(*IndexPtr);
			}
		}

		for (UAnimGraphNode_RandomPlayer* Node : AssetRandomPlayerPlayerNodes)
		{
			if (int32* IndexPtr = OutCompiledData.GetAnimBlueprintDebugData().NodeGuidToIndexMap.Find(Node->NodeGuid))
			{
				BakedState.PlayerNodeIndices.Add(*IndexPtr);
			}
		}

		for (UAnimGraphNode_LinkedAnimGraphBase* Node : LinkedAnimGraphNodes)
		{
			if (int32* IndexPtr = OutCompiledData.GetAnimBlueprintDebugData().NodeGuidToIndexMap.Find(Node->NodeGuid))
			{
				BakedState.LayerNodeIndices.Add(*IndexPtr);
			}
		}
		// Handle all the transitions out of this node
		TArray<class UAnimStateTransitionNode*> TransitionList;

		// Add aliased state transitions to transition list
		TArray<int32> StateAliasIndices;
		Oven.StateIndexToStateAliasNodeIndices.MultiFind(StateIndex, StateAliasIndices);

		// Clone any alias transitions. Alias nodes share a single transition graph for all alias states. But we need a clone per alias state to allow
		// for each alias transition graph to have correct hardcoded state index pin values. Note: Alias's are simply decompiled into multiple connections.
		TMap<UAnimStateTransitionNode*, UEdGraph*> AliasTransitionGraphClones;

		for (const int32 AliasIndex : StateAliasIndices)
		{
			// Operate on a separate list to avoid re-processing nodes in the final list.
			TArray<UAnimStateTransitionNode*> AliasTransitionList;

			// Let the final transition list do the sort.
			Oven.StateAliasNodes[AliasIndex]->GetTransitionList(AliasTransitionList, /*bWantSortedList=*/ false);

			for (UAnimStateTransitionNode* AliasTransitionNode : AliasTransitionList)
			{
				if (UEdGraph* AliasTransitionGraph = AliasTransitionNode->BoundGraph)
				{
					UEdGraph* AliasTransitionGraphClone = FEdGraphUtilities::CloneGraph(AliasTransitionGraph, InCompilationContext.GetConsolidatedEventGraph(), &InCompilationContext.GetMessageLog(), true);

					// Note: We do this here instead of in `UAnimBlueprintExtension_StateMachine::AutoWireAnimGetter` since we have the context of correct state index.
					for (UEdGraphNode* Node : AliasTransitionGraphClone->Nodes)
					{
						if (UK2Node_AnimGetter* GetterNode = Cast<UK2Node_AnimGetter>(Node))
						{
							for (UEdGraphPin* Pin : GetterNode->Pins)
							{
								if (Pin->PinName == TEXT("StateIndex"))
								{
									Pin->DefaultValue = FString::FromInt(StateIndex);
								}
							}
						}
					}

					AliasTransitionGraphClones.Add(AliasTransitionNode, MoveTemp(AliasTransitionGraphClone));
				}
			}

			TransitionList.Append(MoveTemp(AliasTransitionList));
		}
		
		StateNode->GetTransitionList(/*out*/ TransitionList, /*bWantSortedList=*/ true);

		for (auto TransitionIt = TransitionList.CreateIterator(); TransitionIt; ++TransitionIt)
		{
			UAnimStateTransitionNode* TransitionNode = *TransitionIt;

			if (TransitionNode->bDisabled)
			{
				continue;
			}

			TransitionNode->BlendProfileWrapper.UpdateCachedBlendProfile();
			
			const int32 TransitionIndex = Oven.FindOrAddTransition(FMachineCreator::FUniqueTransition(TransitionNode, StateNode));
			FAnimationTransitionBetweenStates& BakedTransition = Oven.GetMachine().Transitions[TransitionIndex];

			BakedTransition.CrossfadeDuration = TransitionNode->CrossfadeDuration;
			BakedTransition.StartNotify = OutCompiledData.FindOrAddNotify(TransitionNode->TransitionStart);
			BakedTransition.EndNotify = OutCompiledData.FindOrAddNotify(TransitionNode->TransitionEnd);
			BakedTransition.InterruptNotify = OutCompiledData.FindOrAddNotify(TransitionNode->TransitionInterrupt);
			BakedTransition.BlendMode = TransitionNode->BlendMode;
			BakedTransition.CustomCurve = TransitionNode->CustomBlendCurve;
			BakedTransition.BlendProfile = TransitionNode->BlendProfileWrapper.GetBlendProfile();
			BakedTransition.MinTimeBeforeReentry = TransitionNode->MinTimeBeforeReentry;
			BakedTransition.LogicType = TransitionNode->LogicType;
			BakedTransition.bAllowInertializationForSelfTransitions = TransitionNode->bAllowInertializationForSelfTransitions;

			UAnimStateNodeBase* PreviousState = StateNode;
			UAnimStateNodeBase* NextState = TransitionNode->GetNextState();

			UAnimStateAliasNode* NextAliasNode = Cast<UAnimStateAliasNode>(NextState);
			if (NextAliasNode)
			{
				NextState = NextAliasNode->GetAliasedState();
			}

			if ((PreviousState != nullptr) && (NextState != nullptr))
			{
				const int32 PreviousStateIndex = Oven.FindState(PreviousState);
				const int32 NextStateIndex = Oven.FindState(NextState);

				if (NextAliasNode)
				{
					Oven.AddStateAliasTransitionMapping(NextAliasNode, { TransitionIndex, NextStateIndex });
				}

				if (TransitionNode->Bidirectional)
				{
					InCompilationContext.GetMessageLog().Warning(*LOCTEXT("BidirectionalTransWarning", "Bidirectional transitions aren't supported yet @@").ToString(), TransitionNode);
				}

				BakedTransition.PreviousState = PreviousStateIndex;
				BakedTransition.NextState = NextStateIndex;
			}

			if((BakedTransition.PreviousState == INDEX_NONE) || (BakedTransition.NextState == INDEX_NONE))
			{
				InCompilationContext.GetMessageLog().Error(*LOCTEXT("BogusTransition", "@@ is incomplete, without a previous or next state").ToString(), TransitionNode);
			}

			// Validate the blend profile for this transition - incase the skeleton of the node has
			// changed or the blend profile no longer exists.
			TransitionNode->ValidateBlendProfile();

			FBakedStateExitTransition& Rule = *new (BakedState.Transitions) FBakedStateExitTransition();

			UAnimStateNodeBase* TransitionPrevNode = TransitionNode->GetPreviousState();
			if (UAnimStateAliasNode* PrevAliasNode = Cast<UAnimStateAliasNode>(TransitionPrevNode))
			{
				Rule.bDesiredTransitionReturnValue = PrevAliasNode->bGlobalAlias ? true : PrevAliasNode->GetAliasedStates().Contains(StateNode);
				Oven.AddStateAliasTransitionMapping(PrevAliasNode, { TransitionIndex, StateIndex });
			}
			else
			{
				Rule.bDesiredTransitionReturnValue = (TransitionPrevNode == StateNode);
			}

			Rule.TransitionIndex = TransitionIndex;
			
			UEdGraph* TransitionGraph = TransitionNode->BoundGraph;
			if (UEdGraph** AliasTransitionGraphClone = AliasTransitionGraphClones.Find(TransitionNode))
			{
				TransitionGraph = *AliasTransitionGraphClone;
			}

			if (UAnimGraphNode_TransitionResult* TransitionResultNode = CastChecked<UAnimationTransitionGraph>(TransitionGraph)->GetResultNode())
			{
				if (int32* pIndex = AlreadyMergedTransitionList.Find(TransitionResultNode))
				{
					Rule.CanTakeDelegateIndex = *pIndex;
				}
				else
				{
					Rule.CanTakeDelegateIndex = Extension->ExpandGraphAndProcessNodes(TransitionGraph, TransitionResultNode, InCompilationContext, OutCompiledData, TransitionNode);
					AlreadyMergedTransitionList.Add(TransitionResultNode, Rule.CanTakeDelegateIndex);
				}

				if (TransitionNode->bAutomaticRuleBasedOnSequencePlayerInState)
				{
					if (UAnimationTransitionGraph* TransGraph = Cast<UAnimationTransitionGraph>(TransitionGraph))
					{
						if (UAnimGraphNode_TransitionResult* ResultNode = TransGraph->GetResultNode())
						{
							if (UEdGraphPin* CanExecPin = ResultNode->FindPin(TEXT("bCanEnterTransition")))
							{
								if (CanExecPin->LinkedTo.Num() > 0)
								{
									InCompilationContext.GetMessageLog().Note(*LOCTEXT("TransitionWithAutomaticRuleBased", "@@ has an automatic Rule Based Transition that will override graph exit rule.").ToString(), TransitionNode);
								}
							}
						}
					}
				}
			}
			else
			{
				Rule.CanTakeDelegateIndex = INDEX_NONE;
				InCompilationContext.GetMessageLog().Error(*LOCTEXT("TransitionWithNoResult", "@@ has no result node").ToString(), TransitionNode);
			}

			// Handle automatic time remaining rules
			Rule.bAutomaticRemainingTimeRule = TransitionNode->bAutomaticRuleBasedOnSequencePlayerInState;
			Rule.AutomaticRuleTriggerTime = TransitionNode->AutomaticRuleTriggerTime;
			Rule.SyncGroupNameToRequireValidMarkersRule = TransitionNode->SyncGroupNameToRequireValidMarkersRule;
			Rule.bOnlyEvaluateWhenActive = TransitionNode->bOnlyEvaluateWhenActive;

			// Handle custom transition graphs
			Rule.CustomResultNodeIndex = INDEX_NONE;
			if (UAnimationCustomTransitionGraph* CustomTransitionGraph = Cast<UAnimationCustomTransitionGraph>(TransitionNode->CustomTransitionGraph))
			{
				TArray<UEdGraphNode*> ClonedNodes;
				if (CustomTransitionGraph->GetResultNode())
				{
					Rule.CustomResultNodeIndex = Extension->ExpandGraphAndProcessNodes(TransitionNode->CustomTransitionGraph, CustomTransitionGraph->GetResultNode(), InCompilationContext, OutCompiledData, nullptr, &ClonedNodes);
				}

				// Find all the pose evaluators used in this transition, save handles to them because we need to populate some pose data before executing
				TArray<UAnimGraphNode_TransitionPoseEvaluator*> TransitionPoseList;
				for (auto ClonedNodesIt = ClonedNodes.CreateIterator(); ClonedNodesIt; ++ClonedNodesIt)
				{
					UEdGraphNode* Node = *ClonedNodesIt;
					if (UAnimGraphNode_TransitionPoseEvaluator* TypedNode = Cast<UAnimGraphNode_TransitionPoseEvaluator>(Node))
					{
						TransitionPoseList.Add(TypedNode);
					}
				}

				Rule.PoseEvaluatorLinks.Empty(TransitionPoseList.Num());

				for (auto TransitionPoseListIt = TransitionPoseList.CreateIterator(); TransitionPoseListIt; ++TransitionPoseListIt)
				{
					UAnimGraphNode_TransitionPoseEvaluator* TransitionPoseNode = *TransitionPoseListIt;
					Rule.PoseEvaluatorLinks.Add( InCompilationContext.GetAllocationIndexOfNode(TransitionPoseNode) );
				}
			}
		}
	}

	Oven.Validate();
}

void UAnimGraphNode_StateMachineBase::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	for (const UEdGraphNode* Node : EditorStateMachineGraph->Nodes)
	{
		if (const UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(Node))
		{
			if(TransitionNode->LogicType == ETransitionLogicType::TLT_Inertialization)
			{
				OutAttributes.Add(UE::Anim::IInertializationRequester::Attribute);
				break;
			}
		}
	}
}

void UAnimGraphNode_StateMachineBase::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	OutExtensions.Add(UAnimBlueprintExtension_StateMachine::StaticClass());
}

#undef LOCTEXT_NAMESPACE
