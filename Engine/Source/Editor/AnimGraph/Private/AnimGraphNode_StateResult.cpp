// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_StateResult.h"
#include "FindInBlueprintManager.h"
#include "GraphEditorSettings.h"
#include "IAnimBlueprintCompilationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_StateResult)

#define LOCTEXT_NAMESPACE "A3Nodes"

/////////////////////////////////////////////////////
// UAnimGraphNode_StateResult

UAnimGraphNode_StateResult::UAnimGraphNode_StateResult(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_StateResult::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText UAnimGraphNode_StateResult::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNodeStateResult_Title", "Output Animation Pose");
}

FText UAnimGraphNode_StateResult::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNodeStateResult_Tooltip", "This is the output of this animation state");
}

bool UAnimGraphNode_StateResult::IsSinkNode() const
{
	return true;
}

void UAnimGraphNode_StateResult::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Intentionally empty. This node is auto-generated when a new graph is created.
}

FString UAnimGraphNode_StateResult::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/AnimationStateMachine");
}

void UAnimGraphNode_StateResult::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{
	UAnimGraphNode_StateResult* TrueNode = InCompilationContext.GetMessageLog().FindSourceObjectTypeChecked<UAnimGraphNode_StateResult>(this);

	Node.SetName(TrueNode->GetGraph()->GetFName());

	// Resolve functions
	Node.StateEntryFunction.SetFromFunction(StateEntryFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
	Node.StateFullyBlendedInFunction.SetFromFunction(StateFullyBlendedInFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
	Node.StateExitFunction.SetFromFunction(StateExitFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
	Node.StateFullyBlendedOutFunction.SetFromFunction(StateFullyBlendedOutFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
}

void UAnimGraphNode_StateResult::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
	
	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateEntryFunction), StateEntryFunction, LOCTEXT("StateEntryFunctionName", "State Entry"), MessageLog);
	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateFullyBlendedInFunction), StateFullyBlendedInFunction, LOCTEXT("StateFullyBlendedInFunctionName", "State Fully Blended In"), MessageLog);
	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateExitFunction), StateExitFunction, LOCTEXT("StateExitFunctionName", "State Exit"), MessageLog);
	ValidateFunctionRef(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateFullyBlendedOutFunction), StateFullyBlendedOutFunction, LOCTEXT("StateFullyBlendedOutFunctionName", "State Fully Blended Out"), MessageLog);
}

void UAnimGraphNode_StateResult::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	bool bAnyAnimNodeFunctionChanged = false;
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateEntryFunction))
	{
		Node.StateEntryFunction.SetFromFunction(StateEntryFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		bAnyAnimNodeFunctionChanged = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateFullyBlendedInFunction))
	{
		Node.StateFullyBlendedInFunction.SetFromFunction(StateFullyBlendedInFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		bAnyAnimNodeFunctionChanged = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateExitFunction))
	{
		Node.StateExitFunction.SetFromFunction(StateExitFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		bAnyAnimNodeFunctionChanged = true;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateFullyBlendedOutFunction))
	{
		Node.StateFullyBlendedOutFunction.SetFromFunction(StateFullyBlendedOutFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()));
		bAnyAnimNodeFunctionChanged = true;
	}

	if (bAnyAnimNodeFunctionChanged)
	{
		GetAnimBlueprint()->RequestRefreshExtensions();
		GetSchema()->ReconstructNode(*this);
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_StateResult::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	const auto ConditionallyTagNodeFuncRef = [&OutTaggedMetaData](const FMemberReference& FuncMember, const FText& LocText)
	{
		if (IsPotentiallyBoundFunction(FuncMember))
		{
			const FText FunctionName = FText::FromName(FuncMember.GetMemberName());
			OutTaggedMetaData.Add(FSearchTagDataPair(LocText, FunctionName));
		}
	};

	// Conditionally include anim node function references as part of the node's search metadata
	ConditionallyTagNodeFuncRef(StateEntryFunction, LOCTEXT("StateEntryFunctionName", "State Entry"));
	ConditionallyTagNodeFuncRef(StateFullyBlendedInFunction, LOCTEXT("StateFullyBlendedInFunctionName", "State Fully Blended In"));
	ConditionallyTagNodeFuncRef(StateExitFunction, LOCTEXT("StateExitFunctionName", "State Exit"));
	ConditionallyTagNodeFuncRef(StateFullyBlendedOutFunction, LOCTEXT("StateFullyBlendedOutFunctionName", "State Fully Blended Out"));
}

void UAnimGraphNode_StateResult::GetBoundFunctionsInfo(TArray<TPair<FName, FName>>& InOutBindingsInfo)
{
	Super::GetBoundFunctionsInfo(InOutBindingsInfo);

	const FName CategoryName = TEXT("Functions|State");
	
	if (StateEntryFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateEntryFunction));
	}

	if (StateFullyBlendedInFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateFullyBlendedInFunction));
	}

	if (StateExitFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateExitFunction));
	}

	if (StateFullyBlendedOutFunction.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr)
	{
		InOutBindingsInfo.Emplace(CategoryName, GET_MEMBER_NAME_CHECKED(UAnimGraphNode_StateResult, StateFullyBlendedOutFunction));
	}
}

bool UAnimGraphNode_StateResult::ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const
{
	return Super::ReferencesFunction(InFunctionName, InScope)
	|| Node.StateEntryFunction.GetFunctionName() == InFunctionName
	|| Node.StateFullyBlendedInFunction.GetFunctionName() == InFunctionName
	|| Node.StateExitFunction.GetFunctionName() == InFunctionName
	|| Node.StateFullyBlendedOutFunction.GetFunctionName() == InFunctionName;
}

#undef LOCTEXT_NAMESPACE
