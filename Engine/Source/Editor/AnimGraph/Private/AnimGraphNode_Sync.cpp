// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_Sync.h"
#include "Kismet2/CompilerResultsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Sync)

#define LOCTEXT_NAMESPACE "AnimGraphNode_Sync"

void UAnimGraphNode_Sync::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNode_Sync, GroupName))
	{
		OnNodeTitleChangedEvent().Broadcast();
	}
}

FText UAnimGraphNode_Sync::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if(TitleType == ENodeTitleType::FullTitle)
	{
		return FText::Format(LOCTEXT("NodeFullTitle", "Sync\nSync group {0}"), FText::FromName(Node.GetGroupName()));
	}
	else
	{
		return LOCTEXT("NodeTitle", "Sync");
	}
}

FText UAnimGraphNode_Sync::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Synchronizes asset players and blendspaces that are children of this node.");
}

FText UAnimGraphNode_Sync::GetMenuCategory() const
{
	return LOCTEXT("Category", "Animation|Synchronization");
}

void UAnimGraphNode_Sync::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if(Node.GetGroupName() != NAME_None)
	{
		OutAttributes.Add(UE::Anim::FAnimSync::Attribute);
	}
}

void UAnimGraphNode_Sync::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if(Node.GetGroupName() == NAME_None)
	{
		MessageLog.Error(*LOCTEXT("NoSyncGroupSupplied", "Node @@ does not specifiy a sync group").ToString(), this);
	}
}

void UAnimGraphNode_Sync::BakeDataDuringCompilation(FCompilerResultsLog& MessageLog)
{
	if(Node.GetGroupName() != NAME_None)
	{
		UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
		AnimBlueprint->FindOrAddGroup(Node.GetGroupName());
	}
}

#undef LOCTEXT_NAMESPACE
