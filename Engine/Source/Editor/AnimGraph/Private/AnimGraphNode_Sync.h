// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodes/AnimNode_Sync.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_Sync.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_Sync : public UAnimGraphNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_Sync Node;

	// UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// UAnimGraphNode_Base interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual void BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog) override;
	virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
};