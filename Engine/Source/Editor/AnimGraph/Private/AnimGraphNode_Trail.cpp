// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_Trail.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PrimitiveDrawingUtils.h"
#include "UObject/ReleaseObjectVersion.h"
/////////////////////////////////////////////////////
// UAnimGraphNode_TrailBone

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Trail)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_Trail::UAnimGraphNode_Trail(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_Trail::GetControllerDescription() const
{
	return LOCTEXT("TrailController", "Trail controller");
}

FText UAnimGraphNode_Trail::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_Trail_Tooltip", "The Trail Controller.");
}

FText UAnimGraphNode_Trail::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.TrailBone.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.TrailBone.BoneName));

		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_Trail_ListTitle", "{ControllerDescription} - Bone: {BoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_Trail_Title", "{ControllerDescription}\nBone: {BoneName}"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

void UAnimGraphNode_Trail::PostLoad()
{
	Super::PostLoad();
	Node.PostLoad();
}

void UAnimGraphNode_Trail::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::TrailNodeBlendVariableNameChange)
	{
		if (Node.TrailBoneRotationBlendAlpha_DEPRECATED != 1.f)
		{
			Node.LastBoneRotationAnimAlphaBlend = FMath::Clamp<float>(1.f - Node.TrailBoneRotationBlendAlpha_DEPRECATED, 0.f, 1.f);
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}

void UAnimGraphNode_Trail::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_Trail, RelaxationSpeedScale))
	{
		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.RelaxationSpeedScaleInputProcessor.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_Trail::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
	{
		ReconstructNode();
	}

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_Trail, ChainLength) 
		|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_Trail, TrailBone))
	{
		Node.EnsureChainSize();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}



void UAnimGraphNode_Trail::OnNodeSelected(bool bInIsSelected, class FEditorModeTools& InModeTools, struct FAnimNode_Base* InRuntimeNode)
{
	if (InRuntimeNode)
	{
		((FAnimNode_Trail*)InRuntimeNode)->bEditorDebugEnabled = bInIsSelected;
		UAnimGraphNode_Base::OnNodeSelected(bInIsSelected, InModeTools, InRuntimeNode);
	}
}
void UAnimGraphNode_Trail::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const
{
	// initialize bone list
	// draw bone Angular limit
	if (Node.bLimitRotation && Node.ChainLength > 1 && PreviewSkelMeshComp->GetSkeletalMeshAsset())
	{
		TArray<FName> TrailBoneList;
		TrailBoneList.Reset(Node.ChainLength);
		TrailBoneList.AddDefaulted(Node.ChainLength);
		int32 CurrentIndex = Node.ChainLength - 1;
		TrailBoneList[CurrentIndex] = Node.TrailBone.BoneName;
		FName CurrentName = Node.TrailBone.BoneName;
		const FReferenceSkeleton& RefSkeleton = PreviewSkelMeshComp->GetSkeletalMeshAsset()->GetRefSkeleton();
		while (--CurrentIndex >= 0 && CurrentName != NAME_None)
		{
			const int32 ParentIndex = RefSkeleton.GetParentIndex(RefSkeleton.FindBoneIndex(CurrentName));
			TrailBoneList[CurrentIndex] = RefSkeleton.GetBoneName(ParentIndex);
			CurrentName = TrailBoneList[CurrentIndex];
		}

		DrawAngularLimits(PDI, PreviewSkelMeshComp, Node, TrailBoneList);
	}
}

void UAnimGraphNode_Trail::DrawAngularLimits(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp, const FAnimNode_Trail& NodeToVisualize, const TArray<FName>& TrailBoneList) const
{
	for (int32 Index = 0; Index < NodeToVisualize.RotationLimits.Num(); ++Index)
	{
		const FRotationLimit& AngularRangeLimit = NodeToVisualize.RotationLimits[Index];
		const int32 BoneIndex = SkelMeshComp->GetBoneIndex(TrailBoneList[Index]);
		if (BoneIndex != INDEX_NONE)
		{
			FTransform JointTransform = SkelMeshComp->GetBoneTransform(BoneIndex);

			FVector XAxis = JointTransform.GetUnitAxis(EAxis::X);
			FVector YAxis = JointTransform.GetUnitAxis(EAxis::Y);
			FVector ZAxis = JointTransform.GetUnitAxis(EAxis::Z);

			const FVector& MinAngles = AngularRangeLimit.LimitMin;
			const FVector& MaxAngles = AngularRangeLimit.LimitMax;
			FVector AngleRange = MaxAngles - MinAngles;
			FVector Middle = MinAngles + AngleRange * 0.5f;

			FTransform XAxisConeTM(YAxis, XAxis ^ YAxis, XAxis, JointTransform.GetTranslation());
			XAxisConeTM.SetRotation(FQuat(XAxis, FMath::DegreesToRadians(-Middle.X)) * XAxisConeTM.GetRotation());
			DrawCone(PDI, FScaleMatrix(30.0f) * XAxisConeTM.ToMatrixWithScale(), static_cast<float>(FMath::DegreesToRadians(AngleRange.X / 2.0)), 0.0f, 24, false, FLinearColor::Red, GEngine->ConstraintLimitMaterialX->GetRenderProxy(), SDPG_World);

			FTransform YAxisConeTM(ZAxis, YAxis ^ ZAxis, YAxis, JointTransform.GetTranslation());
			YAxisConeTM.SetRotation(FQuat(YAxis, FMath::DegreesToRadians(Middle.Y)) * YAxisConeTM.GetRotation());
			DrawCone(PDI, FScaleMatrix(30.0f) * YAxisConeTM.ToMatrixWithScale(), static_cast<float>(FMath::DegreesToRadians(AngleRange.Y / 2.0)), 0.0f, 24, false, FLinearColor::Green, GEngine->ConstraintLimitMaterialY->GetRenderProxy(), SDPG_World);

			FTransform ZAxisConeTM(XAxis, ZAxis ^ XAxis, ZAxis, JointTransform.GetTranslation());
			ZAxisConeTM.SetRotation(FQuat(ZAxis, FMath::DegreesToRadians(Middle.Z)) * ZAxisConeTM.GetRotation());
			DrawCone(PDI, FScaleMatrix(30.0f) * ZAxisConeTM.ToMatrixWithScale(), static_cast<float>(FMath::DegreesToRadians(AngleRange.Z / 2.0)), 0.0f, 24, false, FLinearColor::Blue, GEngine->ConstraintLimitMaterialZ->GetRenderProxy(), SDPG_World);
		}
	}
}
#undef LOCTEXT_NAMESPACE
