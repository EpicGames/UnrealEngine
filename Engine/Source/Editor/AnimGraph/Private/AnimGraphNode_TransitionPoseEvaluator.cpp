// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_TransitionPoseEvaluator.h"
#include "AnimationCustomTransitionGraph.h"

#include "Kismet2/CompilerResultsLog.h"

#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Animation/AnimAttributes.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_TransitionPoseEvaluator

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_TransitionPoseEvaluator)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_TransitionPoseEvaluator"

UAnimGraphNode_TransitionPoseEvaluator::UAnimGraphNode_TransitionPoseEvaluator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_TransitionPoseEvaluator::GetNodeTitleColor() const
{
	return FColor(200, 100, 100);
}

FText UAnimGraphNode_TransitionPoseEvaluator::GetTooltipText() const
{
	return (Node.DataSource == EEvaluatorDataSource::EDS_DestinationPose) ? LOCTEXT("GetDestinationStatePose_Tooltip", "Evaluates and returns the pose generated by the destination state of this transition") : LOCTEXT("GetSourceStatePose_Tooltip", "Evaluates and returns the pose generated by the setup prior to this transition firing");
}

FText UAnimGraphNode_TransitionPoseEvaluator::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::FullTitle)
	{
		return (Node.DataSource == EEvaluatorDataSource::EDS_DestinationPose) ? LOCTEXT("GetDestinationStatePose", "Get Destination State Pose") : LOCTEXT("GetSourceStatePose", "Get Source State Pose");
	}

	return LOCTEXT("InputPoseEvaluator", "Input Pose Evaluator");
}

void UAnimGraphNode_TransitionPoseEvaluator::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	if ((Node.EvaluatorMode != EEvaluatorMode::EM_Standard) && Node.FramesToCachePose < 1)
	{
		MessageLog.Error(TEXT("@@ is set to a mode that caches the pose, but frames to cache is less then 1."), this);
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

FText UAnimGraphNode_TransitionPoseEvaluator::GetMenuCategory() const
{
	return LOCTEXT("InputPoseEvaluatorCategory", "Animation|State Machines");
}

bool UAnimGraphNode_TransitionPoseEvaluator::CanUserDeleteNode() const
{
	// Allow deleting the node if we're in the wrong kind of graph (via some accident or regression)
	return !(GetGraph()->IsA(UAnimationCustomTransitionGraph::StaticClass()));
}

void UAnimGraphNode_TransitionPoseEvaluator::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Intentionally empty: Don't allow an option to create them, as they're auto-created when custom blend graphs are made
}

void UAnimGraphNode_TransitionPoseEvaluator::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	IDetailCategoryBuilder& PoseCategory = DetailBuilder.EditCategory("Pose", LOCTEXT("PoseCategoryName", "Pose"));

	FString CacheFramesPropertyName = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TransitionPoseEvaluator, FramesToCachePose));
	TSharedPtr<IPropertyHandle> FramesToCachePoseProperty = DetailBuilder.GetProperty(*CacheFramesPropertyName, GetClass());

	// Hide this property, we only want this to appear when using delayed freeze
	FramesToCachePoseProperty->MarkHiddenByCustomization();
	
	// Bind visibility helper for the property
	TAttribute<EVisibility> VisibilityAttr = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UAnimGraphNode_TransitionPoseEvaluator::GetCacheFramesVisibility, &DetailBuilder));
	PoseCategory.AddProperty(FramesToCachePoseProperty).Visibility(VisibilityAttr);
}

EVisibility UAnimGraphNode_TransitionPoseEvaluator::GetCacheFramesVisibility(IDetailLayoutBuilder* DetailLayoutBuilder)
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjectsList = DetailLayoutBuilder->GetSelectedObjects();
	for(TWeakObjectPtr<UObject> Object : SelectedObjectsList)
	{
		if(UAnimGraphNode_TransitionPoseEvaluator* TransitionPoseEvaluator = Cast<UAnimGraphNode_TransitionPoseEvaluator>(Object.Get()))
		{
			if(TransitionPoseEvaluator->Node.EvaluatorMode == EEvaluatorMode::EM_DelayedFreeze)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Hidden;
}

void UAnimGraphNode_TransitionPoseEvaluator::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
	OutAttributes.Add(UE::Anim::FAttributes::Attributes);
}

#undef LOCTEXT_NAMESPACE
