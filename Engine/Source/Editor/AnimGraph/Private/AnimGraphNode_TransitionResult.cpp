// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_TransitionResult.h"
#include "GraphEditorSettings.h"
#include "AnimStateTransitionNode.h"


/////////////////////////////////////////////////////
// UAnimGraphNode_TransitionResult

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_TransitionResult)

#define LOCTEXT_NAMESPACE "UAnimGraphNode_TransitionResult"

UAnimGraphNode_TransitionResult::UAnimGraphNode_TransitionResult(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor UAnimGraphNode_TransitionResult::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText UAnimGraphNode_TransitionResult::GetTooltipText() const
{
	return LOCTEXT("TransitionResultTooltip", "This expression is evaluated to determine if the state transition can be taken");
}

FText UAnimGraphNode_TransitionResult::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Result", "Result");
}

bool UAnimGraphNode_TransitionResult::ShowVisualWarning() const
{
	UObject * Outer = GetOuter();
	while (Outer != nullptr)
	{
		if (UAnimStateTransitionNode* AnimStateTransitionNode = Cast<UAnimStateTransitionNode>(Outer))
		{
			// only show warning if the auto rule is set and there is logic connected
			if (AnimStateTransitionNode->bAutomaticRuleBasedOnSequencePlayerInState)
			{
				UEdGraphPin* CanExecPin = FindPin(TEXT("bCanEnterTransition"));
				if (CanExecPin != nullptr && CanExecPin->LinkedTo.Num() > 0) 
				{
					return true;
				}
			}
			break;
		}
		Outer = Outer->GetOuter();
	}

	return false;
}

FText UAnimGraphNode_TransitionResult::GetVisualWarningTooltipText() const
{
	return LOCTEXT("TransitionResult_VisualWarning", "Warning : Automatic Rule Based Transition will override graph exit rule.");
}

void UAnimGraphNode_TransitionResult::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Intentionally empty. This node is auto-generated when a transition graph is created.
}

#undef LOCTEXT_NAMESPACE
