// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_TwistCorrectiveNode.h"
#include "Kismet2/CompilerResultsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_TwistCorrectiveNode)

#define LOCTEXT_NAMESPACE "TwistCorrectiveNode"

UAnimGraphNode_TwistCorrectiveNode::UAnimGraphNode_TwistCorrectiveNode(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer)
{

}

FText UAnimGraphNode_TwistCorrectiveNode::GetTooltipText() const
{
	return LOCTEXT("UAnimGraphNode_TwistCorrectiveNode_ToolTip", "Drives curve value (of usually morph target) using the transform of delta angle between base and twist frame to the direction of twist plane. ");
}

FText UAnimGraphNode_TwistCorrectiveNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((Node.BaseFrame.Bone.BoneName == NAME_None) && (Node.TwistFrame.Bone.BoneName == NAME_None) 
		&& ((TitleType == ENodeTitleType::ListView) || (TitleType == ENodeTitleType::MenuTitle)))
	{
		return GetControllerDescription();
	}
	else
	{
		FText FormattedText;

		FFormatNamedArguments Args;
		Args.Add(TEXT("BaseBone"), FText::FromName(Node.BaseFrame.Bone.BoneName));
		Args.Add(TEXT("TwistBone"), FText::FromName(Node.TwistFrame.Bone.BoneName));
		Args.Add(TEXT("CurveName"), FText::FromName(Node.CurveName));

		FormattedText = FText::Format(LOCTEXT("AnimGraphNode_TwistCorrectiveNode_Title", "Twist {CurveName} = {BaseBone}:{TwistBone} "), Args);
		return FormattedText;
	}	
}

FText UAnimGraphNode_TwistCorrectiveNode::GetControllerDescription() const
{
	return LOCTEXT("TwistCorrectiveNode", "Twist Corrective Node");
}

void UAnimGraphNode_TwistCorrectiveNode::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	if (Node.CurveName == NAME_None)
	{
		MessageLog.Warning(TEXT("@@ has missing Curve Name."), this);
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

#undef LOCTEXT_NAMESPACE
