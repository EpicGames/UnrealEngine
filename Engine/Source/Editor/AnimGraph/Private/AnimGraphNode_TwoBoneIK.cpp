// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_TwoBoneIK.h"
#include "AnimNodeEditModes.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"

// for customization details
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

// version handling
#include "AnimationCustomVersion.h"
#include "UObject/ReleaseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_TwoBoneIK)

#define LOCTEXT_NAMESPACE "AnimGraphNode_TwoBoneIK"

/////////////////////////////////////////////////////
// FTwoBoneIKDelegate

class FTwoBoneIKDelegate : public TSharedFromThis<FTwoBoneIKDelegate>
{
public:
	void UpdateLocationSpace(class IDetailLayoutBuilder* DetailBuilder)
	{
		if (DetailBuilder)
		{
			DetailBuilder->ForceRefreshDetails();
		}
	}
};

TSharedPtr<FTwoBoneIKDelegate> UAnimGraphNode_TwoBoneIK::TwoBoneIKDelegate = NULL;

/////////////////////////////////////////////////////
// UAnimGraphNode_TwoBoneIK


UAnimGraphNode_TwoBoneIK::UAnimGraphNode_TwoBoneIK(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_TwoBoneIK::GetControllerDescription() const
{
	return LOCTEXT("TwoBoneIK", "Two Bone IK");
}

FText UAnimGraphNode_TwoBoneIK::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_TwoBoneIK_Tooltip", "The Two Bone IK control applies an inverse kinematic (IK) solver to a 3-joint chain, such as the limbs of a character.");
}

FText UAnimGraphNode_TwoBoneIK::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if ((TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle) && (Node.IKBone.BoneName == NAME_None))
	{
		return GetControllerDescription();
	}
	// @TODO: the bone can be altered in the property editor, so we have to 
	//        choose to mark this dirty when that happens for this to properly work
	else //if (!CachedNodeTitles.IsTitleCached(TitleType, this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ControllerDescription"), GetControllerDescription());
		Args.Add(TEXT("BoneName"), FText::FromName(Node.IKBone.BoneName));

		// FText::Format() is slow, so we cache this to save on performance
		if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_IKBone_ListTitle", "{ControllerDescription} - Bone: {BoneName}"), Args), this);
		}
		else
		{
			CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("AnimGraphNode_IKBone_Title", "{ControllerDescription}\nBone: {BoneName}"), Args), this);
		}
	}
	return CachedNodeTitles[TitleType];
}

void UAnimGraphNode_TwoBoneIK::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_TwoBoneIK* TwoBoneIK = static_cast<FAnimNode_TwoBoneIK*>(InPreviewNode);

	// copies Pin values from the internal node to get data which are not compiled yet
	TwoBoneIK->EffectorLocation = Node.EffectorLocation;
	TwoBoneIK->JointTargetLocation = Node.JointTargetLocation;
}

void UAnimGraphNode_TwoBoneIK::CopyPinDefaultsToNodeData(UEdGraphPin* InPin)
{
	if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, EffectorLocation))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, EffectorLocation), Node.EffectorLocation);
	}
	else if (InPin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, JointTargetLocation))
	{
		GetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, JointTargetLocation), Node.JointTargetLocation);
	}
}

void UAnimGraphNode_TwoBoneIK::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	// initialize just once
	if (!TwoBoneIKDelegate.IsValid())
	{
		TwoBoneIKDelegate = MakeShareable(new FTwoBoneIKDelegate());
	}

	// do this first, so that we can include these properties first
	const FString EffectorLocationSpace = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, EffectorLocationSpace));
	const FString JointTargetLocationSpace = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, JointTargetLocationSpace));

	IDetailCategoryBuilder& IKCategory = DetailBuilder.EditCategory("IK");
	IDetailCategoryBuilder& EffectorCategory = DetailBuilder.EditCategory("Effector");
	IDetailCategoryBuilder& JointCategory = DetailBuilder.EditCategory("JointTarget");
	// refresh UIs when bone space is changed
	TSharedRef<IPropertyHandle> EffectorLocHandle = DetailBuilder.GetProperty(*EffectorLocationSpace, GetClass());
	if (EffectorLocHandle->IsValidHandle())
	{
		EffectorCategory.AddProperty(EffectorLocHandle);

		FSimpleDelegate UpdateEffectorSpaceDelegate = FSimpleDelegate::CreateSP(TwoBoneIKDelegate.Get(), &FTwoBoneIKDelegate::UpdateLocationSpace, &DetailBuilder);
		EffectorLocHandle->SetOnPropertyValueChanged(UpdateEffectorSpaceDelegate);
	}

	TSharedRef<IPropertyHandle> JointTragetLocHandle = DetailBuilder.GetProperty(*JointTargetLocationSpace, GetClass());
	if (JointTragetLocHandle->IsValidHandle())
	{
		JointCategory.AddProperty(JointTragetLocHandle);
		FSimpleDelegate UpdateJointSpaceDelegate = FSimpleDelegate::CreateSP(TwoBoneIKDelegate.Get(), &FTwoBoneIKDelegate::UpdateLocationSpace, &DetailBuilder);
		JointTragetLocHandle->SetOnPropertyValueChanged(UpdateJointSpaceDelegate);
	}

	EBoneControlSpace Space = Node.EffectorLocationSpace;
	const FString TakeRotationPropName = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, bTakeRotationFromEffectorSpace));
	const FString EffectorTargetName = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, EffectorTarget));
	const FString EffectorLocationPropName = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, EffectorLocation));

	if (Space == BCS_BoneSpace || Space == BCS_ParentBoneSpace)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle;
		PropertyHandle = DetailBuilder.GetProperty(*TakeRotationPropName, GetClass());
		EffectorCategory.AddProperty(PropertyHandle);
		PropertyHandle = DetailBuilder.GetProperty(*EffectorTargetName, GetClass());
		EffectorCategory.AddProperty(PropertyHandle);
	}
	else // hide all properties in EndEffector category
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(*EffectorLocationPropName, GetClass());
		DetailBuilder.HideProperty(PropertyHandle);
		PropertyHandle = DetailBuilder.GetProperty(*TakeRotationPropName, GetClass());
		DetailBuilder.HideProperty(PropertyHandle);
		PropertyHandle = DetailBuilder.GetProperty(*EffectorTargetName, GetClass());
		DetailBuilder.HideProperty(PropertyHandle);
	}

	Space = Node.JointTargetLocationSpace;
	bool bPinVisibilityChanged = false;
	const FString JointTargetName = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, JointTarget));
	const FString JointTargetLocation = FString::Printf(TEXT("Node.%s"), GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, JointTargetLocation));

	if (Space == BCS_BoneSpace || Space == BCS_ParentBoneSpace)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle;
		PropertyHandle = DetailBuilder.GetProperty(*JointTargetName, GetClass());
		JointCategory.AddProperty(PropertyHandle);
	}
	else // hide all properties in JointTarget category except for JointTargetLocationSpace
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(*JointTargetName, GetClass());
		DetailBuilder.HideProperty(PropertyHandle);
	}
}

FEditorModeID UAnimGraphNode_TwoBoneIK::GetEditorMode() const
{
	return AnimNodeEditModes::TwoBoneIK;
}

void UAnimGraphNode_TwoBoneIK::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAnimationCustomVersion::GUID);

	const int32 CustomAnimVersion = Ar.CustomVer(FAnimationCustomVersion::GUID);

	if (CustomAnimVersion < FAnimationCustomVersion::RenamedStretchLimits)
	{
		// fix up deprecated variables
		Node.StartStretchRatio = Node.StretchLimits_DEPRECATED.X;
		Node.MaxStretchScale = Node.StretchLimits_DEPRECATED.Y;
	}

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::RenameNoTwistToAllowTwistInTwoBoneIK)
	{
		Node.bAllowTwist = !Node.bNoTwist_DEPRECATED;
	}

	if (CustomAnimVersion < FAnimationCustomVersion::ConvertIKToSupportBoneSocketTarget)
	{
		if (Node.EffectorSpaceBoneName_DEPRECATED != NAME_None)
		{
			Node.EffectorTarget = FBoneSocketTarget(Node.EffectorSpaceBoneName_DEPRECATED);
		}

		if (Node.JointTargetSpaceBoneName_DEPRECATED != NAME_None)
		{
			Node.JointTarget = FBoneSocketTarget(Node.JointTargetSpaceBoneName_DEPRECATED);
		}
	}
}

void UAnimGraphNode_TwoBoneIK::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const
{
	if (bEnableDebugDraw && SkelMeshComp)
	{
		if (FAnimNode_TwoBoneIK* ActiveNode = GetActiveInstanceNode<FAnimNode_TwoBoneIK>(SkelMeshComp->GetAnimInstance()))
		{
			ActiveNode->ConditionalDebugDraw(PDI, SkelMeshComp);
		}
	}
}

#undef LOCTEXT_NAMESPACE
