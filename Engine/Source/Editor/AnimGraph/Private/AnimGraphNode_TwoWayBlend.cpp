// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_TwoWayBlend.h"
#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_TwoWayBlend

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_TwoWayBlend)

#define LOCTEXT_NAMESPACE "AnimGraphNode_TwoWayBlend"

UAnimGraphNode_TwoWayBlend::UAnimGraphNode_TwoWayBlend(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_TwoWayBlend::GetMenuCategory() const
{
	return LOCTEXT("TwoWayBlendCategory", "Animation|Blends");
}

void UAnimGraphNode_TwoWayBlend::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			BlendNode.AlphaBoolBlend.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

FLinearColor UAnimGraphNode_TwoWayBlend::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.75f);
}

FText UAnimGraphNode_TwoWayBlend::GetTooltipText() const
{
	return LOCTEXT("TwoWayBlendTooltip", "Blend two poses together");
}

FText UAnimGraphNode_TwoWayBlend::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("TwoWayBlend", "Two Way Blend");
}

void UAnimGraphNode_TwoWayBlend::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, Alpha))
	{
		Pin->bHidden = (BlendNode.AlphaInputType != EAnimAlphaInputType::Float);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = BlendNode.AlphaScaleBias.GetFriendlyName(BlendNode.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName));
		}
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, bAlphaBoolEnabled))
	{
		Pin->bHidden = (BlendNode.AlphaInputType != EAnimAlphaInputType::Bool);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, AlphaCurveName))
	{
		Pin->bHidden = (BlendNode.AlphaInputType != EAnimAlphaInputType::Curve);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = BlendNode.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_TwoWayBlend::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	// Reconstruct node to show updates to PinFriendlyNames.
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, AlphaScaleBias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
		|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
	{
		ReconstructNode();
	}

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, AlphaInputType))
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeAlphaInputType", "Change Alpha Input Type"));
		Modify();

		// Break links to pins going away
		for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Pins[PinIndex];
			if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, Alpha))
			{
				if (BlendNode.AlphaInputType != EAnimAlphaInputType::Float)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, bAlphaBoolEnabled))
			{
				if (BlendNode.AlphaInputType != EAnimAlphaInputType::Bool)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoWayBlend, AlphaCurveName))
			{
				if (BlendNode.AlphaInputType != EAnimAlphaInputType::Curve)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
		}

		ReconstructNode();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_TwoWayBlend::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("BlendNode")), GetClass());

	if (BlendNode.AlphaInputType != EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_TwoWayBlend, bAlphaBoolEnabled)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_TwoWayBlend, AlphaBoolBlend)));
	}

	if (BlendNode.AlphaInputType != EAnimAlphaInputType::Float)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_TwoWayBlend, Alpha)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_TwoWayBlend, AlphaScaleBias)));
	}

	if (BlendNode.AlphaInputType != EAnimAlphaInputType::Curve)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_TwoWayBlend, AlphaCurveName)));
	}

	if ((BlendNode.AlphaInputType != EAnimAlphaInputType::Float)
		&& (BlendNode.AlphaInputType != EAnimAlphaInputType::Curve))
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_TwoWayBlend, AlphaScaleBiasClamp)));
	}
}

#undef LOCTEXT_NAMESPACE

