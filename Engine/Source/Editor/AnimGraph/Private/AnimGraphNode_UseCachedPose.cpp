// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_UseCachedPose.h"
#include "AnimGraphNode_SaveCachedPose.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "Kismet2/CompilerResultsLog.h"
#include "BlueprintActionFilter.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "AnimBlueprintExtension_CachedPose.h"
#include "IAnimBlueprintCompilationContext.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "BlueprintEditorModule.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_UseCachedPose

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_UseCachedPose)

#define LOCTEXT_NAMESPACE "A3Nodes"

UAnimGraphNode_UseCachedPose::UAnimGraphNode_UseCachedPose(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_UseCachedPose::EarlyValidation(class FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);
	
	bool bRefreshSavCachedPoseNode = true;

	// Check to see the current cached node is still valid (and not deleted, by checking pin connections)
	if(SaveCachedPoseNode.IsValid())
	{
		// The node has a single pin, make sure it's there
		check(SaveCachedPoseNode->Pins.Num());

		// Deleted nodes have no links, otherwise we will be doing some wasted work on unlinked nodes
		if(SaveCachedPoseNode->Pins[0]->LinkedTo.Num())
		{
			// The node has links, it's valid, continue to use it
			bRefreshSavCachedPoseNode = false;
		}
	}

	// We need to refresh the cached pose node this node is linked to
	if(bRefreshSavCachedPoseNode && !NameOfCache.IsEmpty())
	{
		UBlueprint* GraphBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(GetGraph());
		check(GraphBlueprint);

		TArray<UEdGraph*> AllGraphs;
		GraphBlueprint->GetAllGraphs(AllGraphs);

		for(UEdGraph* Graph : AllGraphs)
		{
			// Get a list of all save cached pose nodes
			TArray<UAnimGraphNode_SaveCachedPose*> CachedPoseNodes;
			Graph->GetNodesOfClass(CachedPoseNodes);

			// Go through all the nodes and find one with a title that matches ours
			for (auto NodeIt = CachedPoseNodes.CreateIterator(); NodeIt; ++NodeIt)
			{
				if((*NodeIt)->CacheName == NameOfCache)
				{
					// Fix the original Blueprint node as well as the compiled version
					MessageLog.FindSourceObjectTypeChecked<UAnimGraphNode_UseCachedPose>(this)->SaveCachedPoseNode = *NodeIt;
					SaveCachedPoseNode = *NodeIt;
					break;
				}
			}
		}
	}
}

FText UAnimGraphNode_UseCachedPose::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_UseCachedPose_Tooltip", "References an animation tree elsewhere in the blueprint, which will be evaluated at most once per frame.\nCannot reference poses in other layers.");
}

FText UAnimGraphNode_UseCachedPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FFormatNamedArguments Args;
	if(SaveCachedPoseNode.IsValid())
	{
		NameOfCache = SaveCachedPoseNode->CacheName;
	}
	Args.Add(TEXT("CachePoseName"), FText::FromString(NameOfCache));
	return FText::Format(LOCTEXT("AnimGraphNode_UseCachedPose_Title", "Use cached pose '{CachePoseName}'"), Args);
}

void UAnimGraphNode_UseCachedPose::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		TSharedPtr<IBlueprintEditor> BlueprintEditor = FKismetEditorUtilities::GetIBlueprintEditorForObject(HyperlinkTarget, true);
		if(BlueprintEditor.IsValid())
		{
			BlueprintEditor->JumpToHyperlink(HyperlinkTarget, false);
		}
	}
}

UObject* UAnimGraphNode_UseCachedPose::GetJumpTargetForDoubleClick() const
{
	// Double click goes to the save cached pose for this node
	return SaveCachedPoseNode.Get();
}

FString UAnimGraphNode_UseCachedPose::GetNodeCategory() const
{
	return TEXT("Animation|Cached Poses");
}

void UAnimGraphNode_UseCachedPose::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FString CacheNodeName, UAnimGraphNode_SaveCachedPose* SaveCachePoseNode)
	{
		UAnimGraphNode_UseCachedPose* UseCachedPose = CastChecked<UAnimGraphNode_UseCachedPose>(NewNode);
		// we use an empty CacheName in GetNodeTitle() to relay the proper menu title
		UseCachedPose->SaveCachedPoseNode = SaveCachePoseNode;
	};


	UObject const* ActionKey = ActionRegistrar.GetActionKeyFilter();

	if(UBlueprint const* Blueprint = Cast<UBlueprint>(ActionKey))
	{
		// Get a list of all save cached pose nodes
		TArray<UAnimGraphNode_SaveCachedPose*> CachedPoseNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass<UAnimGraphNode_SaveCachedPose>(Blueprint, /*out*/ CachedPoseNodes);

		// Offer a use node for each of them with our layer in its outer chain
		for (UAnimGraphNode_SaveCachedPose* CachedPoseNode : CachedPoseNodes)
		{
			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda, CachedPoseNode->CacheName, CachedPoseNode);

			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

bool UAnimGraphNode_UseCachedPose::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;
	if(SaveCachedPoseNode.IsValid())
	{
		FBlueprintActionContext const& FilterContext = Filter.Context;
		for(UBlueprint* Blueprint : FilterContext.Blueprints)
		{
			if(SaveCachedPoseNode->GetBlueprint() != Blueprint)
			{
				bIsFilteredOut = true;
				break;
			}
		}

		for(UEdGraph* Graph : FilterContext.Graphs)
		{
			// Get layer we are in (i.e. the last outer that is a graph)
			UEdGraph* OuterLayer = Graph;
			UObject* TestGraphOuter = OuterLayer->GetOuter();
			while (TestGraphOuter)
			{
				if (UEdGraph* EdGraph = Cast<UEdGraph>(TestGraphOuter))
				{
					OuterLayer = EdGraph;
				}

				TestGraphOuter = TestGraphOuter->GetOuter();
			}

			// Check if the save cached pose node has the layer in its outer chain
			auto HasOuterLayerInOuterChain = [OuterLayer](UAnimGraphNode_SaveCachedPose* InCachedPoseNode)
			{
				UObject* OuterObject = InCachedPoseNode;
				while(UObject* TestOuterObject = OuterObject->GetOuter())
				{
					OuterObject = TestOuterObject;

					if(OuterObject == OuterLayer)
					{
						return true;
					}
				}

				return false;
			};

			if(!HasOuterLayerInOuterChain(SaveCachedPoseNode.Get()))
			{
				bIsFilteredOut = true;
				break;
			}
		}
	}
	return bIsFilteredOut;
}

void UAnimGraphNode_UseCachedPose::OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData)
{	
	UAnimBlueprintExtension_CachedPose* CompilerExtension = UAnimBlueprintExtension::GetExtension<UAnimBlueprintExtension_CachedPose>(GetAnimBlueprint());
	check(CompilerExtension);

	bool bSuccessful = false;

	// Link to the saved cached pose
	if(SaveCachedPoseNode.IsValid())
	{
		if (UAnimGraphNode_SaveCachedPose* AssociatedSaveNode = CompilerExtension->GetSaveCachedPoseNodes().FindRef(SaveCachedPoseNode->CacheName))
		{
			FStructProperty* LinkProperty = FindFProperty<FStructProperty>(FAnimNode_UseCachedPose::StaticStruct(), TEXT("LinkToCachingNode"));
			check(LinkProperty);

			FPoseLinkMappingRecord LinkRecord = FPoseLinkMappingRecord::MakeFromMember(this, AssociatedSaveNode, LinkProperty);
			if (LinkRecord.IsValid())
			{
				InCompilationContext.AddPoseLinkMappingRecord(LinkRecord);
			}
			bSuccessful = true;

			// Save CachePoseName for debug
			FName CachePoseName = FName(*SaveCachedPoseNode->CacheName);
			SaveCachedPoseNode->Node.CachePoseName = CachePoseName;
			Node.CachePoseName = CachePoseName;
		}
	}
	
	if(!bSuccessful)
	{
		InCompilationContext.GetMessageLog().Error(*LOCTEXT("NoAssociatedSaveNode", "@@ does not have an associated Save Cached Pose node").ToString(), this);
	}
}

void UAnimGraphNode_UseCachedPose::GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const
{
	OutExtensions.Add(UAnimBlueprintExtension_CachedPose::StaticClass());
}

#undef LOCTEXT_NAMESPACE
