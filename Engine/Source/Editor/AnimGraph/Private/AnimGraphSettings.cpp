// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphSettings.h"
#include "BlueprintActionDatabase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphSettings)

#define LOCTEXT_NAMESPACE "AnimGraphSettings"

FName UAnimGraphSettings::GetContainerName() const
{
	return TEXT("Editor");
}

FName UAnimGraphSettings::GetCategoryName() const
{
	return TEXT("ContentEditors");
}

#if WITH_EDITOR

FText UAnimGraphSettings::GetSectionText() const
{
	return LOCTEXT("SectionText", "Anim Graph");
}

FText UAnimGraphSettings::GetSectionDescription() const
{
	return LOCTEXT("SectionDescription", "Anim Graph Settings");
}

#endif // WITH_EDITOR

void UAnimGraphSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UAnimGraphSettings, bShowInstancedEnumBlendAnimNodeBlueprintActions))
	{
		FBlueprintActionDatabase::Get().RefreshAll();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#undef LOCTEXT_NAMESPACE
