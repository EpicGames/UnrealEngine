// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimPreviewAttacheInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimPreviewAttacheInstance)

#define LOCTEXT_NAMESPACE "AnimPreviewAttacheInstance"

void FAnimPreviewAttacheInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);

	FAnimationInitializeContext InitContext(this);
	CopyPoseFromMesh.bUseAttachedParent = true;
	CopyPoseFromMesh.bCopyCurves = true;
	CopyPoseFromMesh.Initialize_AnyThread(InitContext);
}

void FAnimPreviewAttacheInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
{
	FAnimInstanceProxy::PreUpdate(InAnimInstance, DeltaSeconds);

	CopyPoseFromMesh.PreUpdate(InAnimInstance);
}

void FAnimPreviewAttacheInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	UpdateCounter.Increment();
	
	CopyPoseFromMesh.Update_AnyThread(InContext);
}

bool FAnimPreviewAttacheInstanceProxy::Evaluate(FPoseContext& Output)
{
	// we cant evaluate on a worker thread here because of the key delegate needing to be fired
	CopyPoseFromMesh.Evaluate_AnyThread(Output);
	return true;
}

UAnimPreviewAttacheInstance::UAnimPreviewAttacheInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootMotionMode = ERootMotionMode::RootMotionFromEverything;
	bUseMultiThreadedAnimationUpdate = false;
}

FAnimInstanceProxy* UAnimPreviewAttacheInstance::CreateAnimInstanceProxy()
{
	return new FAnimPreviewAttacheInstanceProxy(this);
}

#undef LOCTEXT_NAMESPACE
