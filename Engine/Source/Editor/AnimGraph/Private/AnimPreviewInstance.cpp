// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimPreviewInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimationRuntime.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceHelpers.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif 

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimPreviewInstance)

#define LOCTEXT_NAMESPACE "AnimPreviewInstance"

void FAnimPreviewInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimSingleNodeInstanceProxy::Initialize(InAnimInstance);

	bSetKey = false;

	// link up our curve post-process mini-graph
	PoseBlendNode.SourcePose.SetLinkNode(&CurveSource);
	CurveSource.SourcePose.SetLinkNode(&SingleNode);

	FAnimationInitializeContext InitContext(this);
	PoseBlendNode.Initialize_AnyThread(InitContext);
	CurveSource.Initialize_AnyThread(InitContext);
}

void FAnimPreviewInstanceProxy::ResetModifiedBone(bool bCurveController)
{
	TArray<FAnimNode_ModifyBone>& Controllers = (bCurveController)?CurveBoneControllers : BoneControllers;
	Controllers.Empty();
}

FAnimNode_ModifyBone* FAnimPreviewInstanceProxy::FindModifiedBone(const FName& InBoneName, bool bCurveController)
{
	TArray<FAnimNode_ModifyBone>& Controllers = (bCurveController)?CurveBoneControllers : BoneControllers;

	return Controllers.FindByPredicate(
		[InBoneName](const FAnimNode_ModifyBone& InController) -> bool
	{
		return InController.BoneToModify.BoneName == InBoneName;
	}
	);
}

void FAnimPreviewInstanceProxy::SetAnimationAsset(UAnimationAsset* NewAsset, USkeletalMeshComponent* MeshComponent, bool bIsLooping, float InPlayRate)
{
	// reinitialize pose blend node for pose assets
	// this is necessary because sometimes in the editor, we add pose then list of pose changes, but 
	// this node continue use previous information
	// @todo: should we initialize all nodes?
	if (NewAsset && NewAsset->IsA(UPoseAsset::StaticClass()))
	{
		FAnimationInitializeContext Context(this);
		PoseBlendNode.Initialize_AnyThread(Context);
	}

	FAnimSingleNodeInstanceProxy::SetAnimationAsset(NewAsset, MeshComponent, bIsLooping, InPlayRate);
}

FAnimNode_ModifyBone& FAnimPreviewInstanceProxy::ModifyBone(const FName& InBoneName, bool bCurveController)
{
	FAnimNode_ModifyBone* SingleBoneController = FindModifiedBone(InBoneName, bCurveController);
	TArray<FAnimNode_ModifyBone>& Controllers = (bCurveController)?CurveBoneControllers : BoneControllers;

	if(SingleBoneController == nullptr)
	{
		int32 NewIndex = Controllers.Add(FAnimNode_ModifyBone());
		SingleBoneController = &Controllers[NewIndex];
	}

	SingleBoneController->BoneToModify.BoneName = InBoneName;

	if (bCurveController)
	{
		SingleBoneController->TranslationMode = BMM_Additive;
		SingleBoneController->TranslationSpace = BCS_BoneSpace;

		SingleBoneController->RotationMode = BMM_Additive;
		SingleBoneController->RotationSpace = BCS_BoneSpace;

		SingleBoneController->ScaleMode = BMM_Additive;
		SingleBoneController->ScaleSpace = BCS_BoneSpace;
	}
	else
	{
		SingleBoneController->TranslationMode = BMM_Replace;
		SingleBoneController->TranslationSpace = BCS_BoneSpace;

		SingleBoneController->RotationMode = BMM_Replace;
		SingleBoneController->RotationSpace = BCS_BoneSpace;

		SingleBoneController->ScaleMode = BMM_Replace;
		SingleBoneController->ScaleSpace = BCS_BoneSpace;
	}

	return *SingleBoneController;
}

void FAnimPreviewInstanceProxy::RemoveBoneModification(const FName& InBoneName, bool bCurveController)
{
	TArray<FAnimNode_ModifyBone>& Controllers = (bCurveController)?CurveBoneControllers : BoneControllers;
	Controllers.RemoveAll(
		[InBoneName](const FAnimNode_ModifyBone& InController)
	{
		return InController.BoneToModify.BoneName == InBoneName;
	}
	);
}

void FAnimPreviewInstanceProxy::Update(float DeltaSeconds)
{
	// we cant update on a worker thread here because of the key delegate needing to be fired
	check(IsInGameThread());

	FAnimSingleNodeInstanceProxy::Update(DeltaSeconds);
}

void FAnimPreviewInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	if (CopyPoseNode.SourceMeshComponent.IsValid())
	{
		CopyPoseNode.Update_AnyThread(InContext);
	}
	else if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(CurrentAsset))
	{
		PoseBlendNode.PoseAsset = PoseAsset;

		PoseBlendNode.Update_AnyThread(InContext);
	}
	else
	{
		FAnimSingleNodeInstanceProxy::UpdateAnimationNode(InContext);
	}
}

void FAnimPreviewInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
{
	FAnimSingleNodeInstanceProxy::PreUpdate(InAnimInstance, DeltaSeconds);

	if (CopyPoseNode.SourceMeshComponent.IsValid())
	{
		CopyPoseNode.PreUpdate(InAnimInstance);
	}

	CurveSource.PreUpdate(InAnimInstance);
}

bool FAnimPreviewInstanceProxy::Evaluate(FPoseContext& Output)
{
	// we cant evaluate on a worker thread here because of the key delegate needing to be fired
	check(IsInGameThread());

	if (CopyPoseNode.SourceMeshComponent.IsValid())
	{
		CopyPoseNode.Evaluate_AnyThread(Output);
	}
	else
	{
		if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(CurrentAsset))
		{
			PoseBlendNode.Evaluate_AnyThread(Output);
		}
		else
		{
			FAnimSingleNodeInstanceProxy::Evaluate(Output);
		}

		if (bEnableControllers)
		{
			UDebugSkelMeshComponent* Component = Cast<UDebugSkelMeshComponent>(GetSkelMeshComponent());
			if(Component)
			{
				// update curve controllers
				UpdateCurveController();

				// create bone controllers from 
				if(BoneControllers.Num() > 0 || CurveBoneControllers.Num() > 0)
				{
					FPoseContext PreController(Output), PostController(Output);
					// if set key is true, we should save pre controller local space transform 
					// so that we can calculate the delta correctly
					if(bSetKey)
					{
						PreController = Output;
					}

					FComponentSpacePoseContext ComponentSpacePoseContext(Output.AnimInstanceProxy);
					ComponentSpacePoseContext.Pose.InitPose(Output.Pose);

					// apply curve data first
					ApplyBoneControllers(CurveBoneControllers, ComponentSpacePoseContext);

					// and now apply bone controllers data
					// it is possible they can be overlapping, but then bone controllers will overwrite
					ApplyBoneControllers(BoneControllers, ComponentSpacePoseContext);

					// convert back to local @todo check this
					FCSPose<FCompactPose>::ConvertComponentPosesToLocalPoses(ComponentSpacePoseContext.Pose, Output.Pose);

					if(bSetKey)
					{
						// now we have post controller, and calculate delta now
						PostController = Output;
						SetKeyImplementation(PreController.Pose, PostController.Pose);
					}
				}
				// if any other bone is selected, still go for set key even if nothing changed
				else if(Component->BonesOfInterest.Num() > 0)
				{
					if(bSetKey)
					{
						// in this case, pose is same
						SetKeyImplementation(Output.Pose, Output.Pose);
					}
				}
			}

			// we should unset here, just in case somebody clicks the key when it's not valid
			if(bSetKey)
			{
				bSetKey = false;
			}
		}
	}

	return true;
}

void FAnimPreviewInstanceProxy::RefreshCurveBoneControllers(UAnimationAsset* AssetToRefreshFrom)
{
	// go through all curves and see if it has Transform Curve
	// if so, find what bone that belong to and create BoneMOdifier for them
	check(!CurrentAsset || CurrentAsset == AssetToRefreshFrom);
	UAnimSequence* CurrentSequence = Cast<UAnimSequence>(AssetToRefreshFrom);

	CurveBoneControllers.Empty();

	// do not apply if BakedAnimation is on
	if(CurrentSequence && CurrentSequence->IsDataModelValid())
	{
		// make sure if this needs source update
		if (CurrentSequence->GetDataModel()->GetNumberOfTransformCurves() == 0)
		{
			return;
		}

		GetRequiredBones().SetUseSourceData(true);

		const TArray<FTransformCurve>& Curves = CurrentSequence->GetDataModel()->GetCurveData().TransformCurves;
		USkeleton* MySkeleton = CurrentSequence->GetSkeleton();
		for (auto& Curve : Curves)
		{
			// skip if disabled
			if (Curve.GetCurveTypeFlag(AACF_Disabled))
			{
				continue;
			}

			// add bone modifier
			FName BoneName = Curve.GetName();
			if (BoneName != NAME_None && MySkeleton->GetReferenceSkeleton().FindBoneIndex(BoneName) != INDEX_NONE)
			{
				ModifyBone(BoneName, true);
			}
		}
	}
}

void FAnimPreviewInstanceProxy::UpdateCurveController()
{
	// evaluate the curve data first
	UAnimSequenceBase* CurrentSequence = Cast<UAnimSequenceBase>(CurrentAsset);
	USkeleton* PreviewSkeleton = (CurrentSequence) ? CurrentSequence->GetSkeleton() : nullptr;
	if (CurrentSequence && PreviewSkeleton)
	{
		TMap<FName, FTransform> ActiveCurves;
		UE::Anim::EvaluateTransformCurvesFromModel(CurrentSequence->GetDataModel(), ActiveCurves, GetCurrentTime(), 1.f);

		// make sure those curves exists in the bone controller, otherwise problem
		if ( ActiveCurves.Num() > 0 )
		{
			for(auto& SingleBoneController : CurveBoneControllers)
			{
				// make sure the curve exists
				FName CurveName = SingleBoneController.BoneToModify.BoneName;

				// we should add extra key to front and back whenever animation length changes or so. 
				// animation length change requires to bake down animation first 
				// this will make sure all the keys that were embedded at the start/end will automatically be backed to the data
				const FTransform* Value = ActiveCurves.Find(CurveName);
				if (Value)
				{
					// apply this change
					SingleBoneController.Translation = Value->GetTranslation();
					SingleBoneController.Scale = Value->GetScale3D();
					// sasd we're converting twice
					SingleBoneController.Rotation = Value->GetRotation().Rotator();
				}
			}
		}
		else
		{
			// should match
			ensure (CurveBoneControllers.Num() == 0);
			CurveBoneControllers.Empty();
		}
	}
}

void FAnimPreviewInstanceProxy::ApplyBoneControllers(TArray<FAnimNode_ModifyBone> &InBoneControllers, FComponentSpacePoseContext& ComponentSpacePoseContext)
{
	if(USkeleton* LocalSkeleton = ComponentSpacePoseContext.AnimInstanceProxy->GetSkeleton())
	{
		for (auto& SingleBoneController : InBoneControllers)
		{
			TArray<FBoneTransform> BoneTransforms;
			FAnimationCacheBonesContext Proxy(this);
			SingleBoneController.CacheBones_AnyThread(Proxy);
			if (SingleBoneController.IsValidToEvaluate(LocalSkeleton, ComponentSpacePoseContext.Pose.GetPose().GetBoneContainer()))
			{
				SingleBoneController.EvaluateSkeletalControl_AnyThread(ComponentSpacePoseContext, BoneTransforms);
				if (BoneTransforms.Num() > 0)
				{
					ComponentSpacePoseContext.Pose.LocalBlendCSBoneTransforms(BoneTransforms, 1.0f);
				}
			}
		}
	}
}

void FAnimPreviewInstanceProxy::SetKeyImplementation(const FCompactPose& PreControllerInLocalSpace, const FCompactPose& PostControllerInLocalSpace)
{
#if WITH_EDITOR
	// evaluate the curve data first
	UAnimSequence* CurrentSequence = Cast<UAnimSequence>(CurrentAsset);
	UDebugSkelMeshComponent* Component = Cast<UDebugSkelMeshComponent> (GetSkelMeshComponent());

	USkeleton* PreviewSkeleton = (CurrentSequence) ? CurrentSequence->GetSkeleton() : nullptr;
	if(CurrentSequence && PreviewSkeleton && Component && Component->GetSkeletalMeshAsset())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("SetKey", "Set Key"));
		CurrentSequence->Modify(true);
		GetAnimInstanceObject()->Modify();

		IAnimationDataController& Controller = CurrentSequence->GetController();
		Controller.OpenBracket(LOCTEXT("AnimPreviewInstanceProxy_SetKey", "Set Key"));

		TArray<FName> BonesToModify;
		// need to get component transform first. Depending on when this gets called, the transform is not up-to-date. 
		// first look at the bonecontrollers, and convert each bone controller to transform curve key
		// and add new curvebonecontrollers with additive data type
		// clear bone controller data
		for(auto& SingleBoneController : BoneControllers)
		{
			// find bone name, and just get transform of the bone in local space
			// and get the additive data
			// find if this already exists, then just add curve data only
			FName BoneName = SingleBoneController.BoneToModify.BoneName;
			// now convert data
			const FMeshPoseBoneIndex MeshBoneIndex(Component->GetBoneIndex(BoneName));
			const FCompactPoseBoneIndex BoneIndex = GetRequiredBones().MakeCompactPoseIndex(MeshBoneIndex);
			FTransform  LocalTransform = PostControllerInLocalSpace[BoneIndex];

			// now we have LocalTransform and get additive data
			FTransform AdditiveTransform = LocalTransform.GetRelativeTransform(PreControllerInLocalSpace[BoneIndex]);
			AddKeyToSequence(CurrentSequence, GetCurrentTime(), BoneName, AdditiveTransform);

			BonesToModify.Add(BoneName);
		}

		// see if the bone is selected right now and if that is added - if bone is selected, we should add identity key to it. 
		if ( Component->BonesOfInterest.Num() > 0 )
		{
			// if they're selected, we should add to the modifyBone list even if they're not modified, so that they can key that point. 
			// first make sure those are added 
			// if not added, make sure to set the key for them
			for (const auto& BoneIndex : Component->BonesOfInterest)
			{
				FName BoneName = Component->GetBoneName(BoneIndex);
				// if it's not on BonesToModify, add identity here. 
				if (!BonesToModify.Contains(BoneName))
				{
					AddKeyToSequence(CurrentSequence, GetCurrentTime(), BoneName, FTransform::Identity);
				}
			}
		}

		Controller.CloseBracket();

		ResetModifiedBone(false);

		OnSetKeyCompleteDelegate.Broadcast();
	}
#endif
}

void FAnimPreviewInstanceProxy::AddKeyToSequence(UAnimSequence* Sequence, float Time, const FName& BoneName, const FTransform& AdditiveTransform) 
{
	Sequence->AddKeyToSequence(Time, BoneName, AdditiveTransform);

	// now add to the controller
	// find if it exists in CurveBoneController
	// make sure you add it there
	ModifyBone(BoneName, true);

	GetRequiredBones().SetUseSourceData(true);
}

void FAnimPreviewInstanceProxy::SetDebugSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent)
{
	CopyPoseNode.SourceMeshComponent = InSkeletalMeshComponent;
	CopyPoseNode.bCopyCurves = true;
	CopyPoseNode.Initialize_AnyThread(FAnimationInitializeContext(this));
}

USkeletalMeshComponent* FAnimPreviewInstanceProxy::GetDebugSkeletalMeshComponent() const
{
	return CopyPoseNode.SourceMeshComponent.Get();
}

UAnimPreviewInstance::UAnimPreviewInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootMotionMode = ERootMotionMode::RootMotionFromEverything;
	bUseMultiThreadedAnimationUpdate = false;
}

static FArchive& operator<<(FArchive& Ar, FAnimNode_ModifyBone& ModifyBone)
{
	FAnimNode_ModifyBone::StaticStruct()->SerializeItem(Ar, &ModifyBone, nullptr);
	return Ar;
}

void UAnimPreviewInstance::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if(Ar.IsTransacting())
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();		
		Ar << Proxy.GetBoneControllers();
		Ar << Proxy.GetCurveBoneControllers();
	}
}

void UAnimPreviewInstance::NativeInitializeAnimation()
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

	// Cache our play state from the previous animation otherwise set to play
	bool bCachedIsPlaying = (CurrentAsset != nullptr) ? Proxy.IsPlaying() : true;

	Super::NativeInitializeAnimation();

	Proxy.SetPlaying(bCachedIsPlaying);

	Proxy.RefreshCurveBoneControllers(CurrentAsset);
}

void UAnimPreviewInstance::Montage_Advance(float DeltaTime)
{
	/*
		We're running in the Animation Editor.
		Call 'EditorOnly_PreAdvance' on montage instances.
		So they can do editor specific updates.
	*/
	for (int32 InstanceIndex = 0; InstanceIndex < MontageInstances.Num(); InstanceIndex++)
	{
		FAnimMontageInstance* const MontageInstance = MontageInstances[InstanceIndex];
		if (MontageInstance && MontageInstance->IsValid())
		{
			MontageInstance->EditorOnly_PreAdvance();
		}
	}

	Super::Montage_Advance(DeltaTime);
}

FAnimNode_ModifyBone* UAnimPreviewInstance::FindModifiedBone(const FName& InBoneName, bool bCurveController/*=false*/)
{
	return GetProxyOnGameThread<FAnimPreviewInstanceProxy>().FindModifiedBone(InBoneName, bCurveController);
}

FAnimNode_ModifyBone& UAnimPreviewInstance::ModifyBone(const FName& InBoneName, bool bCurveController/*=false*/)
{
	return GetProxyOnGameThread<FAnimPreviewInstanceProxy>().ModifyBone(InBoneName, bCurveController);
}

void UAnimPreviewInstance::RemoveBoneModification(const FName& InBoneName, bool bCurveController/*=false*/)
{
	GetProxyOnGameThread<FAnimPreviewInstanceProxy>().RemoveBoneModification(InBoneName, bCurveController);
}

void UAnimPreviewInstance::ResetModifiedBone(bool bCurveController/*=false*/)
{
	GetProxyOnGameThread<FAnimPreviewInstanceProxy>().ResetModifiedBone(bCurveController);
}

const TArray<FAnimNode_ModifyBone>& UAnimPreviewInstance::GetBoneControllers()
{
	return GetProxyOnGameThread<FAnimPreviewInstanceProxy>().GetBoneControllers();
}

#if WITH_EDITOR	

void UAnimPreviewInstance::SetKey()
{
	GetProxyOnGameThread<FAnimPreviewInstanceProxy>().SetKey();
}

FDelegateHandle UAnimPreviewInstance::AddKeyCompleteDelegate(FSimpleMulticastDelegate::FDelegate InOnSetKeyCompleteDelegate)
{
	return GetProxyOnGameThread<FAnimPreviewInstanceProxy>().AddKeyCompleteDelegate(InOnSetKeyCompleteDelegate);
}

void UAnimPreviewInstance::RemoveKeyCompleteDelegate(FDelegateHandle InDelegateHandle)
{
	GetProxyOnGameThread<FAnimPreviewInstanceProxy>().RemoveKeyCompleteDelegate(InDelegateHandle);
}

#endif

void UAnimPreviewInstance::RefreshCurveBoneControllers()
{
	GetProxyOnGameThread<FAnimPreviewInstanceProxy>().RefreshCurveBoneControllers(CurrentAsset);
}

/** Set SkeletalControl Alpha**/
void UAnimPreviewInstance::SetSkeletalControlAlpha(float InSkeletalControlAlpha)
{
	GetProxyOnGameThread<FAnimPreviewInstanceProxy>().SetSkeletalControlAlpha(InSkeletalControlAlpha);
}

UAnimSequence* UAnimPreviewInstance::GetAnimSequence()
{
	return Cast<UAnimSequence>(CurrentAsset);
}

void UAnimPreviewInstance::RestartMontage(UAnimMontage* Montage, FName FromSection)
{
	if (Montage == CurrentAsset)
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		MontagePreviewType = EMPT_Normal;
		// since this is preview, we would like not to blend in
		// just hard stop here
		Montage_Stop(0.0f, Montage);
		Montage_Play(Montage, Proxy.GetPlayRate());
		MontagePreview_RemoveBlendOut();
		if (FromSection != NAME_None)
		{
			Montage_JumpToSection(FromSection);
		}
		MontagePreview_SetLoopNormal(Proxy.IsLooping(), Montage->GetSectionIndex(FromSection));
	}
}

void UAnimPreviewInstance::SetAnimationAsset(UAnimationAsset* NewAsset, bool bIsLooping, float InPlayRate)
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

	// make sure to turn that off before setting new asset
	Proxy.GetRequiredBones().SetUseSourceData(false);
	
	// Reset Bone controllers when switching assets
	if (NewAsset != CurrentAsset)
	{
		Proxy.ResetModifiedBone();
	}

	Super::SetAnimationAsset(NewAsset, bIsLooping, InPlayRate);
	RootMotionMode = Cast<UAnimMontage>(CurrentAsset) != nullptr ? ERootMotionMode::RootMotionFromMontagesOnly : ERootMotionMode::RootMotionFromEverything;

	// disable playing for single frame assets
	UAnimSequence* AnimSequence = GetAnimSequence();
	bool bSingleFrameSequence = (AnimSequence != nullptr) ? AnimSequence->GetNumberOfSampledKeys() <= 1 : false;
	if (bSingleFrameSequence && Proxy.IsPlaying())
	{
		Proxy.SetPlaying(false);
	}

	// should re sync up curve bone controllers from new asset
	Proxy.RefreshCurveBoneControllers(CurrentAsset);
}

void UAnimPreviewInstance::MontagePreview_SetLooping(bool bIsLooping)
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();
	Proxy.SetLooping(bIsLooping);

	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		switch (MontagePreviewType)
		{
		case EMPT_AllSections:
			MontagePreview_SetLoopAllSections(Proxy.IsLooping());
			break;
		case EMPT_Normal:
		default:
			MontagePreview_SetLoopNormal(Proxy.IsLooping());
			break;
		}
	}
}

void UAnimPreviewInstance::MontagePreview_SetPlaying(bool bIsPlaying)
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();
	Proxy.SetPlaying(bIsPlaying);

	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		CurMontageInstance->bPlaying = Proxy.IsPlaying();
	}
	else if (Proxy.IsPlaying())
	{
		UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset);
		if (Montage)
		{
			switch (MontagePreviewType)
			{
			case EMPT_AllSections:
				MontagePreview_PreviewAllSections();
				break;
			case EMPT_Normal:
			default:
				MontagePreview_PreviewNormal();
				break;
			}
		}
	}
}

void UAnimPreviewInstance::MontagePreview_SetReverse(bool bInReverse)
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();
	Super::SetReverse(bInReverse);

	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		// copy the current playrate
		CurMontageInstance->SetPlayRate(Proxy.GetPlayRate());
	}
}

void UAnimPreviewInstance::MontagePreview_Restart()
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		switch (MontagePreviewType)
		{
		case EMPT_AllSections:
			MontagePreview_PreviewAllSections();
			break;
		case EMPT_Normal:
		default:
			MontagePreview_PreviewNormal();
			break;
		}
	}
}

void UAnimPreviewInstance::MontagePreview_StepForward()
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		bool bWasPlaying = IsPlayingMontage() && (Proxy.IsLooping() || Proxy.IsPlaying()); // we need to handle non-looped case separately, even if paused during playthrough
		MontagePreview_SetReverse(false);
		if (! bWasPlaying)
		{
			if (! Proxy.IsLooping())
			{
				float StoppedAt = Proxy.GetCurrentTime();
				if (! bWasPlaying)
				{
					// play montage but at last known location
					MontagePreview_Restart();
					SetPosition(StoppedAt, false);
				}
				int32 LastPreviewSectionIdx = MontagePreview_FindLastSection(MontagePreviewStartSectionIdx);
				if (FMath::Abs(Proxy.GetCurrentTime() - (Montage->CompositeSections[LastPreviewSectionIdx].GetTime() + Montage->GetSectionLength(LastPreviewSectionIdx))) <= MontagePreview_CalculateStepLength())
				{
					// we're at the end, jump right to the end
					Montage_JumpToSectionsEnd(Montage->GetSectionName(LastPreviewSectionIdx));
					if (! bWasPlaying)
					{
						MontagePreview_SetPlaying(false);
					}
					return; // can't go further than beginning of this
				}
			}
			else
			{
				MontagePreview_Restart();
			}
		}
		MontagePreview_SetPlaying(true);

		// Advance a single frame, leaving it paused afterwards
		const int32 NumKeys = Montage->GetNumberOfSampledKeys();
		// Add DELTA to prefer next frame when we're close to the boundary
		const float CurrentFraction = Proxy.GetCurrentTime() / Montage->GetPlayLength() + DELTA;
		const float NextFrame = FMath::Clamp<float>(FMath::FloorToFloat(CurrentFraction * NumKeys) + 1.0f, 0, NumKeys);
		const float NewTime = Montage->GetPlayLength() * (NextFrame / (NumKeys-1));

		GetSkelMeshComponent()->GlobalAnimRateScale = 1.0f;
		GetSkelMeshComponent()->TickAnimation(NewTime - Proxy.GetCurrentTime(), false);

		MontagePreview_SetPlaying(false);
	}
}

void UAnimPreviewInstance::MontagePreview_StepBackward()
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();
		bool bWasPlaying = IsPlayingMontage() && (Proxy.IsLooping() || Proxy.IsPlaying()); // we need to handle non-looped case separately, even if paused during playthrough
		MontagePreview_SetReverse(true);
		if (! bWasPlaying)
		{
			if (! Proxy.IsLooping())
			{
				float StoppedAt = Proxy.GetCurrentTime();
				if (! bWasPlaying)
				{
					// play montage but at last known location
					MontagePreview_Restart();
					SetPosition(StoppedAt, false);
				}
				int32 LastPreviewSectionIdx = MontagePreview_FindLastSection(MontagePreviewStartSectionIdx);
				if (FMath::Abs(Proxy.GetCurrentTime() - (Montage->CompositeSections[LastPreviewSectionIdx].GetTime() + Montage->GetSectionLength(LastPreviewSectionIdx))) <= MontagePreview_CalculateStepLength())
				{
					// special case as we could stop at the end of our last section which is also beginning of following section - we don't want to get stuck there, but be inside of our starting section
					Montage_JumpToSection(Montage->GetSectionName(LastPreviewSectionIdx));
				}
				else if (FMath::Abs(Proxy.GetCurrentTime() - Montage->CompositeSections[MontagePreviewStartSectionIdx].GetTime()) <= MontagePreview_CalculateStepLength())
				{
					// we're at the end of playing backward, jump right to the end
					Montage_JumpToSectionsEnd(Montage->GetSectionName(MontagePreviewStartSectionIdx));
					if (! bWasPlaying)
					{
						MontagePreview_SetPlaying(false);
					}
					return; // can't go further than beginning of first section
				}
			}
			else
			{
				MontagePreview_Restart();
			}
		}
		MontagePreview_SetPlaying(true);

		// Advance a single frame, leaving it paused afterwards
		const int32 NumKeys = Montage->GetNumberOfSampledKeys();
		// Add DELTA to prefer next frame when we're close to the boundary
		const float CurrentFraction = Proxy.GetCurrentTime() / Montage->GetPlayLength() + DELTA;
		const float NextFrame = FMath::Clamp<float>(FMath::FloorToFloat(CurrentFraction * NumKeys) - 1.0f, 0, NumKeys);
		const float NewTime = Montage->GetPlayLength() * (NextFrame / (NumKeys-1));

		GetSkelMeshComponent()->GlobalAnimRateScale = 1.0f;
		GetSkelMeshComponent()->TickAnimation(FMath::Abs(NewTime - Proxy.GetCurrentTime()), false);

		MontagePreview_SetPlaying(false);
	}
}

float UAnimPreviewInstance::MontagePreview_CalculateStepLength()
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		return static_cast<float>(Montage->GetSamplingFrameRate().AsInterval());
	}

	return static_cast<float>(UAnimationSettings::Get()->GetDefaultFrameRate().AsInterval());
}

void UAnimPreviewInstance::MontagePreview_JumpToStart()
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		int32 SectionIdx = 0;
		if (MontagePreviewType == EMPT_Normal)
		{
			SectionIdx = MontagePreviewStartSectionIdx;
		}
		// TODO hack - Montage_JumpToSection requires montage being played
		bool bWasPlaying = IsPlayingMontage();
		if (! bWasPlaying)
		{
			MontagePreview_Restart();
		}
		if (Proxy.GetPlayRate() < 0.f)
		{
			Montage_JumpToSectionsEnd(Montage->GetSectionName(SectionIdx));
		}
		else
		{
			Montage_JumpToSection(Montage->GetSectionName(SectionIdx));
		}
		if (! bWasPlaying)
		{
			MontagePreview_SetPlaying(false);
		}
	}
}

void UAnimPreviewInstance::MontagePreview_JumpToEnd()
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		int32 SectionIdx = 0;
		if (MontagePreviewType == EMPT_Normal)
		{
			SectionIdx = MontagePreviewStartSectionIdx;
		}
		// TODO hack - Montage_JumpToSectionsEnd requires montage being played
		bool bWasPlaying = IsPlayingMontage();
		if (! bWasPlaying)
		{
			MontagePreview_Restart();
		}
		if (Proxy.GetPlayRate() < 0.f)
		{
			Montage_JumpToSection(Montage->GetSectionName(MontagePreview_FindLastSection(SectionIdx)));
		}
		else
		{
			Montage_JumpToSectionsEnd(Montage->GetSectionName(MontagePreview_FindLastSection(SectionIdx)));
		}
		if (! bWasPlaying)
		{
			MontagePreview_SetPlaying(false);
		}
	}
}

void UAnimPreviewInstance::MontagePreview_JumpToPreviewStart()
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		int32 SectionIdx = 0;
		if (MontagePreviewType == EMPT_Normal)
		{
			SectionIdx = MontagePreviewStartSectionIdx;
		}
		// TODO hack - Montage_JumpToSectionsEnd requires montage being played
		bool bWasPlaying = IsPlayingMontage();
		if (! bWasPlaying)
		{
			MontagePreview_Restart();
		}
		Montage_JumpToSection(Montage->GetSectionName(Proxy.GetPlayRate() > 0.f? SectionIdx : MontagePreview_FindLastSection(SectionIdx)));
		if (! bWasPlaying)
		{
			MontagePreview_SetPlaying(false);
		}
	}
}

void UAnimPreviewInstance::MontagePreview_JumpToPosition(float NewPosition)
{
	SetPosition(NewPosition, false);
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		// this section will be first
		int32 NewMontagePreviewStartSectionIdx = MontagePreview_FindFirstSectionAsInMontage(Montage->GetSectionIndexFromPosition(NewPosition));
		if (MontagePreviewStartSectionIdx != NewMontagePreviewStartSectionIdx &&
			MontagePreviewType == EMPT_Normal)
		{
			MontagePreviewStartSectionIdx = NewMontagePreviewStartSectionIdx;
		}
		// setup looping to match normal playback
		MontagePreview_SetLooping(Proxy.IsLooping());
	}
}

void UAnimPreviewInstance::MontagePreview_RemoveBlendOut()
{
	if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
	{
		CurMontageInstance->DefaultBlendTimeMultiplier = 0.0f;
	}
}

void UAnimPreviewInstance::MontagePreview_PreviewNormal(int32 FromSectionIdx, bool bPlay)
{
	UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset);
	if (Montage && Montage->GetPlayLength() > 0.0f)
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		int32 PreviewFromSection = FromSectionIdx;
		if (FromSectionIdx != INDEX_NONE)
		{
			MontagePreviewStartSectionIdx = MontagePreview_FindFirstSectionAsInMontage(FromSectionIdx);
		}
		else
		{
			FromSectionIdx = MontagePreviewStartSectionIdx;
			PreviewFromSection = MontagePreviewStartSectionIdx;
		}
		MontagePreviewType = EMPT_Normal;
		// since this is preview, we would like not to blend in
		// just hard stop here
		Montage_Stop(0.0f, Montage);
		Montage_Play(Montage, Proxy.GetPlayRate());
		MontagePreview_SetLoopNormal(Proxy.IsLooping(), FromSectionIdx);
		Montage_JumpToSection(Montage->GetSectionName(PreviewFromSection));
		MontagePreview_RemoveBlendOut();
		Proxy.SetPlaying(bPlay);

		FAnimMontageInstance* MontageInstance = GetActiveMontageInstance();
		if (MontageInstance)
		{
			MontageInstance->SetWeight(1.0f);
			MontageInstance->bPlaying = Proxy.IsPlaying();
		}
	}
}

void UAnimPreviewInstance::MontagePreview_PreviewAllSections(bool bPlay)
{
	UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset);
	if (Montage && Montage->GetPlayLength() > 0.0f)
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		MontagePreviewType = EMPT_AllSections;
		// since this is preview, we would like not to blend in
		// just hard stop here
		Montage_Stop(0.0f, Montage);
		Montage_Play(Montage, Proxy.GetPlayRate());
		MontagePreview_SetLoopAllSections(Proxy.IsLooping());
		MontagePreview_JumpToPreviewStart();
		MontagePreview_RemoveBlendOut();
		Proxy.SetPlaying(bPlay);
		FAnimMontageInstance* MontageInstance = GetActiveMontageInstance();
		if (MontageInstance)
		{
			MontageInstance->SetWeight(1.0f);
			MontageInstance->bPlaying = Proxy.IsPlaying();
		}
	}
}

void UAnimPreviewInstance::MontagePreview_SetLoopNormal(bool bIsLooping, int32 PreferSectionIdx)
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		MontagePreview_ResetSectionsOrder();

		if (PreferSectionIdx == INDEX_NONE)
		{
			PreferSectionIdx = Montage->GetSectionIndexFromPosition(Proxy.GetCurrentTime());
		}
		int32 TotalSection = Montage->CompositeSections.Num();
		if (TotalSection > 0)
		{
			int PreferedInChain = TotalSection;
			TArray<bool> AlreadyUsed;
			AlreadyUsed.AddZeroed(TotalSection);
			while (true)
			{
				// find first not already used section
				int32 NotUsedIdx = 0;
				while (NotUsedIdx < TotalSection)
				{
					if (! AlreadyUsed[NotUsedIdx])
					{
						break;
					}
					++ NotUsedIdx;
				}
				if (NotUsedIdx >= TotalSection)
				{
					break;
				}
				// find if this is one we're looking for closest to starting one
				int32 CurSectionIdx = NotUsedIdx;
				int32 InChain = 0;
				while (true)
				{
					// find first that contains this
					if (CurSectionIdx == PreferSectionIdx &&
						InChain < PreferedInChain)
					{
						PreferedInChain = InChain;
						PreferSectionIdx = NotUsedIdx;
					}
					AlreadyUsed[CurSectionIdx] = true;
					FName NextSection = Montage->CompositeSections[CurSectionIdx].NextSectionName;
					CurSectionIdx = Montage->GetSectionIndex(NextSection);
					if (CurSectionIdx == INDEX_NONE || AlreadyUsed[CurSectionIdx]) // break loops
					{
						break;
					}
					++ InChain;
				}
				// loop this section
				SetMontageLoop(Montage, Proxy.IsLooping(), Montage->CompositeSections[NotUsedIdx].SectionName);
			}
			if (Montage->CompositeSections.IsValidIndex(PreferSectionIdx))
			{
				SetMontageLoop(Montage, Proxy.IsLooping(), Montage->CompositeSections[PreferSectionIdx].SectionName);
			}
		}
	}
}

void UAnimPreviewInstance::MontagePreview_SetLoopAllSetupSections(bool bIsLooping)
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		MontagePreview_ResetSectionsOrder();

		int32 TotalSection = Montage->CompositeSections.Num();
		if (TotalSection > 0)
		{
			FName FirstSection = Montage->CompositeSections[0].SectionName;
			FName PreviousSection = FirstSection;
			TArray<bool> AlreadyUsed;
			AlreadyUsed.AddZeroed(TotalSection);
			while (true)
			{
				// find first not already used section
				int32 NotUsedIdx = 0;
				while (NotUsedIdx < TotalSection)
				{
					if (! AlreadyUsed[NotUsedIdx])
					{
						break;
					}
					++ NotUsedIdx;
				}
				if (NotUsedIdx >= TotalSection)
				{
					break;
				}
				// go through all connected to join them into one big chain
				int CurSectionIdx = NotUsedIdx;
				while (true)
				{
					AlreadyUsed[CurSectionIdx] = true;
					FName CurrentSection = Montage->CompositeSections[CurSectionIdx].SectionName;
					Montage_SetNextSection(PreviousSection, CurrentSection);
					PreviousSection = CurrentSection;

					FName NextSection = Montage->CompositeSections[CurSectionIdx].NextSectionName;
					CurSectionIdx = Montage->GetSectionIndex(NextSection);
					if (CurSectionIdx == INDEX_NONE || AlreadyUsed[CurSectionIdx]) // break loops
					{
						break;
					}
				}
			}
			if (Proxy.IsLooping())
			{
				// and loop all
				Montage_SetNextSection(PreviousSection, FirstSection);
			}
		}
	}
}

void UAnimPreviewInstance::MontagePreview_SetLoopAllSections(bool bIsLooping)
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

		int32 TotalSection = Montage->CompositeSections.Num();
		if (TotalSection > 0)
		{
			if (Proxy.IsLooping())
			{
				for (int i = 0; i < TotalSection; ++ i)
				{
					Montage_SetNextSection(Montage->CompositeSections[i].SectionName, Montage->CompositeSections[(i+1) % TotalSection].SectionName);
				}
			}
			else
			{
				for (int i = 0; i < TotalSection - 1; ++ i)
				{
					Montage_SetNextSection(Montage->CompositeSections[i].SectionName, Montage->CompositeSections[i+1].SectionName);
				}
				Montage_SetNextSection(Montage->CompositeSections[TotalSection - 1].SectionName, NAME_None);
			}
		}
	}
}

void UAnimPreviewInstance::MontagePreview_ResetSectionsOrder()
{
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		int32 TotalSection = Montage->CompositeSections.Num();
		// restore to default
		for (int i = 0; i < TotalSection; ++ i)
		{
			Montage_SetNextSection(Montage->CompositeSections[i].SectionName, Montage->CompositeSections[i].NextSectionName);
		}
	}
}

int32 UAnimPreviewInstance::MontagePreview_FindFirstSectionAsInMontage(int32 ForSectionIdx)
{
	int32 ResultIdx = ForSectionIdx;
	// Montage does not have looping set up, so it should be valid and it gets
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		TArray<bool> AlreadyVisited;
		AlreadyVisited.AddZeroed(Montage->CompositeSections.Num());
		bool bFoundResult = false;
		while (! bFoundResult)
		{
			int32 UnusedSectionIdx = INDEX_NONE;
			for (int32 Idx = 0; Idx < Montage->CompositeSections.Num(); ++ Idx)
			{
				if (! AlreadyVisited[Idx])
				{
					UnusedSectionIdx = Idx;
					break;
				}
			}
			if (UnusedSectionIdx == INDEX_NONE)
			{
				break;
			}
			// check if this has ForSectionIdx
			int32 CurrentSectionIdx = UnusedSectionIdx;
			while (CurrentSectionIdx != INDEX_NONE && ! AlreadyVisited[CurrentSectionIdx])
			{
				if (CurrentSectionIdx == ForSectionIdx)
				{
					ResultIdx = UnusedSectionIdx;
					bFoundResult = true;
					break;
				}
				AlreadyVisited[CurrentSectionIdx] = true;
				FName NextSection = Montage->CompositeSections[CurrentSectionIdx].NextSectionName;
				CurrentSectionIdx = Montage->GetSectionIndex(NextSection);
			}
		}
	}
	return ResultIdx;
}

int32 UAnimPreviewInstance::MontagePreview_FindLastSection(int32 StartSectionIdx)
{
	int32 ResultIdx = StartSectionIdx;
	if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
	{
		if (FAnimMontageInstance* CurMontageInstance = GetActiveMontageInstance())
		{
			int32 TotalSection = Montage->CompositeSections.Num();
			if (TotalSection > 0)
			{
				TArray<bool> AlreadyVisited;
				AlreadyVisited.AddZeroed(TotalSection);
				int32 CurrentSectionIdx = StartSectionIdx;
				while (CurrentSectionIdx != INDEX_NONE && ! AlreadyVisited[CurrentSectionIdx])
				{
					AlreadyVisited[CurrentSectionIdx] = true;
					ResultIdx = CurrentSectionIdx;
					CurrentSectionIdx = CurMontageInstance->GetNextSectionID(CurrentSectionIdx);
				}
			}
		}
	}
	return ResultIdx;
}

void UAnimPreviewInstance::EnableControllers(bool bEnable)
{
	GetProxyOnGameThread<FAnimPreviewInstanceProxy>().EnableControllers(bEnable);
}

FAnimInstanceProxy* UAnimPreviewInstance::CreateAnimInstanceProxy()
{
	return new FAnimPreviewInstanceProxy(this);
}

void UAnimPreviewInstance::AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();
	Proxy.AddImpulseAtLocation(Impulse, Location, BoneName);
}

void UAnimPreviewInstance::SetDebugSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent)
{
	FAnimPreviewInstanceProxy& Proxy = GetProxyOnGameThread<FAnimPreviewInstanceProxy>();

	Proxy.InitializeObjects(this);
	Proxy.SetDebugSkeletalMeshComponent(InSkeletalMeshComponent);
	Proxy.ClearObjects();
}

USkeletalMeshComponent* UAnimPreviewInstance::GetDebugSkeletalMeshComponent() const
{
	return GetProxyOnGameThread<FAnimPreviewInstanceProxy>().GetDebugSkeletalMeshComponent();
}

#undef LOCTEXT_NAMESPACE
