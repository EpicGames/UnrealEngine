// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceTransformProviderFactories.h"
#include "Animation/AnimSequenceTransformProviderData.h"

UAnimSequenceTransformProviderLayerStackFactory::UAnimSequenceTransformProviderLayerStackFactory()
{
	SupportedClass = UAnimSequenceTransformProviderLayerStack::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

FText UAnimSequenceTransformProviderLayerStackFactory::GetDisplayName() const
{
	return NSLOCTEXT("AssetDefinition", "AssetDefinition_AnimSequenceTransformProviderLayerStack", "Anim Sequence Transform Provider Layer Stack");
}

UObject* UAnimSequenceTransformProviderLayerStackFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimSequenceTransformProviderLayerStack>(InParent, Class, Name, Flags);
}

UAnimSequenceTransformProviderSequenceListFactory::UAnimSequenceTransformProviderSequenceListFactory()
{
	SupportedClass = UAnimSequenceTransformProviderSequenceList::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

FText UAnimSequenceTransformProviderSequenceListFactory::GetDisplayName() const
{
	return NSLOCTEXT("AssetDefinition", "AssetDefinition_AnimSequenceTransformProviderSequenceList", "Anim Sequence Transform Provider Sequence List");
}

UObject* UAnimSequenceTransformProviderSequenceListFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimSequenceTransformProviderSequenceList>(InParent, Class, Name, Flags);
}

UAnimSequenceTransformProviderBlendSpaceListFactory::UAnimSequenceTransformProviderBlendSpaceListFactory()
{
	SupportedClass = UAnimSequenceTransformProviderBlendSpaceList::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

FText UAnimSequenceTransformProviderBlendSpaceListFactory::GetDisplayName() const
{
	return NSLOCTEXT("AssetDefinition", "UAssetDefinition_AnimSequenceTransformProviderBlendSpaceList", "Anim Sequence Transform Provider Blend Space List");
}

UObject* UAnimSequenceTransformProviderBlendSpaceListFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimSequenceTransformProviderBlendSpaceList>(InParent, Class, Name, Flags);
}
