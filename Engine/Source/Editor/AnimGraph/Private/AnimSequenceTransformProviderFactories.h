// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AnimSequenceTransformProviderFactories.generated.h"

UCLASS()
class UAnimSequenceTransformProviderLayerStackFactory : public UFactory
{
	GENERATED_BODY()

public:
	UAnimSequenceTransformProviderLayerStackFactory();
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

UCLASS()
class UAnimSequenceTransformProviderSequenceListFactory : public UFactory
{
	GENERATED_BODY()

public:
	UAnimSequenceTransformProviderSequenceListFactory();
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

UCLASS()
class UAnimSequenceTransformProviderBlendSpaceListFactory : public UFactory
{
	GENERATED_BODY()

public:
	UAnimSequenceTransformProviderBlendSpaceListFactory();
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
