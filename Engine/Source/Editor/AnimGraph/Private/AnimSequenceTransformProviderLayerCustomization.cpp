// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceTransformProviderLayerCustomization.h"
#include "Animation/AnimSequenceTransformProviderData.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"

TSharedRef<IPropertyTypeCustomization> FAnimSequenceTransformProviderLayerCustomization::MakeInstance()
{
	return MakeShareable(new FAnimSequenceTransformProviderLayerCustomization());
}

void FAnimSequenceTransformProviderLayerCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& Utils)
{
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
}

void FAnimSequenceTransformProviderLayerCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& Utils)
{
	const int32 LayerIndex = PropertyHandle->GetIndexInArray();
	const bool bIsBaseLayer = (LayerIndex == 0);

	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
	{
		TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		ChildBuilder.AddProperty(ChildHandle).IsEnabled(!bIsBaseLayer);
	}
}
