// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimStateAliasNode.h"

#include "AnimStateNode.h"
#include "Containers/Array.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStateAliasNode)

class FArchive;

#define LOCTEXT_NAMESPACE "AnimStateAliasNode"

/////////////////////////////////////////////////////
// UAnimStateAliasNode

UAnimStateAliasNode::UAnimStateAliasNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
	StateAliasName = TEXT("Alias");
}

void UAnimStateAliasNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	RebuildAliasedStateNodeReferences();
}

void UAnimStateAliasNode::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, TEXT("Transition"), TEXT("In"));
	CreatePin(EGPD_Output, TEXT("Transition"), TEXT("Out"));
}

void UAnimStateAliasNode::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if ((GetInputPin()->LinkedTo.Num() > 0) && (bGlobalAlias || AliasedStateNodes.Num() != 1))
	{
		MessageLog.Error(*LOCTEXT("AliasAsEntryState", "A alias (@@) used as a transition's target must alias a single state").ToString(), this);
	}
}

void UAnimStateAliasNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	Super::AutowireNewNode(FromPin);

	if (FromPin)
	{
		if (GetSchema()->TryCreateConnection(FromPin, GetInputPin()))
		{
			FromPin->GetOwningNode()->NodeConnectionListChanged();
		}
	}
}

void UAnimStateAliasNode::PostPasteNode()
{
	Super::PostPasteNode();

	// Find an interesting name, but try to keep the same if possible
	TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
	NameValidator->FindValidString(StateAliasName);
}

void UAnimStateAliasNode::PostPlacedNewNode()
{
	// Find an interesting name, but try to keep the same if possible
	TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
	NameValidator->FindValidString(StateAliasName);
}

FText UAnimStateAliasNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(GetStateName());
}

FText UAnimStateAliasNode::GetTooltipText() const
{
	return LOCTEXT("ConduitNodeTooltip", "This is a conduit, which allows specification of a predicate condition for an entire group of transitions");
}

FString UAnimStateAliasNode::GetStateName() const
{
	return StateAliasName;
}

void UAnimStateAliasNode::OnRenameNode(const FString& NewName)
{
	StateAliasName = NewName;
}

UEdGraphPin* UAnimStateAliasNode::GetInputPin() const
{
	return Pins[0];
}

UEdGraphPin* UAnimStateAliasNode::GetOutputPin() const
{
	return Pins[1];
}

FString UAnimStateAliasNode::GetDesiredNewNodeName() const
{
	return TEXT("Alias");
}

UObject* UAnimStateAliasNode::GetJumpTargetForDoubleClick() const
{
	return GetAliasedState();
}

const TSet<TWeakObjectPtr<UAnimStateNodeBase>>& UAnimStateAliasNode::GetAliasedStates() const
{
	return AliasedStateNodes;
}

TSet<TWeakObjectPtr<UAnimStateNodeBase>>& UAnimStateAliasNode::GetAliasedStates()
{
	return AliasedStateNodes;
}

UAnimStateNodeBase* UAnimStateAliasNode::GetAliasedState() const
{
	// If we alias more than one state, we return null
	if (bGlobalAlias || (AliasedStateNodes.Num() != 1))
	{
		return nullptr;
	}


	if (UAnimStateNodeBase* AliasedState = AliasedStateNodes.CreateConstIterator()->Get())
	{
		if (IsValidChecked(AliasedState))
		{
			if (const UEdGraph* Graph = GetGraph())
			{
				TArray<UAnimStateNodeBase*> StateNodes;
				Graph->GetNodesOfClassEx<UAnimStateNode, UAnimStateNodeBase>(StateNodes);
				return StateNodes.Contains(AliasedState) ? AliasedState : nullptr;
			}
		}
	}
	
	return nullptr;
}

void UAnimStateAliasNode::RebuildAliasedStateNodeReferences()
{
	TSet<TWeakObjectPtr<UAnimStateNodeBase>> NewAliasedStateNodes;

	// We don't use UEdGraphNode::GetGraph because this may be called during deletion and we don't want to assert on a missing graph.
	if (const UEdGraph* Graph = Cast<UEdGraph>(GetOuter()))
	{
		TArray<UAnimStateNodeBase*> StateNodes;
		Graph->GetNodesOfClassEx<UAnimStateNode, UAnimStateNodeBase>(StateNodes);
		TSet<UAnimStateNodeBase*> StateNodesSet(StateNodes);

		for (auto StateNodeIt = AliasedStateNodes.CreateIterator(); StateNodeIt; ++StateNodeIt)
		{
			UAnimStateNodeBase* AliasedState = StateNodeIt->Get();

			// Keep only nodes that are still in the graph
			if (IsValid(AliasedState) && StateNodesSet.Contains(AliasedState))
			{
				NewAliasedStateNodes.Add(*StateNodeIt);
			}
		}

		AliasedStateNodes = NewAliasedStateNodes;
	}
}

#undef LOCTEXT_NAMESPACE
