// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimStateConduitNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimStateTransitionNode.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimationTransitionGraph.h"
#include "AnimationConduitGraphSchema.h"
#include "AnimGraphNode_TransitionResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStateConduitNode)

#define LOCTEXT_NAMESPACE "AnimStateConduitNode"

/////////////////////////////////////////////////////
// UAnimStateConduitNode

UAnimStateConduitNode::UAnimStateConduitNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
}

void UAnimStateConduitNode::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, TEXT("Transition"), TEXT("In"));
	CreatePin(EGPD_Output, TEXT("Transition"), TEXT("Out"));
}

void UAnimStateConduitNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	Super::AutowireNewNode(FromPin);

	if (FromPin)
	{
		if (GetSchema()->TryCreateConnection(FromPin, GetInputPin()))
		{
			FromPin->GetOwningNode()->NodeConnectionListChanged();
		}
	}
}

FText UAnimStateConduitNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(GetStateName());
}

FText UAnimStateConduitNode::GetTooltipText() const
{
	return LOCTEXT("ConduitNodeTooltip", "This is a conduit, which allows specification of a predicate condition for an entire group of transitions");
}

FString UAnimStateConduitNode::GetStateName() const
{
	return (BoundGraph != NULL) ? *(BoundGraph->GetName()) : TEXT("(null)");
}

UEdGraphPin* UAnimStateConduitNode::GetInputPin() const
{
	return Pins[0];
}

UEdGraphPin* UAnimStateConduitNode::GetOutputPin() const
{
	return Pins[1];
}

void UAnimStateConduitNode::PostPlacedNewNode()
{
	// Create a new animation graph
	check(BoundGraph == NULL);
	BoundGraph = FBlueprintEditorUtils::CreateNewGraph(
		this,
		NAME_None,
		UAnimationTransitionGraph::StaticClass(),
		UAnimationConduitGraphSchema::StaticClass());
	check(BoundGraph);

	// Find an interesting name
	TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
	FBlueprintEditorUtils::RenameGraphWithSuggestion(BoundGraph, NameValidator, TEXT("Conduit"));

	// Initialize the transition graph
	const UEdGraphSchema* Schema = BoundGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*BoundGraph);

	// Add the new graph as a child of our parent graph
	UEdGraph* ParentGraph = GetGraph();

	if(ParentGraph->SubGraphs.Find(BoundGraph) == INDEX_NONE)
	{
		ParentGraph->SubGraphs.Add(BoundGraph);
	}
}

void UAnimStateConduitNode::DestroyNode()
{
	UEdGraph* GraphToRemove = BoundGraph;

	BoundGraph = NULL;
	Super::DestroyNode();

	if (GraphToRemove)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
		FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphToRemove, EGraphRemoveFlags::Recompile);
	}
}

void UAnimStateConduitNode::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	UAnimationTransitionGraph* TransGraph = CastChecked<UAnimationTransitionGraph>(BoundGraph);
	UAnimGraphNode_TransitionResult* ResultNode = TransGraph->GetResultNode();
	check(ResultNode);

	if (ResultNode->HasBinding(GET_MEMBER_NAME_CHECKED(FAnimNode_TransitionResult, bCanEnterTransition)))
	{
		// Rule is bound so nothing more to check
	}
	else
	{
		UEdGraphPin* BoolResultPin = ResultNode->Pins[0];
		if ((BoolResultPin->LinkedTo.Num() == 0) && (BoolResultPin->DefaultValue.ToBool() == false))
		{
			MessageLog.Warning(TEXT("@@ will never be taken, please connect something to @@"), this, BoolResultPin);
		}
	}
}

FString UAnimStateConduitNode::GetDesiredNewNodeName() const
{
	return TEXT("Conduit");
}

void UAnimStateConduitNode::PostPasteNode()
{
	// Find an interesting name, but try to keep the same if possible
	TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
	FBlueprintEditorUtils::RenameGraphWithSuggestion(BoundGraph, NameValidator, GetStateName());
	Super::PostPasteNode();
}

#undef LOCTEXT_NAMESPACE
