// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimStateNode.cpp
=============================================================================*/

#include "AnimStateEntryNode.h"

#include "AnimationStateMachineSchema.h"
#include "Containers/Array.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStateEntryNode)

#define LOCTEXT_NAMESPACE "AnimStateEntryNode"

/////////////////////////////////////////////////////
// UAnimStateEntryNode

UAnimStateEntryNode::UAnimStateEntryNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimStateEntryNode::AllocateDefaultPins()
{
	UEdGraphPin* Outputs = CreatePin(EGPD_Output, UAnimationStateMachineSchema::PC_Exec, TEXT("Entry"));
}

FText UAnimStateEntryNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraph* Graph = GetGraph();
	return FText::FromString(Graph->GetName());
}

FText UAnimStateEntryNode::GetTooltipText() const
{
	return LOCTEXT("StateEntryNodeTooltip", "Entry point for state machine");
}

UEdGraphNode* UAnimStateEntryNode::GetOutputNode() const
{
	if(Pins.Num() > 0 && Pins[0] != NULL)
	{
		check(Pins[0]->LinkedTo.Num() <= 1);
		if(Pins[0]->LinkedTo.Num() > 0 && Pins[0]->LinkedTo[0]->GetOwningNode() != NULL)
		{
			return Pins[0]->LinkedTo[0]->GetOwningNode();
		}
	}
	return NULL;
}

UEdGraphPin* UAnimStateEntryNode::GetOutputPin() const
{
	return FindPinChecked(TEXT("Entry"), EGPD_Output);
}

#undef LOCTEXT_NAMESPACE
