// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimStateNode.cpp
=============================================================================*/

#include "AnimStateNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimStateTransitionNode.h"
#include "AnimationStateGraph.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "AnimGraphNode_StateResult.h"
#include "AnimationStateGraphSchema.h"
#include "AnimGraphCommands.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStateNode)

#define LOCTEXT_NAMESPACE "AnimStateNode"

/////////////////////////////////////////////////////
// UAnimStateNode

UAnimStateNode::UAnimStateNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
	bAlwaysResetOnEntry = false;
}

void UAnimStateNode::AllocateDefaultPins()
{
	UEdGraphPin* Inputs = CreatePin(EGPD_Input, TEXT("Transition"), TEXT("In"));
	UEdGraphPin* Outputs = CreatePin(EGPD_Output, TEXT("Transition"), TEXT("Out"));
}

void UAnimStateNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	Super::AutowireNewNode(FromPin);

	//@TODO: If the FromPin is a state, create a transition between us
	if (FromPin)
	{
		if (GetSchema()->TryCreateConnection(FromPin, GetInputPin()))
		{
			FromPin->GetOwningNode()->NodeConnectionListChanged();
		}
	}
}

FText UAnimStateNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(GetStateName());
}

FText UAnimStateNode::GetTooltipText() const
{
	return LOCTEXT("AnimStateNode_Tooltip", "This is a state");
}

void UAnimStateNode::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == FName(TEXT("StateType")))
	{
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


FString UAnimStateNode::GetStateName() const
{
	return (BoundGraph != NULL) ? *(BoundGraph->GetName()) : TEXT("(null)");
}


UEdGraphPin* UAnimStateNode::GetInputPin() const
{
	return Pins[0];
}


UEdGraphPin* UAnimStateNode::GetOutputPin() const
{
	return Pins[1];
}

UEdGraphPin* UAnimStateNode::GetPoseSinkPinInsideState() const
{
	if (UAnimationStateGraph* StateGraph = Cast<UAnimationStateGraph>(BoundGraph))
	{
		return (StateGraph->MyResultNode != NULL) ? StateGraph->MyResultNode->FindPin(TEXT("Result")) : NULL;
	}
	else
	{
		return NULL;
	}
}

UAnimGraphNode_StateResult* UAnimStateNode::GetResultNodeInsideState() const
{
	if (UAnimationStateGraph* StateGraph = Cast<UAnimationStateGraph>(BoundGraph))
	{
		return StateGraph->MyResultNode;
	}
	else
	{
		return nullptr;
	}
}

void UAnimStateNode::PostPasteNode()
{
	// Find an interesting name, but try to keep the same if possible
	TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
	FBlueprintEditorUtils::RenameGraphWithSuggestion(BoundGraph, NameValidator, GetStateName());

	for (UEdGraphNode* GraphNode : BoundGraph->Nodes)
	{
		GraphNode->CreateNewGuid();
		GraphNode->PostPasteNode();
		GraphNode->ReconstructNode();
	}

	Super::PostPasteNode();
}

void UAnimStateNode::PostPlacedNewNode()
{
	// Create a new animation graph
	check(BoundGraph == NULL);
	BoundGraph = FBlueprintEditorUtils::CreateNewGraph(
		this,
		NAME_None,
		UAnimationStateGraph::StaticClass(),
		UAnimationStateGraphSchema::StaticClass());
	check(BoundGraph);

	// Find an interesting name
	TSharedPtr<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this);
	FBlueprintEditorUtils::RenameGraphWithSuggestion(BoundGraph, NameValidator, TEXT("State"));

	// Initialize the anim graph
	const UEdGraphSchema* Schema = BoundGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*BoundGraph);

	// Add the new graph as a child of our parent graph
	UEdGraph* ParentGraph = GetGraph();

	if(ParentGraph->SubGraphs.Find(BoundGraph) == INDEX_NONE)
	{
		ParentGraph->SubGraphs.Add(BoundGraph);
	}
}

void UAnimStateNode::DestroyNode()
{
	UEdGraph* GraphToRemove = BoundGraph;

	BoundGraph = NULL;
	Super::DestroyNode();

	if (GraphToRemove)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
		FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphToRemove, EGraphRemoveFlags::Recompile);
	}
}

void UAnimStateNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	// Add state node actions
	{
		FToolMenuSection& Section = Menu->AddSection("AnimStateNode", LOCTEXT("AnimStateNodeHeading", "State"));
		Section.AddMenuEntry(FAnimGraphCommands::Get().SetActiveState);
		Section.AddMenuEntry(FAnimGraphCommands::Get().CreateSelfTransition);
	}
}

#undef LOCTEXT_NAMESPACE
