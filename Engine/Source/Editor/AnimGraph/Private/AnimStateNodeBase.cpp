// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimStateNodeBase.cpp
=============================================================================*/

#include "AnimStateNodeBase.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimClassInterface.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationStateMachineSchema.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "AnimStateTransitionNode.h"
/////////////////////////////////////////////////////
// FAnimStateNodeNameValidator

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStateNodeBase)

class FAnimStateNodeNameValidator : public FStringSetNameValidator
{
public:
	FAnimStateNodeNameValidator(const UAnimStateNodeBase* InStateNode)
		: FStringSetNameValidator(FString())
	{
		Names.Add(TEXT("None"));

		TArray<UAnimStateNodeBase*> Nodes;
		UAnimationStateMachineGraph* StateMachine = CastChecked<UAnimationStateMachineGraph>(InStateNode->GetOuter());

		StateMachine->GetNodesOfClass<UAnimStateNodeBase>(Nodes);
		for (auto NodeIt = Nodes.CreateIterator(); NodeIt; ++NodeIt)
		{
			auto Node = *NodeIt;
			if (Node != InStateNode)
			{
				Names.Add(Node->GetStateName());
			}
		}

		// Include the name of animation layers
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(StateMachine);
		
		if (Blueprint)
		{
			UClass* TargetClass = *Blueprint->SkeletonGeneratedClass;
			if (TargetClass)
			{
				IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(TargetClass);
				for (const FAnimBlueprintFunction& AnimBlueprintFunction : AnimClassInterface->GetAnimBlueprintFunctions())
				{
					if (AnimBlueprintFunction.Name != UEdGraphSchema_K2::GN_AnimGraph)
					{
						Names.Add(AnimBlueprintFunction.Name.ToString());
					}
				}
			}
		}
	}
};

/////////////////////////////////////////////////////
// UAnimStateNodeBase

UAnimStateNodeBase::UAnimStateNodeBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimStateNodeBase::PostPasteNode()
{
	Super::PostPasteNode();
	
	for(UEdGraph* SubGraph : GetSubGraphs())
	{
		if(SubGraph)
		{
			// Add the new graph as a child of our parent graph
			UEdGraph* ParentGraph = GetGraph();

			if(ParentGraph->SubGraphs.Find(SubGraph) == INDEX_NONE)
			{
				ParentGraph->SubGraphs.Add(SubGraph);
			}

			for (UEdGraphNode* SubGraphNode : SubGraph->Nodes)
			{
				SubGraphNode->CreateNewGuid();
				SubGraphNode->PostPasteNode();
				SubGraphNode->ReconstructNode();
			}

			// Restore transactional flag that is lost during copy/paste process
			SubGraph->SetFlags(RF_Transactional);

			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
	}
}

UObject* UAnimStateNodeBase::GetJumpTargetForDoubleClick() const
{
	TArray<UEdGraph*> SubGraphs = GetSubGraphs();
	check(SubGraphs.Num() > 0);
	return SubGraphs[0];
}

bool UAnimStateNodeBase::CanJumpToDefinition() const
{
	return GetJumpTargetForDoubleClick() != nullptr;
}

void UAnimStateNodeBase::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(HyperlinkTarget);
	}
}

bool UAnimStateNodeBase::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(UAnimationStateMachineSchema::StaticClass());
}

void UAnimStateNodeBase::OnRenameNode(const FString& NewName)
{
	TArray<UEdGraph*> SubGraphs = GetSubGraphs();
	check(SubGraphs.Num() > 0);
	FBlueprintEditorUtils::RenameGraph(SubGraphs[0], NewName);
}

TSharedPtr<class INameValidatorInterface> UAnimStateNodeBase::MakeNameValidator() const
{
	return MakeShareable(new FAnimStateNodeNameValidator(this));
}

FString UAnimStateNodeBase::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/AnimationStateMachine");
}

void UAnimStateNodeBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
}

void UAnimStateNodeBase::PostLoad()
{
	Super::PostLoad();

	const int32 CustomVersion = GetLinkerCustomVersion(FFrameworkObjectVersion::GUID);

	if(CustomVersion < FFrameworkObjectVersion::FixNonTransactionalPins)
	{
		int32 BrokenPinCount = 0;
		for(UEdGraphPin_Deprecated* Pin : DeprecatedPins)
		{
			if(!Pin->HasAnyFlags(RF_Transactional))
			{
				++BrokenPinCount;
				Pin->SetFlags(Pin->GetFlags() | RF_Transactional);
			}
		}

		if(BrokenPinCount > 0)
		{
			UE_LOGF(LogAnimation, Log, "Fixed %d non-transactional pins in %ls", BrokenPinCount, *GetName());
		}
	}
}

void UAnimStateNodeBase::GetTransitionList(TArray<UAnimStateTransitionNode*>& OutTransitions, bool bWantSortedList /*= false*/) const
{
	// Normal transitions
	for (int32 LinkIndex = 0; LinkIndex < Pins[1]->LinkedTo.Num(); ++LinkIndex)
	{
		UEdGraphNode* TargetNode = Pins[1]->LinkedTo[LinkIndex]->GetOwningNode();
		if (UAnimStateTransitionNode* Transition = Cast<UAnimStateTransitionNode>(TargetNode))
		{
			OutTransitions.Add(Transition);
		}
	}

	// Bidirectional transitions where we are the 'backwards' link.
	// Conduits and other states types that don't support bidirectional transitions should hide it from the details panel.
	for (int32 LinkIndex = 0; LinkIndex < Pins[0]->LinkedTo.Num(); ++LinkIndex)
	{
		UEdGraphNode* TargetNode = Pins[0]->LinkedTo[LinkIndex]->GetOwningNode();
		if (UAnimStateTransitionNode* Transition = Cast<UAnimStateTransitionNode>(TargetNode))
		{
			// Anim state nodes that don't support bidirectional transitions should hide this property in FAnimTransitionNodeDetails::CustomizeDetails
			if (Transition->Bidirectional)
			{
				OutTransitions.Add(Transition);
			}
		}
	}

	// Sort the transitions by priority order, lower numbers are higher priority
	if (bWantSortedList)
	{
		struct FCompareTransitionsByPriority
		{
			FORCEINLINE bool operator()(const UAnimStateTransitionNode& A, const UAnimStateTransitionNode& B) const
			{
				return A.PriorityOrder < B.PriorityOrder;
			}
		};

		OutTransitions.Sort(FCompareTransitionsByPriority());
	}
}

UAnimBlueprint* UAnimStateNodeBase::GetAnimBlueprint() const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this);
	return CastChecked<UAnimBlueprint>(Blueprint);
}

