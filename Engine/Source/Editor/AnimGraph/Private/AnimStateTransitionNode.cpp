// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimStateTransitionNode.cpp
=============================================================================*/

#include "AnimStateTransitionNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/AnimInstance.h"
#include "AnimationStateGraph.h"
#include "AnimationTransitionGraph.h"
#include "AnimationTransitionSchema.h"
#include "AnimationCustomTransitionGraph.h"
#include "AnimationCustomTransitionSchema.h"
#include "AnimGraphCommands.h"
#include "AnimGraphNode_BlendSpacePlayer.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "AnimGraphNode_StateResult.h"
#include "AnimGraphNode_TransitionResult.h"
#include "AnimStateAliasNode.h"
#include "AnimStateConduitNode.h"
#include "Kismet2/CompilerResultsLog.h"
#include "EdGraphUtilities.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "Animation/BlendProfile.h"
#include "UObject/UE5MainStreamObjectVersion.h"

//////////////////////////////////////////////////////////////////////////
// IAnimStateTransitionNodeSharedDataHelper

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStateTransitionNode)

#define LOCTEXT_NAMESPACE "A3Nodes"

class ANIMGRAPH_API IAnimStateTransitionNodeSharedDataHelper
{
public:
	void UpdateSharedData(UAnimStateTransitionNode* Node, TSharedPtr<INameValidatorInterface> NameValidator);
	void MakeSureGuidExists(UAnimStateTransitionNode* Node);

protected:
	virtual bool CheckIfNodesShouldShareData(const UAnimStateTransitionNode* NodeA, const UAnimStateTransitionNode* NodeB) = 0;
	virtual bool CheckIfHasDataToShare(const UAnimStateTransitionNode* Node) = 0;
	virtual void ShareData(UAnimStateTransitionNode* NodeWhoWantsToShare, const UAnimStateTransitionNode* ShareFrom) = 0;
	virtual FString& AccessShareDataName(UAnimStateTransitionNode* Node) = 0;
	virtual FGuid& AccessShareDataGuid(UAnimStateTransitionNode* Node) = 0;
};

//////////////////////////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedRulesHelper

class ANIMGRAPH_API FAnimStateTransitionNodeSharedRulesHelper : public IAnimStateTransitionNodeSharedDataHelper
{
protected:
	virtual bool CheckIfNodesShouldShareData(const UAnimStateTransitionNode* NodeA, const UAnimStateTransitionNode* NodeB) override;
	virtual bool CheckIfHasDataToShare(const UAnimStateTransitionNode* Node) override;
	virtual void ShareData(UAnimStateTransitionNode* NodeWhoWantsToShare, const UAnimStateTransitionNode* ShareFrom) override;
	virtual FString& AccessShareDataName(UAnimStateTransitionNode* Node) override;
	virtual FGuid& AccessShareDataGuid(UAnimStateTransitionNode* Node) override;
};

//////////////////////////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedCrossfadeHelper

class ANIMGRAPH_API FAnimStateTransitionNodeSharedCrossfadeHelper : public IAnimStateTransitionNodeSharedDataHelper
{
protected:
	virtual bool CheckIfNodesShouldShareData(const UAnimStateTransitionNode* NodeA, const UAnimStateTransitionNode* NodeB) override;
	virtual bool CheckIfHasDataToShare(const UAnimStateTransitionNode* Node) override;
	virtual void ShareData(UAnimStateTransitionNode* NodeWhoWantsToShare, const UAnimStateTransitionNode* ShareFrom) override;
	virtual FString& AccessShareDataName(UAnimStateTransitionNode* Node) override;
	virtual FGuid& AccessShareDataGuid(UAnimStateTransitionNode* Node) override;
};

/////////////////////////////////////////////////////
// UAnimStateTransitionNode

UAnimStateTransitionNode::UAnimStateTransitionNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	CrossfadeDuration = 0.2f;
	BlendMode = EAlphaBlendOption::HermiteCubic;
	bAutomaticRuleBasedOnSequencePlayerInState = false;
	AutomaticRuleTriggerTime = -1.f;
	MinTimeBeforeReentry = -1.f;
	bSharedRules = false;
	SharedRulesGuid.Invalidate();
	bSharedCrossfade = false;
	SharedCrossfadeIdx = INDEX_NONE;
	SharedCrossfadeGuid.Invalidate();
	Bidirectional = false;
	bDisabled = false;
	bOnlyEvaluateWhenActive = false;
	PriorityOrder = 1;
	LogicType = ETransitionLogicType::TLT_StandardBlend;
	bAllowInertializationForSelfTransitions = false;	// Defaults to false to preserve existing behavior
}

void UAnimStateTransitionNode::AllocateDefaultPins()
{
	UEdGraphPin* Inputs = CreatePin(EGPD_Input, TEXT("Transition"), TEXT("In"));
	Inputs->bHidden = true;
	UEdGraphPin* Outputs = CreatePin(EGPD_Output, TEXT("Transition"), TEXT("Out"));
	Outputs->bHidden = true;
}

void UAnimStateTransitionNode::PostPlacedNewNode()
{
	CreateBoundGraph();

	// Only newly placed nodes fall back to inertialization for self transitions
	bAllowInertializationForSelfTransitions = true;
}

void UAnimStateTransitionNode::PostLoad()
{
	Super::PostLoad();

	// make sure we have guid for shared rules 
	if (bSharedRules && !SharedRulesGuid.IsValid())
	{
		FAnimStateTransitionNodeSharedRulesHelper().MakeSureGuidExists(this);
	}

	// make sure we have guid for shared crossfade 
	if (bSharedCrossfade && !SharedCrossfadeGuid.IsValid())
	{
		FAnimStateTransitionNodeSharedCrossfadeHelper().MakeSureGuidExists(this);
	}

	if(GetLinkerUEVersion() < VER_UE4_ADDED_NON_LINEAR_TRANSITION_BLENDS)
	{
		switch(CrossfadeMode_DEPRECATED)
		{
			case ETransitionBlendMode::TBM_Linear:
				BlendMode = EAlphaBlendOption::Linear;
				break;
			case ETransitionBlendMode::TBM_Cubic:
				// Old cubic was actually an in/out hermite polynomial (FMath::SmoothStep)
				BlendMode = EAlphaBlendOption::HermiteCubic;
				break;
			default:
				break;
		}
	}

	if(GetLinkerCustomVersion(FAnimPhysObjectVersion::GUID) < FAnimPhysObjectVersion::FixupBadBlendProfileReferences)
	{
		ValidateBlendProfile();
	}

	// Fix up previously pasted graphs that were not correctly added to the parent sub-graphs
	if (BoundGraph)
	{
		if (UEdGraph* ParentGraph = GetGraph())
		{
			ParentGraph->SubGraphs.AddUnique(BoundGraph);
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (BlendProfile_DEPRECATED)
	{
		BlendProfileWrapper.SetSkeletonBlendProfile(BlendProfile_DEPRECATED);

		BlendProfile_DEPRECATED = nullptr;

		Modify();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAnimStateTransitionNode::ValidateBlendProfile()
{
	if(TObjectPtr<UBlendProfile> BlendProfile = BlendProfileWrapper.GetBlendProfile())
	{
		// validate the skeleton of our blend profile
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
		UAnimBlueprint* AnimBP = CastChecked<UAnimBlueprint>(Blueprint);

		return AnimBP->TargetSkeleton == BlendProfile->GetSkeleton();
	}

	return false;
}

void UAnimStateTransitionNode::PostPasteNode()
{
	if (bSharedRules)
	{
		FAnimStateTransitionNodeSharedRulesHelper().UpdateSharedData(this, MakeShareable(new FAnimStateTransitionNodeSharedRulesNameValidator(this)));
	}

	if (bSharedCrossfade)
	{
		FAnimStateTransitionNodeSharedCrossfadeHelper().UpdateSharedData(this, MakeShareable(new FAnimStateTransitionNodeSharedCrossfadeNameValidator(this)));
	}

	if (BoundGraph == NULL)
	{
		// fail-safe, create empty transition graph
		CreateBoundGraph();
	}

	// Ensure transition graph is added to blueprint
	if (BoundGraph)
	{
		if (UEdGraph* ParentGraph = GetGraph())
		{
			ParentGraph->SubGraphs.AddUnique(BoundGraph);
		}
	}

	for (UEdGraphNode* GraphNode : BoundGraph->Nodes)
	{
		GraphNode->CreateNewGuid();
		GraphNode->PostPasteNode();
		GraphNode->ReconstructNode();
	}
	
	if(CustomTransitionGraph)
	{
		// Needs to be added to the parent graph
		UEdGraph* ParentGraph = GetGraph();

		if(ParentGraph->SubGraphs.Find(CustomTransitionGraph) == INDEX_NONE)
		{
			ParentGraph->SubGraphs.Add(CustomTransitionGraph);
		}

		// Transactional flag is lost in copy/paste, restore it.
		CustomTransitionGraph->SetFlags(RF_Transactional);

		for (UEdGraphNode* TransitionGraphNode : CustomTransitionGraph->Nodes)
		{
			TransitionGraphNode->CreateNewGuid();
			TransitionGraphNode->PostPasteNode();
			TransitionGraphNode->ReconstructNode();
		}
	}

	ValidateBlendProfile();

	Super::PostPasteNode();

	// We don't want to paste nodes in that aren't fully linked (transition nodes have fixed pins as they
	// really describe the connection between two other nodes). If we find one missing link, get rid of the node.
	for(UEdGraphPin* Pin : Pins)
	{
		if(Pin->LinkedTo.Num() == 0)
		{
			DestroyNode();
			break;
		}
	}
}

FText UAnimStateTransitionNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UAnimStateNodeBase* PrevState = GetPreviousState();
	UAnimStateNodeBase* NextState = GetNextState();

	if (!SharedRulesName.IsEmpty())
	{
		return FText::FromString(SharedRulesName);
	}
	else if ((PrevState != NULL) && (NextState != NULL))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PrevState"), FText::FromString(PrevState->GetStateName()));
		Args.Add(TEXT("NextState"), FText::FromString(NextState->GetStateName()));

		return FText::Format(LOCTEXT("PrevStateToNewState", "{PrevState} to {NextState}"), Args);
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BoundGraph"), (BoundGraph != NULL) ? FText::FromString(BoundGraph->GetName()) : LOCTEXT("Null", "(null)") );
		// @TODO: FText::Format() is slow, and we could benefit from caching 
		//        this off like we do for a lot of other nodes (but we have to
		//        make sure to invalidate the cached string at the appropriate 
		//        times).
		return FText::Format(LOCTEXT("TransitioNState", "Trans {BoundGraph}}"), Args);
	}
}

FText UAnimStateTransitionNode::GetTooltipText() const
{
	return LOCTEXT("StateTransitionTooltip", "This is a state transition");
}

UAnimStateNodeBase* UAnimStateTransitionNode::GetPreviousState() const
{
	if (Pins[0]->LinkedTo.Num() > 0)
	{
		return Cast<UAnimStateNodeBase>(Pins[0]->LinkedTo[0]->GetOwningNode());
	}
	else
	{
		return NULL;
	}
}

UAnimStateNodeBase* UAnimStateTransitionNode::GetNextState() const
{
	if (Pins[1]->LinkedTo.Num() > 0)
	{
		return Cast<UAnimStateNodeBase>(Pins[1]->LinkedTo[0]->GetOwningNode());
	}
	else
	{
		return NULL;
	}
}

FLinearColor UAnimStateTransitionNode::GetNodeTitleColor() const
{
	return FColorList::Red;
}

void UAnimStateTransitionNode::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin->LinkedTo.Num() == 0)
	{
		// Commit suicide; transitions must always have an input and output connection
		Modify();

		// Our parent graph will have our graph in SubGraphs so needs to be modified to record that.
		if(UEdGraph* ParentGraph = GetGraph())
		{
			ParentGraph->Modify();
		}

		DestroyNode();
	}
}

void UAnimStateTransitionNode::CreateConnections(UAnimStateNodeBase* PreviousState, UAnimStateNodeBase* NextState)
{
	// Previous to this
	Pins[0]->Modify();
	Pins[0]->LinkedTo.Empty();

	PreviousState->GetOutputPin()->Modify();
	Pins[0]->MakeLinkTo(PreviousState->GetOutputPin());

	// This to next
	Pins[1]->Modify();
	Pins[1]->LinkedTo.Empty();

	NextState->GetInputPin()->Modify();
	Pins[1]->MakeLinkTo(NextState->GetInputPin());
}

void UAnimStateTransitionNode::RelinkHead(UAnimStateNodeBase* NewTargetState)
{
	UAnimStateNodeBase* TargetStateBeforeRelinking = GetNextState();

	// Remove the incoming transition from the previous target state
	TargetStateBeforeRelinking->GetInputPin()->Modify();
	TargetStateBeforeRelinking->GetInputPin()->BreakLinkTo(GetOutputPin());

	// Add the new incoming transition to the new target state
	NewTargetState->GetInputPin()->Modify();
	NewTargetState->GetInputPin()->MakeLinkTo(GetOutputPin());

	// Relink the target state of the transition node
	Pins[1]->Modify();
	Pins[1]->BreakLinkTo(TargetStateBeforeRelinking->GetInputPin());
	Pins[1]->MakeLinkTo(NewTargetState->GetInputPin());
}

void UAnimStateTransitionNode::RelinkTail(UAnimStateNodeBase* NewSourceState)
{
	UAnimStateNodeBase* SourceStateBeforeRelinking = GetPreviousState();

	// Remove the outgoing transition from the previous source state
	SourceStateBeforeRelinking->GetOutputPin()->Modify();
	SourceStateBeforeRelinking->GetOutputPin()->BreakLinkTo(GetInputPin());

	// Add the new outgoing transition to the new source state
	NewSourceState->GetOutputPin()->Modify();
	NewSourceState->GetOutputPin()->MakeLinkTo(GetInputPin());

	// Relink the source state of the transition node
	Pins[0]->Modify();
	Pins[0]->BreakLinkTo(SourceStateBeforeRelinking->GetOutputPin());
	Pins[0]->MakeLinkTo(NewSourceState->GetOutputPin());
}

void UAnimStateTransitionNode::Reverse()
{
	UAnimStateNodeBase* OldSourceState = GetPreviousState();
	UAnimStateNodeBase* OldTargetState = GetNextState();
	UAnimStateNodeBase* NewTargetState = OldSourceState;
	UAnimStateNodeBase* NewSourceState = OldTargetState;

	// Remove the incoming transition from the previous target state
	OldSourceState->GetOutputPin()->Modify();
	OldSourceState->GetOutputPin()->BreakLinkTo(OldTargetState->GetInputPin());

	// Add the new incoming transition to the new target state
	NewSourceState->GetOutputPin()->Modify();
	NewSourceState->GetOutputPin()->MakeLinkTo(NewTargetState->GetInputPin());

	// Relink the target state of the transition node
	Pins[0]->Modify();
	Pins[0]->BreakLinkTo(OldSourceState->GetOutputPin());
	Pins[0]->MakeLinkTo(NewSourceState->GetOutputPin());

	Pins[1]->Modify();
	Pins[1]->BreakLinkTo(OldTargetState->GetInputPin());
	Pins[1]->MakeLinkTo(NewTargetState->GetInputPin());
}

TArray<UAnimStateTransitionNode*> UAnimStateTransitionNode::GetListTransitionNodesToRelink(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes)
{
	if (SourcePin->Direction == EGPD_Input && OldTargetPin->Direction == EGPD_Output)
	{
		Swap(SourcePin, OldTargetPin);
	}
	
	UAnimStateNodeBase* SourceState = Cast<UAnimStateNodeBase>(SourcePin->GetOwningNode());
	if (SourceState == nullptr || SourceState->GetInputPin() == nullptr || SourceState->GetOutputPin() == nullptr)
	{
		return {};
	}

	// Collect all transition nodes starting at the source state
	TArray<UAnimStateTransitionNode*> TransitionNodeCandidates;
	SourceState->GetTransitionList(TransitionNodeCandidates);

	// Remove the transition nodes from the candidates that are linked to a different target state.
	for (int i = TransitionNodeCandidates.Num() - 1; i >= 0; i--)
	{
		UAnimStateTransitionNode* CurrentTransition = TransitionNodeCandidates[i];

		// Get the actual target states from the transition nodes
		UEdGraphNode* TransitionTargetNode = CurrentTransition->GetNextState();
		UEdGraphNode* OldTargetNode = OldTargetPin->GetOwningNode();

		// Compare the target states rather than comparing against the transition nodes
		if (TransitionTargetNode != OldTargetNode)
		{
			TransitionNodeCandidates.Remove(CurrentTransition);
		}
	}

	// Collect the subset of selected transitions from the list of possible transitions to be relinked
	TSet<UAnimStateTransitionNode*> SelectedTransitionNodes;
	for (UEdGraphNode* GraphNode : InSelectedGraphNodes)
	{
		UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(GraphNode);
		if (!TransitionNode)
		{
			continue;
		}

		if (TransitionNodeCandidates.Find(TransitionNode) != INDEX_NONE)
		{
			SelectedTransitionNodes.Add(TransitionNode);
		}
	}

	TArray<UAnimStateTransitionNode*> Result;
	Result.Reserve(TransitionNodeCandidates.Num());
	for (UAnimStateTransitionNode* TransitionNode : TransitionNodeCandidates)
	{
		// Only relink the selected transitions. If none are selected, relink them all.
		if (!SelectedTransitionNodes.IsEmpty() && SelectedTransitionNodes.Find(TransitionNode) == nullptr)
		{
			continue;
		}

		Result.Add(TransitionNode);
	}

	return Result;
}

void UAnimStateTransitionNode::PrepareForCopying()
{
	Super::PrepareForCopying();
	// move bound graph node here, so during copying it will be referenced
	// for shared nodes at least one of them has to be referencing it, so we will be fine
	BoundGraph->Rename(NULL, this, REN_DoNotDirty | REN_DontCreateRedirectors);
}

void UAnimStateTransitionNode::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, CrossfadeDuration) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, BlendMode) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, CustomBlendCurve) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, BlendProfileWrapper))
	{
		PropagateCrossfadeSettings();
	}

	if (PropertyName == FName(TEXT("LogicType")) )
	{
		if ((LogicType == ETransitionLogicType::TLT_Custom) && (CustomTransitionGraph == NULL))
		{
			CreateCustomTransitionGraph();
		}
		else if (CustomTransitionGraph != NULL)
		{
			// UAnimationCustomTransitionSchema::HandleGraphBeingDeleted resets logic type, so we'll need to restore it after RemoveGraph
			const TEnumAsByte<ETransitionLogicType::Type> DesiredLogicType = LogicType;

			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
			FBlueprintEditorUtils::RemoveGraph(Blueprint, CustomTransitionGraph);
			CustomTransitionGraph = NULL;

			LogicType = DesiredLogicType;
		}	
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FString UAnimStateTransitionNode::GetStateName() const
{
	return (BoundGraph != NULL) ? *(BoundGraph->GetName()) : TEXT("(null)");
}

void UAnimStateTransitionNode::MakeRulesShareable(FString ShareName)
{
	bSharedRules = true;
	SharedRulesName = ShareName;
	SharedRulesGuid = FGuid::NewGuid();
}

void UAnimStateTransitionNode::MakeCrossfadeShareable(FString ShareName)
{
	// Give us a unique idx. This remaps every SharedCrossfadeIdx in the graph (in case some were deleted)
	UEdGraph* CurrentGraph = GetGraph();

	SharedCrossfadeIdx = INDEX_NONE;
	TArray<int32> Remap;
	for (int32 idx=0; idx < CurrentGraph->Nodes.Num(); idx++)
	{
		if (UAnimStateTransitionNode* Node = Cast<UAnimStateTransitionNode>(CurrentGraph->Nodes[idx]))
		{
			if (Node->SharedCrossfadeIdx != INDEX_NONE || Node == this)
			{
				Node->SharedCrossfadeIdx = Remap.AddUnique(Node->SharedCrossfadeIdx)+1; // Remaps existing index to lowest index available
			}
		}
	}

	bSharedCrossfade = true;
	SharedCrossfadeName = ShareName;
	SharedCrossfadeGuid = FGuid::NewGuid();
}

void UAnimStateTransitionNode::UnshareRules()
{
	bSharedRules = false;	
	SharedRulesName.Empty();
	SharedRulesGuid.Invalidate();

	if ((BoundGraph == NULL) || IsBoundGraphShared())
	{
		BoundGraph = NULL;
		CreateBoundGraph();
	}
}

void UAnimStateTransitionNode::UnshareCrossade()
{
	bSharedCrossfade = false;
	SharedCrossfadeIdx = INDEX_NONE;
	SharedCrossfadeName.Empty();
	SharedCrossfadeGuid.Invalidate();
}

void UAnimStateTransitionNode::UseSharedRules(const UAnimStateTransitionNode* Node)
{
	if(Node == this || Node == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("UseSharedRules", "Use Shared Rules"));

	Modify();

	UEdGraph* CurrentGraph = GetGraph();
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(CurrentGraph);

	UEdGraph* GraphToDelete = NULL;
	if ((BoundGraph != NULL) && !IsBoundGraphShared())
	{
		GraphToDelete = BoundGraph;
	}

	BoundGraph = Node->BoundGraph;
	bSharedRules = Node->bSharedRules;
	SharedRulesName = Node->SharedRulesName;
	SharedColor = Node->SharedColor;
	SharedRulesGuid = Node->SharedRulesGuid;

	if (GraphToDelete != NULL)
	{
		FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphToDelete);
	}

	// If this node has shared crossfade settings, and we currently dont... share with it automatically.
	// We'll see if this is actually helpful or just confusing. I think it might be a common operation
	// and this avoid having to manually select to share the rules and then share the crossfade settings.
	if ((SharedCrossfadeIdx == INDEX_NONE) && (Node->SharedCrossfadeIdx != INDEX_NONE))
	{
		UseSharedCrossfade(Node);
	}
}

void UAnimStateTransitionNode::UseSharedCrossfade(const UAnimStateTransitionNode* Node)
{
	if(Node == this || Node == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("UseSharedCrossfade", "Use Shared Crossfade"));

	Modify();

	bSharedCrossfade = Node->bSharedCrossfade;
	SharedCrossfadeName = Node->SharedCrossfadeName;
	SharedCrossfadeGuid = Node->SharedCrossfadeGuid;
	CopyCrossfadeSettings(Node);
}

void UAnimStateTransitionNode::CopyCrossfadeSettings(const UAnimStateTransitionNode* SrcNode)
{
	CrossfadeDuration = SrcNode->CrossfadeDuration;
	CrossfadeMode_DEPRECATED = SrcNode->CrossfadeMode_DEPRECATED;
	BlendMode = SrcNode->BlendMode;
	CustomBlendCurve = SrcNode->CustomBlendCurve;
	BlendProfileWrapper = SrcNode->BlendProfileWrapper;
	SharedCrossfadeIdx = SrcNode->SharedCrossfadeIdx;
	SharedCrossfadeName = SrcNode->SharedCrossfadeName;
	SharedCrossfadeGuid = SrcNode->SharedCrossfadeGuid;
}

void UAnimStateTransitionNode::PropagateCrossfadeSettings()
{
	UEdGraph* CurrentGraph = GetGraph();
	for (int32 idx = 0; idx < CurrentGraph->Nodes.Num(); idx++)
	{
		if (UAnimStateTransitionNode* Node = Cast<UAnimStateTransitionNode>(CurrentGraph->Nodes[idx]))
		{
			if (Node->SharedCrossfadeIdx != INDEX_NONE && Node->SharedCrossfadeGuid == SharedCrossfadeGuid)
			{
				Node->Modify();
				Node->CopyCrossfadeSettings(this);
			}
		}
	}
}

bool UAnimStateTransitionNode::IsReverseTrans(const UAnimStateNodeBase* Node)
{
	return (Bidirectional && GetNextState() == Node);
}

void UAnimStateTransitionNode::CreateBoundGraph()
{
	// Create a new animation graph
	check(BoundGraph == NULL);
	BoundGraph = FBlueprintEditorUtils::CreateNewGraph(this, NAME_None, UAnimationTransitionGraph::StaticClass(), UAnimationTransitionSchema::StaticClass());
	check(BoundGraph);

	// Find an interesting name
	FEdGraphUtilities::RenameGraphToNameOrCloseToName(BoundGraph, TEXT("Transition"));

	// Initialize the anim graph
	const UEdGraphSchema* Schema = BoundGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*BoundGraph);

	// Add the new graph as a child of our parent graph
	UEdGraph* ParentGraph = GetGraph();

	if(ParentGraph->SubGraphs.Find(BoundGraph) == INDEX_NONE)
	{
		ParentGraph->SubGraphs.Add(BoundGraph);
	}
}

void UAnimStateTransitionNode::CreateCustomTransitionGraph()
{
	// Create a new animation graph
	check(CustomTransitionGraph == NULL);
	CustomTransitionGraph = FBlueprintEditorUtils::CreateNewGraph(
		this,
		NAME_None,
		UAnimationCustomTransitionGraph::StaticClass(),
		UAnimationCustomTransitionSchema::StaticClass());
	check(CustomTransitionGraph);

	// Find an interesting name
	FEdGraphUtilities::RenameGraphToNameOrCloseToName(CustomTransitionGraph, TEXT("CustomTransition"));

	// Initialize the anim graph
	const UEdGraphSchema* Schema = CustomTransitionGraph->GetSchema();
	Schema->CreateDefaultNodesForGraph(*CustomTransitionGraph);

	// Add the new graph as a child of our parent graph
	UEdGraph* ParentGraph = GetGraph();

	if(ParentGraph->SubGraphs.Find(CustomTransitionGraph) == INDEX_NONE)
	{
		ParentGraph->Modify();
		ParentGraph->SubGraphs.Add(CustomTransitionGraph);
	}
}

void UAnimStateTransitionNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
}

void UAnimStateTransitionNode::DestroyNode()
{
	// BoundGraph may be shared with another graph, if so, don't remove it here
	UEdGraph* GraphToRemove = IsBoundGraphShared() ? NULL : GetBoundGraph();

	BoundGraph = NULL;
	Super::DestroyNode();

	if (GraphToRemove)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
		FBlueprintEditorUtils::RemoveGraph(Blueprint, GraphToRemove, EGraphRemoveFlags::Recompile);
	}

	if (CustomTransitionGraph)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
		FBlueprintEditorUtils::RemoveGraph(Blueprint, CustomTransitionGraph, EGraphRemoveFlags::Recompile);
	}	
}

/** Returns true if this nodes BoundGraph is shared with another node in the parent graph */
bool UAnimStateTransitionNode::IsBoundGraphShared() const
{
	if (BoundGraph)
	{
		if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(GetGraph()))
		{
			TArray<UAnimStateNodeBase*> StateNodes;
			FBlueprintEditorUtils::GetAllNodesOfClassEx<UAnimStateNodeBase>(Blueprint, StateNodes);

			for (int32 NodeIdx = 0; NodeIdx < StateNodes.Num(); NodeIdx++)
			{
				UAnimStateNodeBase* AnimNode = Cast<UAnimStateNodeBase>(StateNodes[NodeIdx]);
				if ((AnimNode != NULL) && (AnimNode != this) && (AnimNode->GetBoundGraph() == BoundGraph))
				{
					return true;
				}
			}
		}
	}

	return false;
}

void UAnimStateTransitionNode::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (UAnimationTransitionGraph* TransGraph = Cast<UAnimationTransitionGraph>(BoundGraph))
	{
		UAnimGraphNode_TransitionResult* ResultNode = TransGraph->GetResultNode();
		check(ResultNode);

		if (bAutomaticRuleBasedOnSequencePlayerInState)
		{
			// Check for automatic transition rules that are being triggered from looping asset players, as these can often be symptomatic of logic errors
			if (UAnimStateNodeBase* PreviousState = GetPreviousState())
			{
				// Deal with alias state nodes. Only single source aliases are valid
				if (UAnimStateAliasNode* AliasState = Cast<UAnimStateAliasNode>(PreviousState))
				{
					PreviousState = AliasState->GetAliasedState();
					if (!PreviousState)
					{
						MessageLog.Note(TEXT("Transition @@ is using an automatic transition rule but its source is an Alias node which aliases more than one State"), this);
						return;
					}
				}
				
				UAnimationStateGraph* PreviousStateGraph = Cast<UAnimationStateGraph>(PreviousState->GetBoundGraph());
				if (!PreviousStateGraph)
				{
					MessageLog.Note(TEXT("Transition @@ is using an automatic transition rule but source @@ is not a valid State. Check if the transit is connected to a Conduit"), this, PreviousState);
					return;
				}
				
				if (UAnimGraphNode_StateResult* PreviousStateGraphResultNode = PreviousStateGraph->GetResultNode())
				{
					for (UEdGraphPin* TestPin : PreviousStateGraphResultNode->Pins)
					{
						// Warn for the trivial but common case of a looping asset player connected directly to the result node
						if ((TestPin->Direction == EGPD_Input) && (TestPin->LinkedTo.Num() == 1))
						{
							if (UAnimGraphNode_SequencePlayer* SequencePlayer = Cast<UAnimGraphNode_SequencePlayer>(TestPin->LinkedTo[0]->GetOwningNode()))
							{
								if (SequencePlayer->Node.IsLooping())
								{
									MessageLog.Note(TEXT("Transition @@ is using an automatic transition rule but the source @@ is set as looping.  Please clear the 'Loop Animation' flag"), this, SequencePlayer);
								}
							}
							else if (UAnimGraphNode_BlendSpacePlayer* BlendSpacePlayer = Cast<UAnimGraphNode_BlendSpacePlayer>(TestPin->LinkedTo[0]->GetOwningNode()))
							{
								if (BlendSpacePlayer->Node.IsLooping())
								{
									MessageLog.Note(TEXT("Transition @@ is using an automatic transition rule but the source @@ is set as looping.  Please clear the 'Loop' flag"), this, BlendSpacePlayer);
								}
							}
						}
					}
				}
			}
		}
		else if (ResultNode->HasBinding(GET_MEMBER_NAME_CHECKED(FAnimNode_TransitionResult, bCanEnterTransition)))
		{
			// Rule is bound so nothing more to check
		}
		else if (ResultNode->Pins.Num() > 0)
		{
			UEdGraphPin* BoolResultPin = ResultNode->Pins[0];
			if (BoolResultPin && (BoolResultPin->LinkedTo.Num() == 0) && (BoolResultPin->DefaultValue.ToBool() == false))
			{
				// check for native transition rule before warning
				bool bHasNative = false;
				UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
				if(Blueprint && Blueprint->ParentClass)
				{
					UAnimInstance* AnimInstance = CastChecked<UAnimInstance>(Blueprint->ParentClass->GetDefaultObject());
					if(AnimInstance)
					{
						UEdGraph* ParentGraph = GetGraph();
						UAnimStateNodeBase* PrevState = GetPreviousState();
						UAnimStateNodeBase* NextState = GetNextState();
						if(PrevState != nullptr && NextState != nullptr && ParentGraph != nullptr)
						{
							FName FunctionName;
							bHasNative = AnimInstance->HasNativeTransitionBinding(ParentGraph->GetFName(), FName(*PrevState->GetStateName()), FName(*NextState->GetStateName()), FunctionName);
						}
					}
				}

				if (!bHasNative && !bAutomaticRuleBasedOnSequencePlayerInState)
				{
					MessageLog.Warning(TEXT("@@ will never be taken, please connect something to @@"), this, BoolResultPin);
				}
			}
		}
	}
	else
	{
		MessageLog.Error(TEXT("@@ contains an invalid or NULL BoundGraph.  Please delete and recreate the transition."), this);
	}
}

UObject* UAnimStateTransitionNode::GetJumpTargetForDoubleClick() const
{
	// Our base class uses GetSubGraphs. Since we explicitly ignore a shared bound graph, use BoundGraph directly instead.
	return BoundGraph;
}

TArray<UEdGraph*> UAnimStateTransitionNode::GetSubGraphs() const
{ 
	TArray<UEdGraph*> SubGraphs;
	if(!IsBoundGraphShared())
	{
		SubGraphs.Add(BoundGraph);
	}
	SubGraphs.Add(CustomTransitionGraph);
	return SubGraphs; 
}

void UAnimStateTransitionNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	// Add transition actions
	{
		FToolMenuSection& Section = Menu->AddSection("AnimTransitionNode", LOCTEXT("AnimTransitionNodeHeading", "Transition"));
		Section.AddMenuEntry(FAnimGraphCommands::Get().ReverseTransition);
	}
}

//////////////////////////////////////////////////////////////////////////
// IAnimStateTransitionNodeSharedDataHelper

void IAnimStateTransitionNodeSharedDataHelper::UpdateSharedData(UAnimStateTransitionNode* Node, TSharedPtr<INameValidatorInterface> NameValidator)
{
	// get all other transition nodes
	TArray<UAnimStateTransitionNode*> TransitionNodes;

	UEdGraph* ParentGraph = Node->GetGraph();
	ParentGraph->GetNodesOfClass(TransitionNodes);

	// check if there is other node that can provide us with data
	for (TArray<UAnimStateTransitionNode*>::TIterator It(TransitionNodes); It; ++ It)
	{
		UAnimStateTransitionNode* OtherNode = *It;
		if (OtherNode != Node &&
			CheckIfHasDataToShare(OtherNode) &&
			CheckIfNodesShouldShareData(Node, OtherNode))
		{
			// use shared data of that node (to make sure everything is linked up properly)
			ShareData(Node, OtherNode);
			break;
		}
	}

	// check if our shared rule name is original
	if (NameValidator->FindValidString(AccessShareDataName(Node)) != EValidatorResult::Ok)
	{
		// rename all shared rules name in nodes that should share same name
		for (TArray<UAnimStateTransitionNode*>::TIterator It(TransitionNodes); It; ++ It)
		{
			UAnimStateTransitionNode* OtherNode = *It;
			if (OtherNode != Node &&
				CheckIfNodesShouldShareData(Node, OtherNode))
			{
				AccessShareDataName(OtherNode) = AccessShareDataName(Node);
			}
		}
	}
}

void IAnimStateTransitionNodeSharedDataHelper::MakeSureGuidExists(UAnimStateTransitionNode* Node)
{
	UEdGraph* CurrentGraph = Node->GetGraph();
	for (int32 idx=0; idx < CurrentGraph->Nodes.Num(); idx++)
	{
		if (UAnimStateTransitionNode* OtherNode = Cast<UAnimStateTransitionNode>(CurrentGraph->Nodes[idx]))
		{
			if (OtherNode != Node &&
				CheckIfNodesShouldShareData(Node, OtherNode))
			{
				AccessShareDataName(Node) = AccessShareDataName(OtherNode);
			}
		}
	}
	
	if (! AccessShareDataGuid(Node).IsValid())
	{
		AccessShareDataGuid(Node) = FGuid::NewGuid();
	}
}

//////////////////////////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedRulesHelper

bool FAnimStateTransitionNodeSharedRulesHelper::CheckIfNodesShouldShareData(const UAnimStateTransitionNode* NodeA, const UAnimStateTransitionNode* NodeB)
{
	return NodeA->bSharedRules && NodeB->bSharedRules && NodeA->SharedRulesGuid == NodeB->SharedRulesGuid;
}

bool FAnimStateTransitionNodeSharedRulesHelper::CheckIfHasDataToShare(const UAnimStateTransitionNode* Node)
{
	return Node->BoundGraph != NULL;
}

void FAnimStateTransitionNodeSharedRulesHelper::ShareData(UAnimStateTransitionNode* NodeWhoWantsToShare, const UAnimStateTransitionNode* ShareFrom)
{
	NodeWhoWantsToShare->UseSharedRules(ShareFrom);
}

FString& FAnimStateTransitionNodeSharedRulesHelper::AccessShareDataName(UAnimStateTransitionNode* Node)
{
	return Node->SharedRulesName;
}

FGuid& FAnimStateTransitionNodeSharedRulesHelper::AccessShareDataGuid(UAnimStateTransitionNode* Node)
{
	return Node->SharedRulesGuid;
}

//////////////////////////////////////////////////////////////////////////
// FAnimStateTransitionNodeSharedCrossfadeHelper

bool FAnimStateTransitionNodeSharedCrossfadeHelper::CheckIfNodesShouldShareData(const UAnimStateTransitionNode* NodeA, const UAnimStateTransitionNode* NodeB)
{
	return NodeA->bSharedCrossfade && NodeB->bSharedCrossfade && NodeA->SharedCrossfadeGuid == NodeB->SharedCrossfadeGuid;
}

bool FAnimStateTransitionNodeSharedCrossfadeHelper::CheckIfHasDataToShare(const UAnimStateTransitionNode* Node)
{
	return Node->SharedCrossfadeIdx != INDEX_NONE;
}

void FAnimStateTransitionNodeSharedCrossfadeHelper::ShareData(UAnimStateTransitionNode* NodeWhoWantsToShare, const UAnimStateTransitionNode* ShareFrom)
{
	NodeWhoWantsToShare->UseSharedCrossfade(ShareFrom);
}

FString& FAnimStateTransitionNodeSharedCrossfadeHelper::AccessShareDataName(UAnimStateTransitionNode* Node)
{
	return Node->SharedCrossfadeName;
}

FGuid& FAnimStateTransitionNodeSharedCrossfadeHelper::AccessShareDataGuid(UAnimStateTransitionNode* Node)
{
	return Node->SharedCrossfadeGuid;
}

#undef LOCTEXT_NAMESPACE
