// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationConduitGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimGraphNode_TransitionResult.h"
#include "AnimationTransitionGraph.h"
#include "AnimStateConduitNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationConduitGraphSchema)

#define LOCTEXT_NAMESPACE "AnimationConduitSchema"

/////////////////////////////////////////////////////
// UAnimationConduitGraphSchema

UAnimationConduitGraphSchema::UAnimationConduitGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimationConduitGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	FGraphNodeCreator<UAnimGraphNode_TransitionResult> NodeCreator(Graph);
	UAnimGraphNode_TransitionResult* ResultSinkNode = NodeCreator.CreateNode();
	SetNodeMetaData(ResultSinkNode, FNodeMetadata::DefaultGraphNode);

	UAnimationTransitionGraph* TypedGraph = CastChecked<UAnimationTransitionGraph>(&Graph);
	TypedGraph->MyResultNode = ResultSinkNode;

	NodeCreator.Finalize();
}

void UAnimationConduitGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	DisplayInfo.PlainName = FText::FromString( Graph.GetName() );
	
	if (const UAnimStateConduitNode* ConduitNode = Cast<const UAnimStateConduitNode>(Graph.GetOuter()))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), ConduitNode->GetNodeTitle(ENodeTitleType::FullTitle) );

		DisplayInfo.PlainName = FText::Format( LOCTEXT("ConduitRuleGraphTitle", "{NodeTitle} (conduit rule)"), Args);
	}

	DisplayInfo.DisplayName = DisplayInfo.PlainName;
	DisplayInfo.Tooltip = LOCTEXT("GraphTooltip_ConduitSchema", "Conduits can be used to create 1-to-many, many-to-1, or many-to-many transitions");
}

void UAnimationConduitGraphSchema::HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const
{
	Super::HandleGraphBeingDeleted(GraphBeingRemoved);

	if(UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&GraphBeingRemoved))
	{
		// Handle composite anim graph nodes
		TArray<UAnimStateNodeBase*> StateNodes;
		FBlueprintEditorUtils::GetAllNodesOfClassEx<UAnimStateConduitNode>(Blueprint, StateNodes);

		TSet<UAnimStateNodeBase*> NodesToDelete;
		for(int32 i = 0; i < StateNodes.Num(); ++i)
		{
			UAnimStateNodeBase* StateNode = StateNodes[i];
			if(StateNode->GetBoundGraph() == &GraphBeingRemoved)
			{
				NodesToDelete.Add(StateNode);
			}
		}

		// Delete the node that owns us
		ensure(NodesToDelete.Num() <= 1);
		for(TSet<UAnimStateNodeBase*>::TIterator It(NodesToDelete); It; ++It)
		{
			UAnimStateNodeBase* NodeToDelete = *It;

			FBlueprintEditorUtils::RemoveNode(Blueprint, NodeToDelete, true);

			// Prevent re-entrancy here
			NodeToDelete->ClearBoundGraph();
		}
	}
}

#undef LOCTEXT_NAMESPACE
