// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationCustomTransitionGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationCustomTransitionGraph)

#define LOCTEXT_NAMESPACE "AnimationCustomTransitionGraph"

/////////////////////////////////////////////////////
// UAnimationStateGraph

UAnimationCustomTransitionGraph::UAnimationCustomTransitionGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAnimGraphNode_CustomTransitionResult* UAnimationCustomTransitionGraph::GetResultNode()
{
	return MyResultNode;
}

#undef LOCTEXT_NAMESPACE
