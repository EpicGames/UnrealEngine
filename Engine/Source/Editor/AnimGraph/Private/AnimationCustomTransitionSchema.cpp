// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationCustomTransitionSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimGraphNode_CustomTransitionResult.h"
#include "AnimGraphNode_TransitionPoseEvaluator.h"
#include "AnimStateTransitionNode.h"
#include "AnimationCustomTransitionGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationCustomTransitionSchema)

#define LOCTEXT_NAMESPACE "AnimationCustomTransitionSchema"

/////////////////////////////////////////////////////
// UAnimationCustomTransitionSchema

UAnimationCustomTransitionSchema::UAnimationCustomTransitionSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimationCustomTransitionSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	// Create the result node
	FGraphNodeCreator<UAnimGraphNode_CustomTransitionResult> ResultNodeCreator(Graph);
	UAnimGraphNode_CustomTransitionResult* ResultSinkNode = ResultNodeCreator.CreateNode();
	ResultSinkNode->NodePosX = 0;
	ResultSinkNode->NodePosY = 0;
	ResultNodeCreator.Finalize();
	SetNodeMetaData(ResultSinkNode, FNodeMetadata::DefaultGraphNode);

	UAnimationCustomTransitionGraph* TypedGraph = CastChecked<UAnimationCustomTransitionGraph>(&Graph);
	TypedGraph->MyResultNode = ResultSinkNode;

	// Create the source and destination input states
	{
		FGraphNodeCreator<UAnimGraphNode_TransitionPoseEvaluator> SourceNodeCreator(Graph);
		UAnimGraphNode_TransitionPoseEvaluator* SourcePoseNode = SourceNodeCreator.CreateNode();
		SourcePoseNode->Node.DataSource = EEvaluatorDataSource::EDS_SourcePose;
		SourcePoseNode->NodePosX = -300;
		SourcePoseNode->NodePosY = -150;
		SourceNodeCreator.Finalize();
		SetNodeMetaData(SourcePoseNode, FNodeMetadata::DefaultGraphNode);

	}
	{
		FGraphNodeCreator<UAnimGraphNode_TransitionPoseEvaluator> DestinationNodeCreator(Graph);
		UAnimGraphNode_TransitionPoseEvaluator* DestinationPoseNode = DestinationNodeCreator.CreateNode();
		DestinationPoseNode->Node.DataSource = EEvaluatorDataSource::EDS_DestinationPose;
		DestinationPoseNode->NodePosX = -300;
		DestinationPoseNode->NodePosY = 150;
		DestinationNodeCreator.Finalize();
		SetNodeMetaData(DestinationPoseNode, FNodeMetadata::DefaultGraphNode);
	}
}

void UAnimationCustomTransitionSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	DisplayInfo.PlainName = FText::FromString( Graph.GetName() );

	if (const UAnimStateTransitionNode* TransNode = Cast<const UAnimStateTransitionNode>(Graph.GetOuter()))
	{
		DisplayInfo.PlainName = FText::Format( LOCTEXT("CustomBlendGraphTitle", "{0} (custom blend)"), TransNode->GetNodeTitle(ENodeTitleType::FullTitle) );
	}

	DisplayInfo.DisplayName = DisplayInfo.PlainName;
	DisplayInfo.Tooltip = LOCTEXT("GraphTooltip_CustomTransitionSchema", "Custom blend graphs offer manual blending from source to destination states");
}

void UAnimationCustomTransitionSchema::HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const
{
	if(UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&GraphBeingRemoved))
	{
		// Look for state nodes that reference this graph
		TArray<UAnimStateTransitionNode*> TransitionNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass<UAnimStateTransitionNode>(Blueprint, TransitionNodes);

		for(UAnimStateTransitionNode* Node : TransitionNodes)
		{
			if(Node->CustomTransitionGraph == &GraphBeingRemoved)
			{
				// Clear out the logic for this node as we're removing the graph
				Node->Modify();
				Node->LogicType = ETransitionLogicType::TLT_StandardBlend;
				Node->CustomTransitionGraph = nullptr;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
