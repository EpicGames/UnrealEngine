// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationGraph.h"

#include "AnimGraphNode_LinkedInputPose.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationGraph)

#define LOCTEXT_NAMESPACE "AnimationGraph"

/////////////////////////////////////////////////////
// UAnimationGraph

UAnimationGraph::UAnimationGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimationGraph::GetGraphNodesOfClass(TSubclassOf<UAnimGraphNode_Base> NodeClass, TArray<UAnimGraphNode_Base*>& GraphNodes, bool bIncludeChildClasses /*= true*/)
{
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
	{
		UEdGraphNode* Node = Nodes[NodeIndex];
		if (UAnimGraphNode_Base* TypedNode = Cast<UAnimGraphNode_Base>(Node))
		{
			UClass* TypedNodeClass = TypedNode->GetClass();
			if (TypedNodeClass == NodeClass || (bIncludeChildClasses && TypedNode->GetClass()->IsChildOf(NodeClass)))
			{				
				GraphNodes.Add(TypedNode);
			}
		}
	}
}

void UAnimationGraph::PostEditUndo()
{
	Super::PostEditUndo();

	// We may have added/removed input nodes, which requires a refresh of any node that use them
	// We need to defer this a tick to make sure we reconstruct nodes post-compile
	GEditor->GetTimerManager()->SetTimerForNextTick(this, &UAnimationGraph::ReconstructLayerNodes);
}

void UAnimationGraph::ReconstructLayerNodes() const
{
	UAnimGraphNode_LinkedInputPose::ReconstructLayerNodes(GetTypedOuter<UBlueprint>());
}

#undef LOCTEXT_NAMESPACE
