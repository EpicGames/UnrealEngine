// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationGraphSchema.cpp
=============================================================================*/

#include "AnimationGraphSchema.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimBlueprint.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "ObjectEditorUtils.h"
#include "K2Node.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Editor.h"
#include "Editor/AssetReferenceFilter.h"
#include "Editor/EditorEngine.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/AnimSequence.h"
#include "AnimStateNode.h"
#include "Animation/BlendSpace.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "Animation/AnimNodeBase.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimGraphNode_BlendSpacePlayer.h"
#include "AnimGraphNode_ComponentToLocalSpace.h"
#include "AnimGraphNode_LocalToComponentSpace.h"
#include "AnimGraphNode_Root.h"
#include "AnimGraphNode_RotationOffsetBlendSpace.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "Animation/PoseAsset.h"
#include "AnimGraphNode_PoseBlendNode.h"
#include "AnimGraphNode_PoseByName.h"
#include "AnimGraphCommands.h"
#include "K2Node_Knot.h"
#include "ScopedTransaction.h"
#include "Animation/AnimMontage.h"
#include "AnimGraphNode_LinkedInputPose.h"
#include "AnimGraphNode_LinkedAnimLayer.h"
#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "AnimGraphNode_RigidBody.h"
#include "AnimationBlendSpaceSampleGraph.h"
#include "GraphEditorDragDropAction.h"
#include "AnimationEditorUtils.h"
#include "Settings/AnimBlueprintSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationGraphSchema)

#define LOCTEXT_NAMESPACE "AnimationGraphSchema"

namespace UE::Anim::BP::Editor
{
	bool IsSkeletonCompatible(const UAnimBlueprint* AnimBlueprint, const UAnimationAsset* Asset)
	{
		if (AnimBlueprint == nullptr)
		{
			return false;	// No blueprint provided, cannot be compatible
		}

		if (AnimBlueprint->bIsTemplate)
		{
			return true;	// Templates are always compatible
		}

		if (AnimBlueprint->TargetSkeleton == nullptr)
		{
			return false;	// No target skeleton provided, cannot be compatible
		}

		return AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(Asset->GetSkeleton());
	}
}

/////////////////////////////////////////////////////
// FAnimationLayerDragDropAction
/** DragDropAction class for drag and dropping animation layers */
class ANIMGRAPH_API FAnimationLayerDragDropAction : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAnimationLayerDragDropAction, FGraphSchemaActionDragDropAction)

	virtual FReply DroppedOnPanel(const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	virtual FReply DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action) override;
	virtual FReply DroppedOnCategory(FText Category) override;
	virtual void HoverTargetChanged() override;

protected:

	/** Constructor */
	FAnimationLayerDragDropAction();

	static TSharedRef<FAnimationLayerDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InAction, FName InFuncName, UAnimBlueprint* InRigBlueprint, UAnimationGraph* InRigGraph);

	UAnimBlueprint* SourceAnimBlueprint;
	UAnimationGraph* SourceAnimLayerGraph;
	FName SourceFuncName;

	friend class UAnimationGraphSchema;
};


FAnimationLayerDragDropAction::FAnimationLayerDragDropAction()
    : FGraphSchemaActionDragDropAction()
    , SourceAnimBlueprint(nullptr)
    , SourceAnimLayerGraph(nullptr)
    , SourceFuncName(NAME_None)
{
}

FReply FAnimationLayerDragDropAction::DroppedOnPanel(const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if (UAnimationGraph* TargetRigGraph = Cast<UAnimationGraph>(&Graph))
	{
		if (UAnimBlueprint* TargetAnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(TargetRigGraph)))
		{
			FGraphNodeCreator<UAnimGraphNode_LinkedAnimLayer> LinkedInputLayerNodeCreator(*TargetRigGraph);
			UAnimGraphNode_LinkedAnimLayer* LinkedAnimLayerNode = LinkedInputLayerNodeCreator.CreateNode();	
			const FName GraphName = TargetRigGraph->GetFName();
			LinkedAnimLayerNode->SetupFromLayerId(SourceFuncName);
			LinkedInputLayerNodeCreator.Finalize();
			LinkedAnimLayerNode->NodePosX = static_cast<int32>(GraphPosition.X);
			LinkedAnimLayerNode->NodePosY = static_cast<int32>(GraphPosition.Y);

			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(&Graph);
			// See if we need to recompile skeleton after adding this node, or just mark dirty
			if (LinkedAnimLayerNode->NodeCausesStructuralBlueprintChange())
			{
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
			}
			else
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}
		}
	}
	return FReply::Unhandled();
}

FReply FAnimationLayerDragDropAction::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if (UEdGraphNode* TargetNode = GetHoveredNode())
	{
		if (UAnimGraphNode_LinkedAnimLayer* LinkedAnimLayer = Cast<UAnimGraphNode_LinkedAnimLayer>(TargetNode))
		{
			LinkedAnimLayer->Node.Layer = SourceFuncName; 
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

FReply FAnimationLayerDragDropAction::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	return FReply::Unhandled();
}

FReply FAnimationLayerDragDropAction::DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action)
{
	return FReply::Unhandled();
}

FReply FAnimationLayerDragDropAction::DroppedOnCategory(FText Category)
{
	return FReply::Unhandled();
}

void FAnimationLayerDragDropAction::HoverTargetChanged()
{
	FGraphSchemaActionDragDropAction::HoverTargetChanged();
	bDropTargetValid = true;
}


TSharedRef<FAnimationLayerDragDropAction> FAnimationLayerDragDropAction::New(TSharedPtr<FEdGraphSchemaAction> InAction, FName InFuncName, UAnimBlueprint* InAnimBlueprint, UAnimationGraph* InAnimationLayerGraph)
{
	TSharedRef<FAnimationLayerDragDropAction> Action = MakeShareable(new FAnimationLayerDragDropAction);
	Action->SourceAction = InAction;
	Action->SourceAnimBlueprint = InAnimBlueprint;
	Action->SourceAnimLayerGraph = InAnimationLayerGraph;
	Action->SourceFuncName = InFuncName; 
	Action->Construct();
	return Action;
}

/////////////////////////////////////////////////////
// UAnimationGraphSchema

UAnimationGraphSchema::UAnimationGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PN_SequenceName = TEXT("Sequence");

	NAME_NeverAsPin = TEXT("NeverAsPin");
	NAME_PinHiddenByDefault = TEXT("PinHiddenByDefault");
	NAME_PinShownByDefault = TEXT("PinShownByDefault");
	NAME_AlwaysAsPin = TEXT("AlwaysAsPin");
	NAME_OnEvaluate = TEXT("OnEvaluate");
	NAME_CustomizeProperty = TEXT("CustomizeProperty");
	DefaultEvaluationHandlerName = TEXT("EvaluateGraphExposedInputs");
}

FLinearColor UAnimationGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	const bool bAdditive = PinType.PinSubCategory == TEXT("Additive");
	if (UAnimationGraphSchema::IsLocalSpacePosePin(PinType))
	{
		if (bAdditive) 
		{
			return FLinearColor(0.12f, 0.60f, 0.10f);
		}
		else
		{
			return FLinearColor::White;
		}
	}
	else if (UAnimationGraphSchema::IsComponentSpacePosePin(PinType))
	{
		//@TODO: Pick better colors
		if (bAdditive) 
		{
			return FLinearColor(0.12f, 0.60f, 0.60f);
		}
		else
		{
			return FLinearColor(0.20f, 0.50f, 1.00f);
		}
	}

	return Super::GetPinTypeColor(PinType);
}

EGraphType UAnimationGraphSchema::GetGraphType(const UEdGraph* TestEdGraph) const
{
	return GT_Animation;
}

void UAnimationGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	// Create the result node
	FGraphNodeCreator<UAnimGraphNode_Root> NodeCreator(Graph);
	UAnimGraphNode_Root* ResultSinkNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();
	SetNodeMetaData(ResultSinkNode, FNodeMetadata::DefaultGraphNode);
}

void UAnimationGraphSchema::HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const
{
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&GraphBeingRemoved))
	{
		// Look for state nodes that reference this graph
		TArray<UAnimStateNodeBase*> StateNodes;
		FBlueprintEditorUtils::GetAllNodesOfClassEx<UAnimStateNode>(Blueprint, StateNodes);

		TSet<UAnimStateNodeBase*> NodesToDelete;
		for (int32 i = 0; i < StateNodes.Num(); ++i)
		{
			UAnimStateNodeBase* StateNode = StateNodes[i];
			if (StateNode->GetBoundGraph() == &GraphBeingRemoved)
			{
				NodesToDelete.Add(StateNode);
			}
		}

		// Delete the node that owns us
		ensure(NodesToDelete.Num() <= 1);
		for (TSet<UAnimStateNodeBase*>::TIterator It(NodesToDelete); It; ++It)
		{
			UAnimStateNodeBase* NodeToDelete = *It;

			FBlueprintEditorUtils::RemoveNode(Blueprint, NodeToDelete, true);

			// Prevent re-entrancy here
			NodeToDelete->ClearBoundGraph();
		}

		// Remove pose watches from nodes in this graph
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
		{
			AnimationEditorUtils::RemovePoseWatchesFromGraph(AnimBlueprint, &GraphBeingRemoved);
		}
	}
}

bool UAnimationGraphSchema::IsPosePin(const FEdGraphPinType& PinType)
{
	return IsLocalSpacePosePin(PinType) || IsComponentSpacePosePin(PinType);
}

bool UAnimationGraphSchema::IsLocalSpacePosePin(const FEdGraphPinType& PinType)
{
	UScriptStruct* PoseLinkStruct = FPoseLink::StaticStruct();
	return (PinType.PinCategory == UAnimationGraphSchema::PC_Struct) && (PinType.PinSubCategoryObject == PoseLinkStruct);
}

bool UAnimationGraphSchema::IsComponentSpacePosePin(const FEdGraphPinType& PinType)
{
	UScriptStruct* ComponentSpacePoseLinkStruct = FComponentSpacePoseLink::StaticStruct();
	return (PinType.PinCategory == UAnimationGraphSchema::PC_Struct) && (PinType.PinSubCategoryObject == ComponentSpacePoseLinkStruct);
}

FEdGraphPinType UAnimationGraphSchema::MakeLocalSpacePosePin()
{
	FEdGraphPinType PinType;
	PinType.ResetToDefaults();

	PinType.PinCategory = UAnimationGraphSchema::PC_Struct;
	PinType.PinSubCategoryObject = FPoseLink::StaticStruct();

	return PinType;
}

FEdGraphPinType UAnimationGraphSchema::MakeComponentSpacePosePin()
{
	FEdGraphPinType PinType;
	PinType.ResetToDefaults();

	PinType.PinCategory = UAnimationGraphSchema::PC_Struct;
	PinType.PinSubCategoryObject = FComponentSpacePoseLink::StaticStruct();

	return PinType;
}

bool UAnimationGraphSchema::TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	UEdGraphPin* OutputPin = nullptr;
	UEdGraphPin* InputPin = nullptr;

	if(A->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		OutputPin = A;
		InputPin = B;
	}
	else
	{
		OutputPin = B;
		InputPin = A;
	}
	check(OutputPin && InputPin);

	UK2Node_Knot* OutputKnotNode = Cast<UK2Node_Knot>(OutputPin->GetOwningNode());
	UK2Node_Knot* InputKnotNode = Cast<UK2Node_Knot>(InputPin->GetOwningNode());
	bool bConnectionWithKnot = OutputKnotNode != nullptr || InputKnotNode != nullptr;

	if(bConnectionWithKnot)
	{
		// Double check this is our "exec"-like line
		bool bOutputIsPose = IsPosePin(OutputPin->PinType);
		bool bInputIsPose = IsPosePin(InputPin->PinType);
		bool bHavePosePin = bOutputIsPose || bInputIsPose;
		bool bHaveWildPin = InputPin->PinType.PinCategory == PC_Wildcard || OutputPin->PinType.PinCategory == PC_Wildcard;

		if((bOutputIsPose && bInputIsPose) || (bHavePosePin && bHaveWildPin))
		{
			// Ok this is a valid exec-like line, we need to kill any connections already on the output pin
			OutputPin->BreakAllPinLinks();
		}
	}

	if(Super::TryCreateConnection(A, B))
	{
		// Connection made - remove any bindings on the input pin
		if(UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(InputPin->GetOwningNode()))
		{
			// Compare FName without number to make sure we catch array properties that are split into multiple pins
			FName ComparisonName = InputPin->GetFName();
			ComparisonName.SetNumber(0);
			AnimGraphNode->RemoveBindings(ComparisonName);
		}

		return true;
	}

	return false;
}

const FPinConnectionResponse UAnimationGraphSchema::DetermineConnectionResponseOfCompatibleTypedPins(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	// Enforce a tree hierarchy; where poses can only have one output (parent) connection
	if (IsPosePin(OutputPin->PinType) && IsPosePin(InputPin->PinType))
	{
		if ((OutputPin->LinkedTo.Num() > 0) || (InputPin->LinkedTo.Num() > 0))
		{
			const ECanCreateConnectionResponse ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_AB;
			return FPinConnectionResponse(ReplyBreakOutputs, TEXT("Replace existing connections"));
		}
	}

	// Fall back to standard K2 rules
	return Super::DetermineConnectionResponseOfCompatibleTypedPins(PinA, PinB, InputPin, OutputPin);
}

bool UAnimationGraphSchema::ArePinsCompatible(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UClass* CallingContext, bool bIgnoreArray) const
{
	// both are pose pin, but doesn't match type, then return false;
	if (IsPosePin(PinA->PinType) && IsPosePin(PinB->PinType) && IsLocalSpacePosePin(PinA->PinType) != IsLocalSpacePosePin(PinB->PinType))
	{
		return false;
	}
	
	// Disallow pose pins connecting to wildcards (apart from reroute nodes)
	if(IsPosePin(PinA->PinType) && PinB->PinType.PinCategory == PC_Wildcard)
	{
		return Cast<UK2Node_Knot>(PinB->GetOwningNode()) != nullptr;
	}
	else if(IsPosePin(PinB->PinType) && PinA->PinType.PinCategory == PC_Wildcard)
	{
		return Cast<UK2Node_Knot>(PinA->GetOwningNode()) != nullptr;
	}

	return Super::ArePinsCompatible(PinA, PinB, CallingContext, bIgnoreArray);
}

bool UAnimationGraphSchema::DoesSupportAnimNotifyActions() const
{
	// Don't offer notify items in anim graph
	return false;
}

void UAnimationGraphSchema::CreateFunctionGraphTerminators(UEdGraph& Graph, UClass* Class) const
{
	if(const UAnimationGraphSchema* Schema = ExactCast<UAnimationGraphSchema>(Graph.GetSchema()))
	{
		const FName GraphName = Graph.GetFName();

		// Get the function GUID from the most up-to-date class
		FGuid GraphGuid;
		FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(Class), GraphName, GraphGuid);

		// Create a root node
		FGraphNodeCreator<UAnimGraphNode_Root> RootNodeCreator(Graph);
		UAnimGraphNode_Root* RootNode = RootNodeCreator.CreateNode();
		RootNodeCreator.Finalize();
		SetNodeMetaData(RootNode, FNodeMetadata::DefaultGraphNode);

		UFunction* InterfaceToImplement = FindUField<UFunction>(Class, GraphName);
		if (InterfaceToImplement)
		{
			// Propagate group from metadata
			TArray<UAnimGraphNode_Root*> RootNodes;
			Graph.GetNodesOfClass<UAnimGraphNode_Root>(RootNodes);

			check(RootNodes.Num() == 1);
			RootNodes[0]->Node.SetGroup(*FObjectEditorUtils::GetCategoryText(InterfaceToImplement).ToString());

			int32 CurrentPoseIndex = 0;
			for (TFieldIterator<FProperty> PropIt(InterfaceToImplement); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
			{
				FProperty* Param = *PropIt;

				const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);

				if (bIsFunctionInput)
				{
					FEdGraphPinType PinType;
					if(Schema->ConvertPropertyToPinType(Param, PinType))
					{
						// Create linked input pose for each pose pin type
						if(UAnimationGraphSchema::IsPosePin(PinType))
						{
							FGraphNodeCreator<UAnimGraphNode_LinkedInputPose> LinkedInputNodeCreator(Graph);
							UAnimGraphNode_LinkedInputPose* LinkedInputNode = LinkedInputNodeCreator.CreateNode();
							LinkedInputNode->FunctionReference.SetExternalMember(GraphName, Class, GraphGuid);
							LinkedInputNode->Node.Name = Param->GetFName();
							LinkedInputNode->InputPoseIndex = CurrentPoseIndex;

							LinkedInputNode->ReconstructNode();
							SetNodeMetaData(LinkedInputNode, FNodeMetadata::DefaultGraphNode);
							LinkedInputNodeCreator.Finalize();

							CurrentPoseIndex++;
						}
					}
				}
			}
		}

		AutoArrangeInterfaceGraph(Graph);
	}
	else
	{
		Super::CreateFunctionGraphTerminators(Graph, Class);
	}
}


bool UAnimationGraphSchema::CanShowDataTooltipForPin(const UEdGraphPin& Pin) const
{
	return !IsPosePin(Pin.PinType) && UEdGraphSchema_K2::CanShowDataTooltipForPin(Pin);
}

bool UAnimationGraphSchema::CanGraphBeDropped(TSharedPtr<FEdGraphSchemaAction> InAction) const
{
	if (!InAction.IsValid())
	{
		return false;
	}

	if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Graph* FuncAction = (FEdGraphSchemaAction_K2Graph*)InAction.Get();
		if (UAnimationGraph* AnimGraph = Cast<UAnimationGraph>((UEdGraph*)FuncAction->EdGraph))
		{
			return true;
		}
	}

	return false;
}

FReply UAnimationGraphSchema::BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction, const FPointerEvent& MouseEvent) const
{
	if (!InAction.IsValid())
	{
		return FReply::Unhandled();
	}

	if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Graph* FuncAction = (FEdGraphSchemaAction_K2Graph*)InAction.Get();
		if (UAnimationGraph* AnimationLayerGraph = Cast<UAnimationGraph>((UEdGraph*)FuncAction->EdGraph))
		{
			if (UAnimBlueprint* TargetAnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(AnimationLayerGraph)))
			{
				return FReply::Handled().BeginDragDrop(FAnimationLayerDragDropAction::New(InAction, FuncAction->FuncName, TargetAnimBlueprint, AnimationLayerGraph));
			}
		}
	}
	
	return FReply::Unhandled();
}

TOptional<UEdGraphSchema_K2::FSearchForAutocastFunctionResults> UAnimationGraphSchema::SearchForAutocastFunction(const FEdGraphPinType& OutputPinType, const FEdGraphPinType& InputPinType) const
{
	TOptional<UEdGraphSchema_K2::FSearchForAutocastFunctionResults> Result;

	if (IsComponentSpacePosePin(OutputPinType) && IsLocalSpacePosePin(InputPinType))
	{
		// Insert a Component To LocalSpace conversion
		Result = UEdGraphSchema_K2::FSearchForAutocastFunctionResults{};
	}
	else if (IsLocalSpacePosePin(OutputPinType) && IsComponentSpacePosePin(InputPinType))
	{
		// Insert a Local To ComponentSpace conversion
		Result = UEdGraphSchema_K2::FSearchForAutocastFunctionResults{};
	}
	else
	{
		Result = Super::SearchForAutocastFunction(OutputPinType, InputPinType);
	}

	return Result;
}

bool UAnimationGraphSchema::CreateAutomaticConversionNodeAndConnections(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	// Determine which pin is an input and which pin is an output
	UEdGraphPin* InputPin = NULL;
	UEdGraphPin* OutputPin = NULL;
	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return false;
	}

	// Look for animation specific conversion operations
	UK2Node* TemplateNode = NULL;
	if (IsComponentSpacePosePin(OutputPin->PinType) && IsLocalSpacePosePin(InputPin->PinType))
	{
		TemplateNode = NewObject<UAnimGraphNode_ComponentToLocalSpace>();
	}
	else if (IsLocalSpacePosePin(OutputPin->PinType) && IsComponentSpacePosePin(InputPin->PinType))
	{
		TemplateNode = NewObject<UAnimGraphNode_LocalToComponentSpace>();
	}

	// Spawn the conversion node if it's specific to animation
	if (TemplateNode != NULL)
	{
		UEdGraph* Graph = InputPin->GetOwningNode()->GetGraph();
		FVector2D AverageLocation = CalculateAveragePositionBetweenNodes(InputPin, OutputPin);

		UK2Node* ConversionNode = FEdGraphSchemaAction_K2NewNode::SpawnNodeFromTemplate<UK2Node>(Graph, TemplateNode, AverageLocation);
		AutowireConversionNode(InputPin, OutputPin, ConversionNode);

		return true;
	}
	else
	{
		// Give the regular conversions a shot
		return Super::CreateAutomaticConversionNodeAndConnections(PinA, PinB);
	}
}

bool IsAimOffsetBlendSpace(UBlendSpace* BlendSpace)
{
	return	BlendSpace->IsA(UAimOffsetBlendSpace::StaticClass()) ||
			BlendSpace->IsA(UAimOffsetBlendSpace1D::StaticClass());
}

void UAnimationGraphSchema::SpawnNodeFromAsset(UAnimationAsset* Asset, const UE::Slate::FDeprecateVector2DParameter& GraphPosition, UEdGraph* Graph, UEdGraphPin* PinIfAvailable)
{
	check(Graph);
	check(Graph->GetSchema()->IsA(UAnimationGraphSchema::StaticClass()));
	check(Asset);

	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(Graph));

	const bool bSkelMatch = UE::Anim::BP::Editor::IsSkeletonCompatible(AnimBlueprint, Asset);
	const bool bTypeMatch = (PinIfAvailable == nullptr) || UAnimationGraphSchema::IsLocalSpacePosePin(PinIfAvailable->PinType);
	const bool bDirectionMatch = (PinIfAvailable == nullptr) || (PinIfAvailable->Direction == EGPD_Input);

	if (bSkelMatch && bTypeMatch && bDirectionMatch)
	{
		FEdGraphSchemaAction_K2NewNode Action;

		UClass* NewNodeClass = GetNodeClassForAsset(Asset->GetClass());
		
		if (NewNodeClass)
		{
			check(NewNodeClass->IsChildOf(UAnimGraphNode_AssetPlayerBase::StaticClass()));

			UAnimGraphNode_AssetPlayerBase* NewNode = NewObject<UAnimGraphNode_AssetPlayerBase>(GetTransientPackage(), NewNodeClass);
			NewNode->SetAnimationAsset(Asset);
			NewNode->CopySettingsFromAnimationAsset(Asset);
			Action.NodeTemplate = NewNode;

			Action.PerformAction(Graph, PinIfAvailable, GraphPosition);
		}
	}
}

void UAnimationGraphSchema::SpawnRigidBodyNodeFromAsset(UPhysicsAsset* Asset, const UE::Slate::FDeprecateVector2DParameter& GraphPosition, UEdGraph* Graph)
{
	check(Graph);
	check(Graph->GetSchema()->IsA(UAnimationGraphSchema::StaticClass()));
	check(Asset);

	FEdGraphSchemaAction_K2NewNode Action;

	UAnimGraphNode_RigidBody* NewNode = NewObject<UAnimGraphNode_RigidBody>(GetTransientPackage());
	NewNode->Node.OverridePhysicsAsset = Asset;
	Action.NodeTemplate = NewNode;

	Action.PerformAction(Graph, nullptr, GraphPosition);
}

void UAnimationGraphSchema::UpdateNodeWithAsset(UK2Node* K2Node, UAnimationAsset* Asset)
{
	if (Asset != NULL)
	{
		if (UAnimGraphNode_AssetPlayerBase* AssetPlayerNode = Cast<UAnimGraphNode_AssetPlayerBase>(K2Node))
		{
			if (AssetPlayerNode->SupportsAssetClass(Asset->GetClass()) != EAnimAssetHandlerType::NotSupported)
			{
				const FScopedTransaction Transaction(LOCTEXT("UpdateNodeWithAsset", "Updating Node with Asset"));
				AssetPlayerNode->Modify();

				AssetPlayerNode->SetAnimationAsset(Asset);

				K2Node->GetSchema()->ForceVisualizationCacheClear();
				K2Node->ReconstructNode();
			}
		}
	}
}

void UAnimationGraphSchema::DroppedAssetsOnGraph( const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph ) const 
{
	if (Graph != nullptr)
	{
		if (UAnimationAsset* AnimationAsset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets))
		{
			const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);
			if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(FAssetData(AnimationAsset)))
			{
				SpawnNodeFromAsset(AnimationAsset, GraphPosition, Graph, nullptr);
			}
		}
		else if (UPhysicsAsset* PhysicsAsset = FAssetData::GetFirstAsset<UPhysicsAsset>(Assets))
		{
			const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);
			if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(FAssetData(PhysicsAsset)))
			{
				SpawnRigidBodyNodeFromAsset(PhysicsAsset, FDeprecateSlateVector2D(GraphPosition), Graph);
			}
		}
	}
}

void UAnimationGraphSchema::DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	UK2Node* K2Node = Cast<UK2Node>(Node);
	if ((Asset != nullptr) && (K2Node!= nullptr))
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(K2Node->GetGraph());
		if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(FAssetData(Asset)))
		{
			UpdateNodeWithAsset(K2Node, Asset);
		}
	}
}

void UAnimationGraphSchema::DroppedAssetsOnPin(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphPin* Pin) const
{
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	if ((Asset != nullptr) && (Pin != nullptr))
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Pin->GetOwningNode()->GetGraph());
		if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(FAssetData(Asset)))
		{
			// Don't have access to bounding information for node, using fixed offset that should work for most cases.
			const FVector2f FixedOffset(-250.0f, 50.0f);
			SpawnNodeFromAsset(Asset, GraphPosition + FixedOffset, Pin->GetOwningNode()->GetGraph(), Pin);
		}
	}
}

void UAnimationGraphSchema::GetAssetsNodeHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const
{ 
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	if ((Asset == nullptr) || (HoverNode == nullptr) || !HoverNode->IsA(UAnimGraphNode_Base::StaticClass()))
	{
		OutTooltipText.Reset();
		OutOkIcon = false;
		return;
	}

	bool bCanPlayAsset = SupportNodeClassForAsset(Asset->GetClass(), HoverNode->GetClass());

	// this one only should happen when there is an Anim Blueprint
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(HoverNode));
	const bool bSkelMatch = UE::Anim::BP::Editor::IsSkeletonCompatible(AnimBlueprint, Asset);

	if (!bSkelMatch)
	{
		OutOkIcon = false;
		OutTooltipText = LOCTEXT("SkeletonsNotCompatible", "Skeletons are not compatible").ToString();
	}
	else if (bCanPlayAsset)
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverNode->GetGraph());
		FText FailureReason;
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(FAssetData(Asset), &FailureReason))
		{
			OutOkIcon = false;
			OutTooltipText = FailureReason.ToString();
		}
		else
		{
			OutOkIcon = true;
			OutTooltipText = FText::Format(LOCTEXT("AssetNodeHoverMessage_Success", "Change node to play '{0}'"), FText::FromString(Asset->GetName())).ToString();
		}
	}
	else
	{
		OutOkIcon = false;
		OutTooltipText = FText::Format(LOCTEXT("AssetNodeHoverMessage_Fail", "Cannot play '{0}' on this node type"),  FText::FromString(Asset->GetName())).ToString();
	}
}

void UAnimationGraphSchema::GetAssetsPinHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphPin* HoverPin, FString& OutTooltipText, bool& OutOkIcon) const 
{ 
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	if ((Asset == nullptr) || (HoverPin == nullptr))
	{
		OutTooltipText.Reset();
		OutOkIcon = false;
		return;
	}

	// this one only should happen when there is an Anim Blueprint
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(HoverPin->GetOwningNode()));

	const bool bSkelMatch = UE::Anim::BP::Editor::IsSkeletonCompatible(AnimBlueprint, Asset);
	const bool bTypeMatch = UAnimationGraphSchema::IsLocalSpacePosePin(HoverPin->PinType);
	const bool bDirectionMatch = HoverPin->Direction == EGPD_Input;

	if (bSkelMatch && bTypeMatch && bDirectionMatch)
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverPin->GetOwningNode()->GetGraph());
		FText FailureReason;
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(FAssetData(Asset), &FailureReason))
		{
			OutOkIcon = false;
			OutTooltipText = FailureReason.ToString();
		}
		else
		{
			OutOkIcon = true;
			OutTooltipText = FText::Format(LOCTEXT("AssetPinHoverMessage_Success", "Play {0} and feed to {1}"), FText::FromString(Asset->GetName()), FText::FromName(HoverPin->PinName)).ToString();
		}
	}
	else
	{
		OutOkIcon = false;
		OutTooltipText = LOCTEXT("AssetPinHoverMessage_Fail", "Type or direction mismatch; must be wired to a pose input").ToString();
	}
}

void UAnimationGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	if (UAnimationAsset* AnimationAsset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets))
	{
		UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(HoverGraph));
		const bool bSkelMatch = UE::Anim::BP::Editor::IsSkeletonCompatible(AnimBlueprint, AnimationAsset);
		if (!bSkelMatch)
		{
			OutOkIcon = false;
			if(AnimBlueprint && AnimBlueprint->bIsTemplate)
			{
				OutTooltipText = LOCTEXT("TemplateNotAllowed", "Template animation blueprints cannot reference assets").ToString();
			}
			else
			{
				OutTooltipText = LOCTEXT("SkeletonsNotCompatible", "Skeletons are not compatible").ToString();
			}
		}
		else if(UAnimMontage* Montage = Cast<UAnimMontage>(AnimationAsset))
		{
			OutOkIcon = false;
			OutTooltipText = LOCTEXT("NoMontagesInAnimGraphs", "Montages cannot be used in animation graphs").ToString();
		}
		else
		{
			const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
			FText FailureReason;
			if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(FAssetData(AnimationAsset), &FailureReason))
			{
				OutOkIcon = false;
				OutTooltipText = FailureReason.ToString();
			}
			else
			{
				OutOkIcon = true;
				OutTooltipText.Reset();
			}
		}
	}
	else if(UPhysicsAsset* PhysicsAsset = FAssetData::GetFirstAsset<UPhysicsAsset>(Assets))
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
		FText FailureReason;
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(FAssetData(PhysicsAsset), &FailureReason))
		{
			OutOkIcon = false;
			OutTooltipText = FailureReason.ToString();
		}
		else
		{
			OutOkIcon = true;
			OutTooltipText.Reset();
		}
	}
}

void UAnimationGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetContextMenuActions(Menu, Context);
	
	if (const UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(Context->Node))
	{
		{
			// Node contextual actions
			FToolMenuSection& Section = Menu->AddSection("AnimGraphSchemaNodeActions", LOCTEXT("AnimNodeActionsMenuHeader", "Anim Node Actions"));
			if (GetDefault<UAnimBlueprintSettings>()->bAllowPoseWatches)
			{
				Section.AddMenuEntry(FAnimGraphCommands::Get().TogglePoseWatch);
			}
			Section.AddMenuEntry(FAnimGraphCommands::Get().HideUnboundPropertyPins);
		}

		if(Context->Pin && !IsPosePin(Context->Pin->PinType))
		{
			TSharedPtr<SWidget> BindingWidget = MakeBindingWidgetForPin({ const_cast<UAnimGraphNode_Base*>(AnimGraphNode) }, Context->Pin->GetFName(), false, true);
			if(BindingWidget.IsValid())
			{
				FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaPinActions");
				Section.AddEntry(FToolMenuEntry::InitWidget("BindingWidget", BindingWidget.ToSharedRef(), LOCTEXT("BindingWidgetLabel", "Binding"), true));
			}
		}
	}
}

void UAnimationGraphSchema::HideUnboundPropertyPins(UAnimGraphNode_LinkedAnimGraphBase* Node)
{
	TArrayView<FOptionalPinFromProperty> OptionalPins = Node->CustomPinProperties;

	for (FOptionalPinFromProperty& OptionalPin : OptionalPins)
	{
		FName PropertyName;
		FProperty* Property = nullptr;
		int32 PinIndex = 0;
		Node->GetPinBindingInfo(OptionalPin.PropertyName, PropertyName, Property, PinIndex);
		if (Node->IsPinUnlinkedUnboundAndUnset(OptionalPin.PropertyName.ToString(), EGPD_Input))
		{
			Node->SetCustomPinVisibility(false, PinIndex);
		}
	}
}

TSharedPtr<SWidget> UAnimationGraphSchema::MakeBindingWidgetForPin(const TArray<UAnimGraphNode_Base*>& InAnimGraphNodes, FName InPinName, bool bInOnGraphNode, TAttribute<bool> bInIsEnabled)
{
	const UAnimGraphNode_Base* FirstNode = InAnimGraphNodes[0];
	
	FProperty* PinProperty = nullptr;
	int32 OptionalPinIndex = INDEX_NONE;
	FName BindingName = NAME_None;
	if(FirstNode && FirstNode->GetPinBindingInfo(InPinName, BindingName, PinProperty, OptionalPinIndex))
	{
		check(PinProperty);
		check(OptionalPinIndex != INDEX_NONE);
		check(BindingName != NAME_None);

		const bool bPropertyIsOnFNode = FirstNode->GetFNodeProperty() != nullptr && (FirstNode->GetFNodeProperty()->Struct->IsChildOf(PinProperty->GetOwner<UScriptStruct>()));
		
		UAnimGraphNode_Base::FAnimPropertyBindingWidgetArgs BindingArgs(InAnimGraphNodes, PinProperty, InPinName, BindingName, OptionalPinIndex);

		BindingArgs.OnGetOptionalPins = UAnimGraphNode_Base::FAnimPropertyBindingWidgetArgs::FOnGetOptionalPins::CreateLambda([bPropertyIsOnFNode](UAnimGraphNode_Base* InNode, TArrayView<FOptionalPinFromProperty>& OutOptionalPins)
		{
			if(UAnimGraphNode_CustomProperty* CustomProperty = Cast<UAnimGraphNode_CustomProperty>(InNode))
			{
				if(bPropertyIsOnFNode)
				{
					OutOptionalPins = InNode->ShowPinForProperties;
				}
				else
				{
					OutOptionalPins = CustomProperty->CustomPinProperties;
				}
			}
			else
			{
				OutOptionalPins = InNode->ShowPinForProperties;
			}
		});
		BindingArgs.OnSetPinVisibility = UAnimGraphNode_Base::FAnimPropertyBindingWidgetArgs::FOnSetPinVisibility::CreateLambda([bPropertyIsOnFNode](UAnimGraphNode_Base* InNode, bool bInVisible, int32 InOptionalPinIndex)
		{
			if(UAnimGraphNode_CustomProperty* CustomProperty = Cast<UAnimGraphNode_CustomProperty>(InNode))
			{
				if(bPropertyIsOnFNode)
				{
					InNode->SetPinVisibility(bInVisible, InOptionalPinIndex);
				}
				else
				{
					CustomProperty->SetCustomPinVisibility(bInVisible, InOptionalPinIndex);
				}
			}
			else
			{
				InNode->SetPinVisibility(bInVisible, InOptionalPinIndex);
			}
		});
		
		// Only show 'always dynamic' for properties of the internal FAnimNode_Base
		BindingArgs.bPropertyIsOnFNode = bPropertyIsOnFNode;
		BindingArgs.bOnGraphNode = bInOnGraphNode; 

		// Wrap in a box to control the widget's enabled & visibility states
		return SNew(SBox)
		.IsEnabled(bInIsEnabled)
		.Visibility_Lambda([BindingName, FirstNode, bInOnGraphNode]()
		{
			if(bInOnGraphNode)
			{
				if (FirstNode->HasBinding(BindingName))
				{
					return EVisibility::Visible;
				}

				return EVisibility::Collapsed;
			}
			
			return EVisibility::Visible;
		})
		[
			UAnimGraphNode_Base::MakePropertyBindingWidget(BindingArgs)
		];
	}

	return nullptr;
}

FText UAnimationGraphSchema::GetPinDisplayName(const UEdGraphPin* Pin) const 
{
	check(Pin != NULL);

	FText DisplayName = Super::GetPinDisplayName(Pin);

	if (UAnimGraphNode_Base* Node = Cast<UAnimGraphNode_Base>(Pin->GetOwningNode()))
	{
		FString ProcessedDisplayName = DisplayName.ToString();
		Node->PostProcessPinName(Pin, ProcessedDisplayName);
		DisplayName = FText::FromString(ProcessedDisplayName);
	}

	return DisplayName;
}

bool UAnimationGraphSchema::CanDuplicateGraph(UEdGraph* InSourceGraph) const
{
	return InSourceGraph->GetFName() != UEdGraphSchema_K2::GN_AnimGraph && !InSourceGraph->IsA<UAnimationBlendSpaceSampleGraph>();
}

void UAnimationGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	if (GetGraphType(&Graph) == GT_Animation)
	{
		DisplayInfo.PlainName = FText::FromString(Graph.GetName());
		DisplayInfo.DisplayName = DisplayInfo.PlainName;
		DisplayInfo.Tooltip = Graph.GetFName() == UEdGraphSchema_K2::GN_AnimGraph ? 
			LOCTEXT("GraphTooltip_AnimGraph", "Graph used to blend together different animations.") : 
			LOCTEXT("GraphTooltip_AnimGraphLayer", "Layer used to organize blending into sub-graphs.");

		if(!Graph.bAllowDeletion)
		{
			// Might be from an interface, so check
			if(UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&Graph))
			{
				TSubclassOf<UInterface> Interface;

				auto FindInterfaceForGraph = [&Blueprint, &Graph](TSubclassOf<UInterface>& OutInterface)
				{
					for(const FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
					{
						for(UEdGraph* InterfaceGraph : InterfaceDesc.Graphs)
						{
							if(InterfaceGraph == &Graph)
							{
								OutInterface = InterfaceDesc.Interface;
								return true;
							}
						}
					}

					return false;
				};

				if(FindInterfaceForGraph(Interface))
				{
					DisplayInfo.Tooltip = FText::Format(LOCTEXT("GraphTooltip_AnimGraphInterface", "Layer inherited from interface '{0}'."), FText::FromString(Interface.Get()->GetName()));
				}
			}
		}
			
		DisplayInfo.DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphTypes");
		DisplayInfo.DocExcerptName = TEXT("AnimGraph");
	}
	else
	{
		Super::GetGraphDisplayInformation(Graph, DisplayInfo);
	}
}

void UAnimationGraphSchema::AutoArrangeInterfaceGraph(UEdGraph& Graph)
{
	// auto-arrange all nodes now
	TArray<UAnimGraphNode_Root*> RootNodes;
	Graph.GetNodesOfClass<UAnimGraphNode_Root>(RootNodes);
	check(RootNodes.Num() == 1);
	UAnimGraphNode_Root* Root = RootNodes[0];
				
	TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
	Graph.GetNodesOfClass<UAnimGraphNode_LinkedInputPose>(LinkedInputPoseNodes);

	FBox2D RootBounds(FVector2D(Root->NodePosX, Root->NodePosY), FVector2D(Root->NodePosX + 130, Root->NodePosY + 200));

	double TotalHeight = 0.0;
	double MaxWidth = 0.0;
	const int32 HeightPerProperty = 30;

	for(UAnimGraphNode_LinkedInputPose* Node : LinkedInputPoseNodes)
	{
		FBox2D LinkedInputPoseBounds(
			FVector2D(Node->NodePosX, Node->NodePosY),
			FVector2D(Node->NodePosX + 400.0, Node->NodePosY + 100.0 + (Node->GetNumInputs() * HeightPerProperty))
		);

		FVector2D BoundsSize = LinkedInputPoseBounds.GetSize();
		TotalHeight += BoundsSize.Y + 10.0;
		MaxWidth = FMath::Max(BoundsSize.X, MaxWidth);
	}

	double NodeOffset = RootBounds.GetCenter().Y - (TotalHeight * 0.5);
	double NodePosX = RootBounds.Min.X - (MaxWidth + 100.0);
	for(UAnimGraphNode_LinkedInputPose* Node : LinkedInputPoseNodes)
	{
		Node->NodePosX = static_cast<int32>(NodePosX);
		Node->NodePosY = static_cast<int32>(NodeOffset);

		FBox2D LinkedInputPoseBounds(
			FVector2D(Node->NodePosX, Node->NodePosY),
			FVector2D(Node->NodePosX + 400.0, Node->NodePosY + 100.0 + (Node->GetNumInputs() * HeightPerProperty))
		);

		NodeOffset += LinkedInputPoseBounds.GetSize().Y + 10.0;
	}
}

void UAnimationGraphSchema::ConformAnimGraphToInterface(UBlueprint* InBlueprint, UEdGraph& InGraph, UFunction* InFunction)
{
	if(const UAnimationGraphSchema* Schema = ExactCast<UAnimationGraphSchema>(InGraph.GetSchema()))
	{
		// Propagate group from metadata
		TArray<UAnimGraphNode_Root*> RootNodes;
		InGraph.GetNodesOfClass<UAnimGraphNode_Root>(RootNodes);

		check(RootNodes.Num() == 1);
		RootNodes[0]->Node.SetGroup(*FObjectEditorUtils::GetCategoryText(InFunction).ToString());

		TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
		InGraph.GetNodesOfClass<UAnimGraphNode_LinkedInputPose>(LinkedInputPoseNodes);

		for(UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode : LinkedInputPoseNodes)
		{
			// Sync pose names in case they have changed
			LinkedInputPoseNode->ConformInputPoseName();

			// Clean up any old linked input poses that no longer exist
			if(!LinkedInputPoseNode->ValidateAgainstFunctionReference())
			{
				InGraph.RemoveNode(LinkedInputPoseNode);
			}
		}

		UClass* InterfaceClass = CastChecked<UClass>(InFunction->GetOuter());

		// Add any inputs that are not present in the graph (matching by pose index)
		int32 CurrentPoseIndex = 0;
		for (TFieldIterator<FProperty> PropIt(InFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* Param = *PropIt;

			const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);

			if (bIsFunctionInput)
			{
				FEdGraphPinType PinType;
				if(Schema->ConvertPropertyToPinType(Param, PinType))
				{
					// Create linked input pose for each pose pin type
					if(UAnimationGraphSchema::IsPosePin(PinType))
					{
						UAnimGraphNode_LinkedInputPose** MatchingNode = LinkedInputPoseNodes.FindByPredicate(
							[CurrentPoseIndex](UAnimGraphNode_LinkedInputPose* InLinkedInputPoseNode)
							{
								return InLinkedInputPoseNode->InputPoseIndex == CurrentPoseIndex;
							});

						if(MatchingNode == nullptr)
						{
							const FName GraphName = InGraph.GetFName();

							// Get the function GUID from the most up-to-date class
							FGuid GraphGuid;
							FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(InterfaceClass), GraphName, GraphGuid);

							// not found, add this node
							FGraphNodeCreator<UAnimGraphNode_LinkedInputPose> LinkedInputPoseNodeCreator(InGraph);
							UAnimGraphNode_LinkedInputPose* LinkedInputPoseNode = LinkedInputPoseNodeCreator.CreateNode();
							LinkedInputPoseNode->FunctionReference.SetExternalMember(GraphName, InterfaceClass, GraphGuid);
							LinkedInputPoseNode->Node.Name = Param->GetFName();
							LinkedInputPoseNode->InputPoseIndex = CurrentPoseIndex;

							LinkedInputPoseNode->ReconstructNode();
							SetNodeMetaData(LinkedInputPoseNode, FNodeMetadata::DefaultGraphNode);
							LinkedInputPoseNodeCreator.Finalize();

							FVector2D NewPosition = GetPositionForNewLinkedInputPoseNode(InGraph);
							LinkedInputPoseNode->NodePosX = static_cast<int32>(NewPosition.X);
							LinkedInputPoseNode->NodePosY = static_cast<int32>(NewPosition.Y);
						}

						CurrentPoseIndex++;
					}
				}
			}
		}
	}
}

void UAnimationGraphSchema::ConformAnimLayersByGuid(const UAnimBlueprint* InAnimBlueprint, const FBPInterfaceDescription& CurrentInterfaceDesc)
{
	const UBlueprint* InterfaceBlueprint = CastChecked<UBlueprint>(CurrentInterfaceDesc.Interface->ClassGeneratedBy);

	TArray<UEdGraph*> InterfaceGraphs;
	InterfaceBlueprint->GetAllGraphs(InterfaceGraphs);

	TArray<UEdGraph*> Graphs;
	InAnimBlueprint->GetAllGraphs(Graphs);

	for (UEdGraph* Graph : Graphs)
	{
		TArray<UAnimGraphNode_LinkedAnimLayer*> LayerNodes;
		Graph->GetNodesOfClass<UAnimGraphNode_LinkedAnimLayer>(LayerNodes);

		for (UAnimGraphNode_LinkedAnimLayer* LayerNode : LayerNodes)
		{
			LayerNode->UpdateGuidForLayer();

			if (LayerNode->InterfaceGuid.IsValid())
			{
				for (UEdGraph* InterfaceGraph : InterfaceGraphs)
				{
					// Check to see if GUID matches but name does not and update if so
					if (InterfaceGraph->GraphGuid == LayerNode->InterfaceGuid && InterfaceGraph->GetFName() != LayerNode->GetLayerName())
					{
						LayerNode->SetLayerName(InterfaceGraph->GetFName());
					}
				}
			}
		}
	}
}

FVector2D UAnimationGraphSchema::GetPositionForNewLinkedInputPoseNode(UEdGraph& InGraph)
{
	TArray<UAnimGraphNode_LinkedInputPose*> LinkedInputPoseNodes;
	InGraph.GetNodesOfClass<UAnimGraphNode_LinkedInputPose>(LinkedInputPoseNodes);

	if(LinkedInputPoseNodes.Num() == 0)
	{
		TArray<UAnimGraphNode_Base*> AllNodes;
		InGraph.GetNodesOfClass<UAnimGraphNode_Base>(AllNodes);

		// No nodes, so insert to the top-left of all existing nodes.
		FBox2D AllNodesBounds(ForceInit);
		for(UAnimGraphNode_Base* Node : AllNodes)
		{
			FBox2D NodeBounds(
				FVector2D(Node->NodePosX, Node->NodePosY),
				FVector2D(Node->NodePosX + 400, Node->NodePosY + 100)
			);

			AllNodesBounds += NodeBounds;
		}

		return FVector2D(AllNodesBounds.Min.X - 300.0f, AllNodesBounds.Min.Y);
	}
	else
	{
		const int32 HeightPerProperty = 30;

		// Some existing linked input poses. Insert below the bottom-most one.
		UAnimGraphNode_LinkedInputPose* Node = LinkedInputPoseNodes[0];

		FBox2D BottomMostLinkedInputPoseBounds(
			FVector2D(Node->NodePosX, Node->NodePosY),
			FVector2D(Node->NodePosX + 400, Node->NodePosY + 100 + (Node->GetNumInputs() * HeightPerProperty))
		);

		for(int32 LinkedInputPoseIndex = 1; LinkedInputPoseIndex < LinkedInputPoseNodes.Num(); ++LinkedInputPoseIndex)
		{
			Node = LinkedInputPoseNodes[LinkedInputPoseIndex];

			FBox2D LinkedInputPoseBounds(
				FVector2D(Node->NodePosX, Node->NodePosY),
				FVector2D(Node->NodePosX + 400, Node->NodePosY + 100 + (Node->GetNumInputs() * HeightPerProperty))
			);

			if(LinkedInputPoseBounds.Min.Y > BottomMostLinkedInputPoseBounds.Min.Y)
			{
				BottomMostLinkedInputPoseBounds = LinkedInputPoseBounds;
			}
		}

		return FVector2D(BottomMostLinkedInputPoseBounds.Min.X, BottomMostLinkedInputPoseBounds.Max.Y + 10.0f);
	}
}

TSharedPtr<IAssetReferenceFilter> UAnimationGraphSchema::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph))
	{
		if (GEditor)
		{
			FAssetReferenceFilterContext AssetReferenceFilterContext;
			AssetReferenceFilterContext.AddReferencingAsset(Blueprint);
			return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
		}
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
