// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationStateGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationStateGraph)

#define LOCTEXT_NAMESPACE "AnimationStateGraph"

/////////////////////////////////////////////////////
// UAnimationStateGraph

UAnimationStateGraph::UAnimationStateGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAnimGraphNode_StateResult* UAnimationStateGraph::GetResultNode()
{
	return MyResultNode;
}

#undef LOCTEXT_NAMESPACE
