// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationStateGraphSchema.h"
#include "AnimationStateGraph.h"
#include "AnimStateNode.h"
#include "AnimGraphNode_StateResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationStateGraphSchema)
#define LOCTEXT_NAMESPACE "AnimationStateGraphSchema"


/////////////////////////////////////////////////////
// UAnimationStateGraphSchema

UAnimationStateGraphSchema::UAnimationStateGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimationStateGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	// Create the result Create
	FGraphNodeCreator<UAnimGraphNode_StateResult> NodeCreator(Graph);
	UAnimGraphNode_StateResult* ResultSinkNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();
	SetNodeMetaData(ResultSinkNode, FNodeMetadata::DefaultGraphNode);

	UAnimationStateGraph* TypedGraph = CastChecked<UAnimationStateGraph>(&Graph);
	TypedGraph->MyResultNode = ResultSinkNode;
}

void UAnimationStateGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	DisplayInfo.PlainName = FText::FromString( Graph.GetName() );

	if (const UAnimStateNode* StateNode = Cast<const UAnimStateNode>(Graph.GetOuter()))
	{
		DisplayInfo.PlainName = FText::Format( LOCTEXT("StateNameGraphTitle", "{0} (state)"), FText::FromString( StateNode->GetStateName() ) );
	}

	DisplayInfo.DisplayName = DisplayInfo.PlainName;
	DisplayInfo.Tooltip = LOCTEXT("GraphTooltip_StateSchema", "States contain animation graphs that contribute to the state machine's output pose when the state is active");
}

bool UAnimationStateGraphSchema::CanDuplicateGraph(UEdGraph* InSourceGraph) const
{
	return false;
}


#undef LOCTEXT_NAMESPACE
