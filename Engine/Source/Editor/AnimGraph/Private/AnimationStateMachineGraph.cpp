// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationStateMachineGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationStateMachineGraph)

#define LOCTEXT_NAMESPACE "AnimationGraph"

/////////////////////////////////////////////////////
// UAnimationStateMachineGraph

UAnimationStateMachineGraph::UAnimationStateMachineGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAllowDeletion = false;
	bAllowRenaming = true;
}


#undef LOCTEXT_NAMESPACE
