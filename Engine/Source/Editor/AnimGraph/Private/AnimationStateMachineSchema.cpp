// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationStateMachineSchema.cpp
=============================================================================*/

#include "AnimationStateMachineSchema.h"
#include "Layout/SlateRect.h"
#include "Animation/AnimationAsset.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "EdGraph/EdGraph.h"
#include "Editor.h"
#include "Editor/AssetReferenceFilter.h"
#include "Editor/EditorEngine.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "EdGraphNode_Comment.h"

#include "AnimationStateMachineGraph.h"
#include "AnimationGraphSchema.h"
#include "AnimStateEntryNode.h"
#include "AnimStateNodeBase.h"
#include "AnimStateNode.h"
#include "AnimStateTransitionNode.h"
#include "AnimStateConduitNode.h"
#include "AnimStateAliasNode.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "ScopedTransaction.h"
#include "GraphEditorActions.h"

#include "EdGraphUtilities.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Framework/Commands/GenericCommands.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationStateMachineSchema)

#define LOCTEXT_NAMESPACE "AnimationStateMachineSchema"

/////////////////////////////////////////////////////

TSharedPtr<FEdGraphSchemaAction_NewStateNode> AddNewStateNodeAction(FGraphContextMenuBuilder& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip, const int32 Grouping = 0)
{
	TSharedPtr<FEdGraphSchemaAction_NewStateNode> NewStateNode( new FEdGraphSchemaAction_NewStateNode(Category, MenuDesc, Tooltip, Grouping) );
	ContextMenuBuilder.AddAction( NewStateNode );
	return NewStateNode;
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_NewStateNode

UEdGraphNode* FEdGraphSchemaAction_NewStateNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphNode* ResultNode = NULL;

	// If there is a template, we actually use it
	if (NodeTemplate != NULL)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "K2_AddNode", "Add Node") );
		ParentGraph->Modify();
		if (FromPin)
		{
			FromPin->Modify();
		}

		// set outer to be the graph so it doesn't go away
		NodeTemplate->Rename(NULL, ParentGraph);
		ParentGraph->AddNode(NodeTemplate, true, bSelectNewNode);

		NodeTemplate->CreateNewGuid();
		NodeTemplate->PostPlacedNewNode();
		NodeTemplate->AllocateDefaultPins();
		NodeTemplate->AutowireNewNode(FromPin);

		NodeTemplate->NodePosX = static_cast<int32>(Location.X);
		NodeTemplate->NodePosY = static_cast<int32>(Location.Y);
		NodeTemplate->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

		ResultNode = NodeTemplate;

		ResultNode->SetFlags(RF_Transactional);

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}

	return ResultNode;
}

void FEdGraphSchemaAction_NewStateNode::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction::AddReferencedObjects( Collector );

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject( NodeTemplate );
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_NewStateComment

UEdGraphNode* FEdGraphSchemaAction_NewStateComment::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	// Add menu item for creating comment boxes
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(ParentGraph);

	FVector2f SpawnLocation = Location;

	FSlateRect Bounds;
	if ((Blueprint != NULL) && FKismetEditorUtilities::GetBoundsForSelectedNodes(Blueprint, Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}

	return FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation);
}

/////////////////////////////////////////////////////
// UAnimationStateMachineSchema

const FName UAnimationStateMachineSchema::PC_Exec(TEXT("exec"));

UAnimationStateMachineSchema::UAnimationStateMachineSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimationStateMachineSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	// Create the entry/exit tunnels
	FGraphNodeCreator<UAnimStateEntryNode> NodeCreator(Graph);
	UAnimStateEntryNode* EntryNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();
	SetNodeMetaData(EntryNode, FNodeMetadata::DefaultGraphNode);

	if (UAnimationStateMachineGraph* StateMachineGraph = CastChecked<UAnimationStateMachineGraph>(&Graph))
	{
		StateMachineGraph->EntryNode = EntryNode;
	}
}

const FPinConnectionResponse UAnimationStateMachineSchema::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are on the same node"));
	}

	// Connect entry node to a state is OK
	const bool bPinAIsEntry = PinA->GetOwningNode()->IsA(UAnimStateEntryNode::StaticClass());
	const bool bPinBIsEntry = PinB->GetOwningNode()->IsA(UAnimStateEntryNode::StaticClass());
	const bool bPinAIsStateNode = PinA->GetOwningNode()->IsA(UAnimStateNodeBase::StaticClass());
	const bool bPinBIsStateNode = PinB->GetOwningNode()->IsA(UAnimStateNodeBase::StaticClass());

	// Special case handling for entry states: Only allow creating connections starting at the entry state.
	if (bPinAIsEntry || bPinBIsEntry)
	{
		if (bPinAIsEntry && bPinBIsStateNode)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, TEXT(""));
		}

		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Entry must connect to a state node"));
	}

	const bool bPinAIsTransition = PinA->GetOwningNode()->IsA(UAnimStateTransitionNode::StaticClass());
	const bool bPinBIsTransition = PinB->GetOwningNode()->IsA(UAnimStateTransitionNode::StaticClass());

	if (bPinAIsTransition && bPinBIsTransition)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Cannot wire a transition to a transition"));
	}
	
	// Compare the directions
	bool bDirectionsOK = false;

	if ((PinA->Direction == EGPD_Input) && (PinB->Direction == EGPD_Output))
	{
		bDirectionsOK = true;
	}
	else if ((PinB->Direction == EGPD_Input) && (PinA->Direction == EGPD_Output))
	{
		bDirectionsOK = true;
	}

	/*
	if (!bDirectionsOK)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Directions are not compatible"));
	}
	*/

	// Transitions are exclusive (both input and output), but states are not
	if (bPinAIsTransition)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, TEXT(""));
	}
	else if (bPinBIsTransition)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_B, TEXT(""));
	}
	else if (!bPinAIsTransition && !bPinBIsTransition)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, TEXT("Create a transition"));
	}
	else
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
	}
}

bool UAnimationStateMachineSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	if (PinB->Direction == PinA->Direction)
	{
		if (UAnimStateNodeBase* Node = Cast<UAnimStateNodeBase>(PinB->GetOwningNode()))
		{
			if (PinA->Direction == EGPD_Input)
			{
				PinB = Node->GetOutputPin();
			}
			else
			{
				PinB = Node->GetInputPin();
			}
		}
	}

	const bool bModified = UEdGraphSchema::TryCreateConnection(PinA, PinB);

	if (bModified)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(PinA->GetOwningNode());
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}

	return bModified;
}

bool UAnimationStateMachineSchema::CreateAutomaticConversionNodeAndConnections(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	UAnimStateNodeBase* NodeA = Cast<UAnimStateNodeBase>(PinA->GetOwningNode());
	UAnimStateNodeBase* NodeB = Cast<UAnimStateNodeBase>(PinB->GetOwningNode());

	if ((NodeA != NULL) && (NodeB != NULL) 
		&& (NodeA->GetInputPin() != NULL) && (NodeA->GetOutputPin() != NULL)
		&& (NodeB->GetInputPin() != NULL) && (NodeB->GetOutputPin() != NULL))
	{
		FVector2f Location = (FVector2f(NodeA->NodePosX, NodeA->NodePosY) + FVector2f(NodeB->NodePosX, NodeB->NodePosY)) * 0.5f;
		UAnimStateTransitionNode* TransitionNode = FEdGraphSchemaAction_NewStateNode::SpawnNodeFromTemplate<UAnimStateTransitionNode>(NodeA->GetGraph(), NewObject<UAnimStateTransitionNode>(), Location, false);

		if (PinA->Direction == EGPD_Output)
		{
			TransitionNode->CreateConnections(NodeA, NodeB);
		}
		else
		{
			TransitionNode->CreateConnections(NodeB, NodeA);
		}

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(TransitionNode->GetBoundGraph());
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

		return true;
	}

	return false;
}

bool UAnimationStateMachineSchema::TryRelinkConnectionTarget(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, UEdGraphPin* NewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const
{
	const FPinConnectionResponse Response = CanCreateConnection(SourcePin, NewTargetPin);
	if (Response.Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW)
	{
		return false;
	}

	bool bRelinkHead = SourcePin->Direction == EGPD_Output;
	UAnimStateNodeBase* OldTargetState = Cast<UAnimStateNodeBase>(OldTargetPin->GetOwningNode());
	UAnimStateNodeBase* NewTargetState = Cast<UAnimStateNodeBase>(NewTargetPin->GetOwningNode());
	if (OldTargetState == nullptr || OldTargetState->GetInputPin() == nullptr || OldTargetState->GetOutputPin() == nullptr ||
		NewTargetState == nullptr || NewTargetState->GetInputPin() == nullptr || NewTargetState->GetOutputPin() == nullptr)
	{
		return false;
	}

	// In the case we are relinking the transition starting at the entry state, the SourceState is nullptr. Special case handling.
	UAnimStateEntryNode* EntryState = Cast<UAnimStateEntryNode>(SourcePin->GetOwningNode());
	if (EntryState)
	{
		// Remove the incoming transition from the previous target state
		OldTargetPin->Modify();
		OldTargetPin->LinkedTo.Remove(SourcePin);
		SourcePin->Modify();
		SourcePin->LinkedTo.Remove(OldTargetPin);

		// Add the new incoming transition to the new target state
		TryCreateConnection(SourcePin, NewTargetPin);

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(EntryState->GetGraph());
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

		return true;
	}

	// Collect all transition nodes starting at the source state, filter them by the transitions and perform the actual relink operation.
	const TArray<UAnimStateTransitionNode*> TransitionNodes = UAnimStateTransitionNode::GetListTransitionNodesToRelink(SourcePin, OldTargetPin, InSelectedGraphNodes);

	if (bRelinkHead)
	{
		for (UAnimStateTransitionNode* TransitionNode : TransitionNodes)
		{
			TransitionNode->RelinkHead(NewTargetState);
		}
	}
	else
	{
		for (UAnimStateTransitionNode* TransitionNode : TransitionNodes)
		{
			TransitionNode->RelinkTail(NewTargetState);
		}
	}

	// In case one or more transitions got relinked, inform the blueprint about the changes
#if WITH_EDITOR
	if (!TransitionNodes.IsEmpty())
	{
		//UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(OneRelinkedTransition->GetBoundGraph());
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(TransitionNodes[0]->GetBoundGraph());
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);


		SourcePin->GetOwningNode()->PinConnectionListChanged(SourcePin);
		OldTargetPin->GetOwningNode()->PinConnectionListChanged(OldTargetPin);
		NewTargetPin->GetOwningNode()->PinConnectionListChanged(NewTargetPin);
	}
#endif//#if WITH_EDITOR

	return true;
}

bool UAnimationStateMachineSchema::IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const
{
	if (InPin && InPin->GetOwningNode())
	{
		UAnimStateNodeBase* StateNode = Cast<UAnimStateNodeBase>(InPin->GetOwningNode());
		UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(InPin->GetOwningNode());
		UAnimStateEntryNode* EntryNode = Cast<UAnimStateEntryNode>(InPin->GetOwningNode());
		if (StateNode || TransitionNode || EntryNode)
		{
			return true;
		}
	}

	return false;
}

const FPinConnectionResponse UAnimationStateMachineSchema::CanRelinkConnectionToPin(const UEdGraphPin* OldSourcePin, const UEdGraphPin* TargetPinCandidate) const
{
	FPinConnectionResponse Response = CanCreateConnection(OldSourcePin, TargetPinCandidate);
	if (Response.Response != CONNECT_RESPONSE_DISALLOW)
	{
		Response.Message = FText::FromString("Relink transition");
	}

	return Response;
}

void UAnimationStateMachineSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	// Add state node
	{
		TSharedPtr<FEdGraphSchemaAction_NewStateNode> Action = AddNewStateNodeAction(ContextMenuBuilder, FText::GetEmpty(), LOCTEXT("AddState", "Add State"), LOCTEXT("AddStateTooltip", "A new state"));
		Action->NodeTemplate = NewObject<UAnimStateNode>(ContextMenuBuilder.OwnerOfTemporaries);
	}

	// Add state alias node
	{
		TSharedPtr<FEdGraphSchemaAction_NewStateNode> Action = AddNewStateNodeAction(ContextMenuBuilder, FText::GetEmpty(), LOCTEXT("AddStateAlias", "Add State Alias"), LOCTEXT("AddStateAliasTooltip", "A new state alias"));
		Action->NodeTemplate = NewObject<UAnimStateAliasNode>(ContextMenuBuilder.OwnerOfTemporaries);
	}

	// Add conduit node
	{
		TSharedPtr<FEdGraphSchemaAction_NewStateNode> Action = AddNewStateNodeAction(ContextMenuBuilder, FText::GetEmpty(), LOCTEXT("AddConduit", "Add Conduit"), LOCTEXT("AddConduitTooltip", "A new conduit state"));
		Action->NodeTemplate = NewObject<UAnimStateConduitNode>(ContextMenuBuilder.OwnerOfTemporaries);
	}

	// Entry point (only if doesn't already exist)
	{
		bool bHasEntry = false;
		for (auto NodeIt = ContextMenuBuilder.CurrentGraph->Nodes.CreateConstIterator(); NodeIt; ++NodeIt)
		{
			UEdGraphNode* Node = *NodeIt;
			if (const UAnimStateEntryNode* StateNode = Cast<UAnimStateEntryNode>(Node))
			{
				bHasEntry = true;
				break;
			}
		}

		if (!bHasEntry)
		{
			TSharedPtr<FEdGraphSchemaAction_NewStateNode> Action = AddNewStateNodeAction(ContextMenuBuilder, FText::GetEmpty(), LOCTEXT("AddEntryPoint", "Add Entry Point"), LOCTEXT("AddEntryPointTooltip", "Define State Machine's Entry Point"));
			Action->NodeTemplate = NewObject<UAnimStateEntryNode>(ContextMenuBuilder.OwnerOfTemporaries);
		}
	}

	// Add Comment
	if (!ContextMenuBuilder.FromPin)
	{
		UBlueprint* OwnerBlueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ContextMenuBuilder.CurrentGraph);
		const bool bIsManyNodesSelected = (FKismetEditorUtilities::GetNumberOfSelectedNodes(OwnerBlueprint) > 0);
		const FText MenuDescription = bIsManyNodesSelected ? LOCTEXT("CreateCommentSelection", "Create Comment from Selection") : LOCTEXT("AddComment", "Add Comment");
		const FText ToolTip = LOCTEXT("CreateCommentSelectionTooltip", "Create a resizeable comment box around selected nodes.");

		TSharedPtr<FEdGraphSchemaAction_NewStateComment> NewComment( new FEdGraphSchemaAction_NewStateComment(FText::GetEmpty(), MenuDescription, ToolTip, 0) );
		ContextMenuBuilder.AddAction( NewComment );
	}
}

EGraphType UAnimationStateMachineSchema::GetGraphType(const UEdGraph* TestEdGraph) const
{
	return GT_StateMachine;
}

void UAnimationStateMachineSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	check(Context && Context->Graph);
	UBlueprint* OwnerBlueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Context->Graph);

	if (Context->Node)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("AnimationStateMachineNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"));
			if (!Context->bIsDebugging)
			{
				// Node contextual actions
				Section.AddMenuEntry(FGenericCommands::Get().Delete);
				Section.AddMenuEntry(FGenericCommands::Get().Cut);
				Section.AddMenuEntry(FGenericCommands::Get().Copy);
				Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
				Section.AddMenuEntry(FGraphEditorCommands::Get().ReconstructNodes);
				Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
				if(Context->Node->bCanRenameNode)
				{
					Section.AddMenuEntry(FGenericCommands::Get().Rename);
				}
			}
		}
	}

	Super::GetContextMenuActions(Menu, Context);
}

FLinearColor UAnimationStateMachineSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	if (PinType.PinCategory == TEXT("Transition"))
	{
		return FLinearColor::White;
	}

	return GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);
}

void UAnimationStateMachineSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	DisplayInfo.PlainName = FText::FromString( Graph.GetName() );
	DisplayInfo.DisplayName = DisplayInfo.PlainName;
	DisplayInfo.Tooltip = LOCTEXT("GraphTooltip_StateMachineSchema", "Graph used to transition between different states each with separate animation graphs");
}

void UAnimationStateMachineSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	if(Asset != nullptr && GetNodeClassForAsset(Asset->GetClass()))
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);
		if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(FAssetData(Asset)))
		{
			// Spawn new state
			UAnimStateNode* NewStateNode = FEdGraphSchemaAction_NewStateNode::SpawnNodeFromTemplate<UAnimStateNode>(Graph, NewObject<UAnimStateNode>(), GraphPosition);

			// Try to name the state close to the asset
			FEdGraphUtilities::RenameGraphToNameOrCloseToName(NewStateNode->BoundGraph, Asset->GetName());

			// Change the current graph context to the inner graph, so that the rest of the drag drop happens inside it
			FVector2f NewGraphPosition = FVector2f(-300.0f, 0.0f);
			UAnimationGraphSchema::SpawnNodeFromAsset(Asset, NewGraphPosition, NewStateNode->BoundGraph, NewStateNode->GetPoseSinkPinInsideState());
		}
	}
}

void UAnimationStateMachineSchema::DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	UAnimStateNode* StateNodeUnderCursor = Cast<UAnimStateNode>(Node);
	if(Asset != nullptr && StateNodeUnderCursor != nullptr)
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(StateNodeUnderCursor->GetGraph());
		if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(FAssetData(Asset)))
		{
			// Dropped onto a state machine state; try some user friendly resposnes
			if (UEdGraphPin* PosePin = StateNodeUnderCursor->GetPoseSinkPinInsideState())
			{
				if (PosePin->LinkedTo.Num() > 0)
				{
					//@TODO: A2REMOVAL: This doesn't do anything
					/*
					// Try dropping in onto the node attached to the sink inside the state
					check(PosePin->LinkedTo[0] != NULL);
					UA2Node* A2Node = Cast<UA2Node>(PosePin->LinkedTo[0]->GetOwningNode());
					if(A2Node != NULL)
					{
						UAnimationGraphSchema::UpdateNodeWithAsset(A2Node, Asset);
					}
					*/
				}
				else
				{
					// Try dropping onto the pose pin, since nothing is currently connected
					const FVector2f NewGraphPosition(-300.0f, 0.0f);
					UAnimationGraphSchema::SpawnNodeFromAsset(Asset, NewGraphPosition, StateNodeUnderCursor->BoundGraph, PosePin);
				}
			}
		}
	}
}

void UAnimationStateMachineSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	if ((Asset == nullptr) || (HoverGraph == nullptr))
	{
		OutTooltipText.Reset();
		OutOkIcon = false;
		return;
	}

	if (GetNodeClassForAsset(Asset->GetClass()) != nullptr)
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
		FText FailureReason;
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(FAssetData(Asset), &FailureReason))
		{
			OutOkIcon = false;
			OutTooltipText = FailureReason.ToString();
		}
		else
		{
			OutOkIcon = true;
			OutTooltipText.Reset();
		}
	}
	else
	{
		OutTooltipText.Reset();
		OutOkIcon = false;
	}
}

void UAnimationStateMachineSchema::GetAssetsNodeHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const 
{ 
	UAnimationAsset* Asset = FAssetData::GetFirstAsset<UAnimationAsset>(Assets);
	if ((Asset == nullptr) || (HoverNode == nullptr))
	{
		OutTooltipText.Reset();
		OutOkIcon = false;
		return;
	}

	const UAnimStateNode* StateNodeUnderCursor = Cast<const UAnimStateNode>(HoverNode);
	if (StateNodeUnderCursor != nullptr)
	{
		const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverNode->GetGraph());
		FText FailureReason;
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(FAssetData(Asset), &FailureReason))
		{
			OutOkIcon = false;
			OutTooltipText = FailureReason.ToString();
		}
		else
		{
			OutOkIcon = true;
			OutTooltipText = FText::FormatOrdered(LOCTEXT("DragDropAssetMessage", "Change node to play {0}"), FText::FromName(Asset->GetFName())).ToString();
		}
	}
	else
	{
		OutTooltipText.Reset();
		OutOkIcon = false;
	}
}

void UAnimationStateMachineSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakNodeLinks", "Break Node Links"));

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(&TargetNode);
	Super::BreakNodeLinks(TargetNode);
	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

void UAnimationStateMachineSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links"));
	// cache this here, as BreakPinLinks can trigger a node reconstruction invalidating the TargetPin references
	UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(TargetPin.GetOwningNode());
	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);
	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

void UAnimationStateMachineSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakSinglePinLink", "Break Pin Link"));
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(TargetPin->GetOwningNode());
	Super::BreakSinglePinLink(SourcePin, TargetPin);
	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

TSharedPtr<IAssetReferenceFilter> UAnimationStateMachineSchema::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph))
	{
		if (GEditor)
		{
			FAssetReferenceFilterContext AssetReferenceFilterContext;
			AssetReferenceFilterContext.AddReferencingAsset(Blueprint);
			return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
		}
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
