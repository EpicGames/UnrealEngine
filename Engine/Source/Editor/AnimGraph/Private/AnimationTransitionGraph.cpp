// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationTransitionGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationTransitionGraph)

#define LOCTEXT_NAMESPACE "AnimationStateGraph"

/////////////////////////////////////////////////////
// UAnimationTransitionGraph

UAnimationTransitionGraph::UAnimationTransitionGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAnimGraphNode_TransitionResult* UAnimationTransitionGraph::GetResultNode()
{
	return MyResultNode;
}

#undef LOCTEXT_NAMESPACE
