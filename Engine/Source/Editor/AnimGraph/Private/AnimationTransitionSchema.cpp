// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationTransitionSchema.cpp
=============================================================================*/

#include "AnimationTransitionSchema.h"
#include "Animation/AnimBlueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimStateTransitionNode.h"
#include "AnimationTransitionGraph.h"
#include "AnimGraphNode_TransitionResult.h"


/////////////////////////////////////////////////////
// UAnimationTransitionSchema

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationTransitionSchema)

#define LOCTEXT_NAMESPACE "AnimationTransitionSchema"

UAnimationTransitionSchema::UAnimationTransitionSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Initialize defaults
}

void UAnimationTransitionSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	// Create the entry/exit tunnels
	FGraphNodeCreator<UAnimGraphNode_TransitionResult> NodeCreator(Graph);
	UAnimGraphNode_TransitionResult* ResultSinkNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();
	SetNodeMetaData(ResultSinkNode, FNodeMetadata::DefaultGraphNode);

	UAnimationTransitionGraph* TypedGraph = CastChecked<UAnimationTransitionGraph>(&Graph);
	TypedGraph->MyResultNode = ResultSinkNode;
}

void UAnimationTransitionSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	DisplayInfo.PlainName = FText::FromString( Graph.GetName() );

	const UAnimStateTransitionNode* TransNode = Cast<const UAnimStateTransitionNode>(Graph.GetOuter());

	// If we don't have a node we can get it from our blueprint, unless the graph has been deleted
	// in which case the outer chain will have been broken.
	if (TransNode == NULL && IsValid(&Graph))
	{
		//@TODO: Transition graphs should be created with the transition node as their outer as well!
		if(UAnimBlueprint* Blueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(&Graph)))
		{
			if (UAnimBlueprintGeneratedClass* AnimBlueprintClass = Blueprint->GetAnimBlueprintSkeletonClass())
			{
				TransNode = GetTransitionNodeFromGraph(AnimBlueprintClass->GetAnimBlueprintDebugData(), &Graph);
			}
		}
	}

	if (TransNode)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), TransNode->GetNodeTitle(ENodeTitleType::FullTitle));
		DisplayInfo.PlainName = FText::Format( LOCTEXT( "TransitionRuleGraphTitle", "{NodeTitle} (rule)"), Args );
		DisplayInfo.Tooltip = LOCTEXT("GraphTooltip_TransitionSchema", "Transitions contain rules that define when to move between states");
	}

	DisplayInfo.DisplayName = DisplayInfo.PlainName;
}

UAnimStateTransitionNode* UAnimationTransitionSchema::GetTransitionNodeFromGraph(const FAnimBlueprintDebugData& DebugData, const UEdGraph* Graph)
{
	if (const TWeakObjectPtr<UAnimStateTransitionNode>* TransNodePtr = DebugData.TransitionGraphToNodeMap.Find(Graph))
	{
		return TransNodePtr->Get();
	}

	if (const TWeakObjectPtr<UAnimStateTransitionNode>* TransNodePtr = DebugData.TransitionBlendGraphToNodeMap.Find(Graph))
	{
		return TransNodePtr->Get();
	}

	return NULL;
}

UAnimStateNode* UAnimationTransitionSchema::GetStateNodeFromGraph(const FAnimBlueprintDebugData& DebugData, const UEdGraph* Graph)
{
	if (const TWeakObjectPtr<UAnimStateNode>* StateNodePtr = DebugData.StateGraphToNodeMap.Find(Graph))
	{
		return StateNodePtr->Get();
	}

	return NULL;
}

void UAnimationTransitionSchema::HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const
{
	Super::HandleGraphBeingDeleted(GraphBeingRemoved);

	if(UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&GraphBeingRemoved))
	{
		// Handle composite anim graph nodes
		TArray<UAnimStateNodeBase*> StateNodes;
		FBlueprintEditorUtils::GetAllNodesOfClassEx<UAnimStateTransitionNode>(Blueprint, StateNodes);

		TSet<UAnimStateNodeBase*> NodesToDelete;
		for(int32 i = 0; i < StateNodes.Num(); ++i)
		{
			UAnimStateNodeBase* StateNode = StateNodes[i];
			if(StateNode->GetBoundGraph() == &GraphBeingRemoved)
			{
				NodesToDelete.Add(StateNode);
			}
		}

		// Delete the node that owns us
		ensure(NodesToDelete.Num() <= 1);
		for(TSet<UAnimStateNodeBase*>::TIterator It(NodesToDelete); It; ++It)
		{
			UAnimStateNodeBase* NodeToDelete = *It;

			FBlueprintEditorUtils::RemoveNode(Blueprint, NodeToDelete, true);

			// Prevent re-entrancy here
			NodeToDelete->ClearBoundGraph();
		}
	}
}

#undef LOCTEXT_NAMESPACE
