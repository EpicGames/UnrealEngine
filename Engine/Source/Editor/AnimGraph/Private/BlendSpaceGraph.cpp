// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendSpaceGraph.h"

#include "EdGraph/EdGraphSchema.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSpaceGraph)

void UBlendSpaceGraph::PostLoad()
{
	Super::PostLoad();

	// Fixup graph schema
	if(Schema && Schema != UEdGraphSchema::StaticClass())
	{
		Schema = UEdGraphSchema::StaticClass();
	}
}
