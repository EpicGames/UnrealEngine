// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomPropertyOptionalPinManager.h"

#include "AnimGraphNode_CustomProperty.h"
#include "AnimationGraphSchema.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

FCustomPropertyOptionalPinManager::FCustomPropertyOptionalPinManager(UAnimGraphNode_CustomProperty* InCustomPropertyNode, TArray<UEdGraphPin*>* InOldPins)
	: CustomPropertyNode(InCustomPropertyNode)
	, OldPins(InOldPins)
{
}

void FCustomPropertyOptionalPinManager::GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const
{
	Record.bCanToggleVisibility = true;
	Record.bShowPin = !TestProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance);
	Record.bPropertyIsCustomized = false;
}

void FCustomPropertyOptionalPinManager::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const
{
	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();
	Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
}

bool FCustomPropertyOptionalPinManager::CanTreatPropertyAsOptional(FProperty* TestProperty) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	FEdGraphPinType PinType;
	return (TestProperty->HasAllPropertyFlags(CPF_Edit | CPF_BlueprintVisible) &&
			K2Schema->ConvertPropertyToPinType(TestProperty, PinType) &&
			!TestProperty->HasMetaData(FBlueprintMetadata::MD_Private));
}

void FCustomPropertyOptionalPinManager::CreateCustomPins(UStruct* TargetClass)
{
	RebuildPropertyList(CustomPropertyNode->CustomPinProperties, TargetClass);
	CreateVisiblePins(CustomPropertyNode->CustomPinProperties, TargetClass, EGPD_Input, CustomPropertyNode, nullptr, nullptr);
}
