// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "K2Node.h"
#include "UObject/NameTypes.h"

class FProperty;
class UAnimGraphNode_CustomProperty;
class UClass;
class UEdGraphPin;

struct FCustomPropertyOptionalPinManager : public FOptionalPinManager
{
protected:
	UAnimGraphNode_CustomProperty* CustomPropertyNode;
	TArray<UEdGraphPin*>* OldPins;

public:
	FCustomPropertyOptionalPinManager(UAnimGraphNode_CustomProperty* InCustomPropertyNode, TArray<UEdGraphPin*>* InOldPins);

	/** FOptionalPinManager interface */
	virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override;
	virtual bool CanTreatPropertyAsOptional(FProperty* TestProperty) const override;
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const override;

	void CreateCustomPins(UStruct* TargetClass);
};
