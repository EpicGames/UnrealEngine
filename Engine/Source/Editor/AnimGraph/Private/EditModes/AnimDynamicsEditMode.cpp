// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModes/AnimDynamicsEditMode.h"

#include "AnimGraphNode_AnimDynamics.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimPhysicsSolver.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneContainer.h"
#include "BoneControllers/AnimNode_AnimDynamics.h"
#include "Components/SkeletalMeshComponent.h"
#include "EdMode.h"
#include "EditorViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "HAL/PlatformCrt.h"
#include "HitProxies.h"
#include "IPersonaPreviewScene.h"
#include "InputCoreTypes.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/IntVector.h"
#include "Math/Quat.h"
#include "Math/RotationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/TransformVectorized.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"
#include "UnrealClient.h"

class FSceneView;

// AnimDynamicsEditMode hit proxy
//
// Allow users to select objects in the viewport for editing with a TRS widget.
//
struct HAnimDynamicsEditModeHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY()

	FAnimDynamicsViewportObjectReference ViewportObjectRef;

	HAnimDynamicsEditModeHitProxy(const uint32 InEditorNodeUniqueId, const FAnimDynamicsViewportObjectType InType, const uint32 InIndex) : HHitProxy(HPP_World), ViewportObjectRef(InEditorNodeUniqueId, InType, InIndex) {}
	HAnimDynamicsEditModeHitProxy(const FAnimDynamicsViewportObjectReference& InGeometricObjectRef) : HHitProxy(HPP_World), ViewportObjectRef(InGeometricObjectRef) {}
};

IMPLEMENT_HIT_PROXY(HAnimDynamicsEditModeHitProxy, HHitProxy);

// Class FAnimDynamicsViewportObjectReference
FAnimDynamicsViewportObjectReference::FAnimDynamicsViewportObjectReference(const uint32 InEditorNodeUniqueId, const FAnimDynamicsViewportObjectType InType, const uint32 InIndex)
	: EditorNodeUniqueId(InEditorNodeUniqueId)
	, Type(InType)
	, Index(InIndex)
{}

const bool operator==(const FAnimDynamicsViewportObjectReference& Lhs, const FAnimDynamicsViewportObjectReference& Rhs)
{
	return
		(Lhs.EditorNodeUniqueId == Rhs.EditorNodeUniqueId) &&
		(Lhs.Type == Rhs.Type) &&
		(Lhs.Index == Rhs.Index);
}

// Non-member utility functions
void DrawAngularLimits(FPrimitiveDrawInterface* PDI, FTransform JointTransform, const FAnimPhysConstraintSetup& ConstraintSetup)
{
	FVector XAxis = JointTransform.GetUnitAxis(EAxis::X);
	FVector YAxis = JointTransform.GetUnitAxis(EAxis::Y);
	FVector ZAxis = JointTransform.GetUnitAxis(EAxis::Z);

	const FVector& MinAngles = ConstraintSetup.AngularLimitsMin;
	const FVector& MaxAngles = ConstraintSetup.AngularLimitsMax;
	FVector AngleRange = MaxAngles - MinAngles;
	FVector Middle = MinAngles + AngleRange * 0.5f;

	if (AngleRange.X > 0.0f && AngleRange.X < 180.0f)
	{
		FTransform XAxisConeTM(YAxis, XAxis ^ YAxis, XAxis, JointTransform.GetTranslation());
		XAxisConeTM.SetRotation(FQuat(XAxis, FMath::DegreesToRadians(-Middle.X)) * XAxisConeTM.GetRotation());
		DrawCone(PDI, FScaleMatrix(30.0f) * XAxisConeTM.ToMatrixWithScale(), FMath::DegreesToRadians(static_cast<float>(AngleRange.X) / 2.0f), 0.0f, 24, false, FLinearColor::White, GEngine->ConstraintLimitMaterialX->GetRenderProxy(), SDPG_World);
	}

	if (AngleRange.Y > 0.0f && AngleRange.Y < 180.0f)
	{
		FTransform YAxisConeTM(ZAxis, YAxis ^ ZAxis, YAxis, JointTransform.GetTranslation());
		YAxisConeTM.SetRotation(FQuat(YAxis, FMath::DegreesToRadians(Middle.Y)) * YAxisConeTM.GetRotation());
		DrawCone(PDI, FScaleMatrix(30.0f) * YAxisConeTM.ToMatrixWithScale(), FMath::DegreesToRadians(static_cast<float>(AngleRange.Y) / 2.0f), 0.0f, 24, false, FLinearColor::White, GEngine->ConstraintLimitMaterialY->GetRenderProxy(), SDPG_World);
	}

	if (AngleRange.Z > 0.0f && AngleRange.Z < 180.0f)
	{
		FTransform ZAxisConeTM(XAxis, ZAxis ^ XAxis, ZAxis, JointTransform.GetTranslation());
		ZAxisConeTM.SetRotation(FQuat(ZAxis, FMath::DegreesToRadians(Middle.Z)) * ZAxisConeTM.GetRotation());
		DrawCone(PDI, FScaleMatrix(30.0f) * ZAxisConeTM.ToMatrixWithScale(), FMath::DegreesToRadians(static_cast<float>(AngleRange.Z) / 2.0f), 0.0f, 24, false, FLinearColor::White, GEngine->ConstraintLimitMaterialZ->GetRenderProxy(), SDPG_World);
	}
}

void DrawLinearLimits(FPrimitiveDrawInterface* PDI, FTransform ShapeTransform, const FAnimPhysConstraintSetup& ConstraintSetup)
{
	// Draw linear limits
	FVector LinearLimitHalfExtents(ConstraintSetup.LinearAxesMax - ConstraintSetup.LinearAxesMin);
	// Add a tiny bit so we can see collapsed axes
	LinearLimitHalfExtents += FVector(0.1f);
	LinearLimitHalfExtents /= 2.0f;
	FVector LinearLimitsCenter = ConstraintSetup.LinearAxesMin + LinearLimitHalfExtents;
	FTransform LinearLimitsTransform = ShapeTransform;
	LinearLimitsTransform.SetTranslation(LinearLimitsTransform.GetTranslation() + LinearLimitsTransform.TransformVector(LinearLimitsCenter));

	DrawBox(PDI, LinearLimitsTransform.ToMatrixWithScale(), LinearLimitHalfExtents, GEngine->ConstraintLimitMaterialPrismatic->GetRenderProxy(), SDPG_Foreground);
}

template< typename TShape > void DrawShape(FPrimitiveDrawInterface* const PDI, const TShape& Shape, const FTransform& Transform, const FLinearColor& Color, const float LineWidth)
{
	for (const FIntVector& Triangle : Shape.Triangles)
	{
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			int32 Next = (Idx + 1) % 3;

			FVector FirstVertPosition = Transform.TransformPosition(Shape.Vertices[Triangle[Idx]]);
			FVector SecondVertPosition = Transform.TransformPosition(Shape.Vertices[Triangle[Next]]);

			PDI->DrawLine(FirstVertPosition, SecondVertPosition, Color, SDPG_Foreground, LineWidth);
		}
	}
}

void DrawBasis(FPrimitiveDrawInterface* const PDI, const FTransform& Transform)
{
	const FVector Origin = Transform.GetTranslation();
	PDI->DrawLine(Origin, Origin + Transform.TransformVector(FVector::XAxisVector) * AnimDynamicsNodeConstants::TransformBasisScale, FLinearColor::Red, SDPG_Foreground, AnimDynamicsNodeConstants::TransformLineWidth);
	PDI->DrawLine(Origin, Origin + Transform.TransformVector(FVector::YAxisVector) * AnimDynamicsNodeConstants::TransformBasisScale, FLinearColor::Green, SDPG_Foreground, AnimDynamicsNodeConstants::TransformLineWidth);
	PDI->DrawLine(Origin, Origin + Transform.TransformVector(FVector::ZAxisVector) * AnimDynamicsNodeConstants::TransformBasisScale, FLinearColor::Blue, SDPG_Foreground, AnimDynamicsNodeConstants::TransformLineWidth);
}


// Class FAnimDynamicsEditMode
FAnimDynamicsEditMode::FAnimDynamicsEditMode()
	: CurWidgetMode(UE::Widget::WM_Translate)
	, bIsInteractingWithWidget(false)
{}

void FAnimDynamicsEditMode::ExitMode()
{
	SelectedViewportObjects.Empty();

	for (const EditorRuntimeNodePair& CurrentNodePair : SelectedAnimNodes)
	{
		if (const UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = CastChecked<UAnimGraphNode_AnimDynamics>(CurrentNodePair.EditorAnimNode))
		{
			if (FAnimNode_AnimDynamics* const ActivePreviewNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode())
			{
				ActivePreviewNode->bDoPhysicsUpdateInEditor = true; // Ensure physics update is enabled on premature exit.
			}
		}
	}

	FAnimNodeEditMode::ExitMode();
}

void FAnimDynamicsEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	USkeletalMeshComponent* const PreviewSkelMeshComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	check(View);
	check(Viewport);
	check(PDI);
	check(PreviewSkelMeshComp);

	for (const EditorRuntimeNodePair& CurrentNodePair : SelectedAnimNodes)
	{
		const UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = CastChecked<UAnimGraphNode_AnimDynamics>(CurrentNodePair.EditorAnimNode);

		if (EditorAnimDynamicsNode)
		{
			const FAnimNode_AnimDynamics* const ActivePreviewNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode();

			if (!ActivePreviewNode)
			{
				return;
			}

			if (EditorAnimDynamicsNode->IsPreviewLiveActive())
			{
				for (int32 BodyIndex = 0; BodyIndex < ActivePreviewNode->GetNumBodies(); ++BodyIndex)
				{
					const FAnimPhysRigidBody& Body = ActivePreviewNode->GetPhysBody(BodyIndex);
					FTransform BodyTransform(Body.Pose.Orientation, Body.Pose.Position);

					// Physics bodies are in Simulation Space. Transform into component space before rendering in the viewport.
					if (ActivePreviewNode->SimulationSpace == AnimPhysSimSpaceType::RootRelative)
					{
						const FTransform RelativeBoneTransform = PreviewSkelMeshComp->GetBoneTransform(0);
						BodyTransform = BodyTransform * RelativeBoneTransform;
					}
					else if (ActivePreviewNode->SimulationSpace == AnimPhysSimSpaceType::BoneRelative)
					{
						const FTransform RelativeBoneTransform = PreviewSkelMeshComp->GetBoneTransform(PreviewSkelMeshComp->GetBoneIndex(ActivePreviewNode->RelativeSpaceBone.BoneName));
						BodyTransform = BodyTransform * RelativeBoneTransform;
					}

					for (const FAnimPhysShape& Shape : Body.Shapes)
					{
						for (const FIntVector& Triangle : Shape.Triangles)
						{
							for (int32 Idx = 0; Idx < 3; ++Idx)
							{
								int32 Next = (Idx + 1) % 3;

								FVector FirstVertPosition = BodyTransform.TransformPosition(Shape.Vertices[Triangle[Idx]]);
								FVector SecondVertPosition = BodyTransform.TransformPosition(Shape.Vertices[Triangle[Next]]);

								PDI->DrawLine(FirstVertPosition, SecondVertPosition, AnimDynamicsNodeConstants::ActiveBodyDrawColor, SDPG_Foreground, AnimDynamicsNodeConstants::ShapeLineWidth);
							}
						}
					}

					const int32 BoneIndex = PreviewSkelMeshComp->GetBoneIndex(ActivePreviewNode->BoundBone.BoneName);
					if (BoneIndex != INDEX_NONE)
					{
						FTransform BodyJointTransform = PreviewSkelMeshComp->GetBoneTransform(BoneIndex);
						FTransform ShapeOriginalTransform = BodyJointTransform;

						// Draw pin location
						const FVector LocalPinOffset = - BodyTransform.Rotator().RotateVector(ActivePreviewNode->GetBodyLocalJointOffset(BodyIndex)); // Position of joint relative to physics body in world space.
						PDI->DrawLine(Body.Pose.Position, Body.Pose.Position + LocalPinOffset, FLinearColor::Green, SDPG_Foreground, AnimDynamicsNodeConstants::ShapeLineWidth);

						// Draw basis at body location
						FVector Origin = BodyTransform.GetTranslation();
						FVector XAxis(1.0f, 0.0f, 0.0f);
						FVector YAxis(0.0f, 1.0f, 0.0f);
						FVector ZAxis(0.0f, 0.0f, 1.0f);

						XAxis = BodyTransform.TransformVector(XAxis);
						YAxis = BodyTransform.TransformVector(YAxis);
						ZAxis = BodyTransform.TransformVector(ZAxis);

						PDI->DrawLine(Origin, Origin + XAxis * AnimDynamicsNodeConstants::TransformBasisScale, FLinearColor::Red, SDPG_Foreground, AnimDynamicsNodeConstants::TransformLineWidth);
						PDI->DrawLine(Origin, Origin + YAxis * AnimDynamicsNodeConstants::TransformBasisScale, FLinearColor::Green, SDPG_Foreground, AnimDynamicsNodeConstants::TransformLineWidth);
						PDI->DrawLine(Origin, Origin + ZAxis * AnimDynamicsNodeConstants::TransformBasisScale, FLinearColor::Blue, SDPG_Foreground, AnimDynamicsNodeConstants::TransformLineWidth);
					}
				}
			}

			uint32 PhysicsBodyIndex = 0;

			for (const FAnimPhysBodyDefinition& BodyDefinition : ActivePreviewNode->PhysicsBodyDefinitions)
			{
				const int32 BoneIndex = PreviewSkelMeshComp->GetBoneIndex(BodyDefinition.BoundBone.BoneName);
				if (BoneIndex != INDEX_NONE)
				{
					// World space transform
					const FTransform BoneTransform = PreviewSkelMeshComp->GetBoneTransform(BoneIndex);
					FTransform ShapeTransform = BoneTransform;
					ShapeTransform.SetTranslation(ShapeTransform.GetTranslation() + BoneTransform.GetRotation().RotateVector(BodyDefinition.LocalJointOffset)); // Transform of rendered shape in world space.

					const FAnimPhysShape EditPreviewShape = FAnimPhysShape::MakeBox(BodyDefinition.BoxExtents);

					{
						const FAnimDynamicsViewportObjectReference ViewportObjectRef(EditorAnimDynamicsNode->GetUniqueID(), FAnimDynamicsViewportObjectType::BoxExtents, PhysicsBodyIndex);
						const bool bIsSelected = SelectedViewportObjects.Contains(ViewportObjectRef);
						const float LineWidth = (bIsSelected) ? AnimDynamicsNodeConstants::SelectedShapeLineWidth : AnimDynamicsNodeConstants::ShapeLineWidth;

						// Draw Physics Body
						PDI->SetHitProxy(new HAnimDynamicsEditModeHitProxy(ViewportObjectRef));
						DrawShape(PDI, EditPreviewShape, ShapeTransform, AnimDynamicsNodeConstants::ShapeDrawColor, LineWidth);
						PDI->SetHitProxy(nullptr);
					}

					DrawBasis(PDI, ShapeTransform);

					// Draw line connecting rendered shape to its associated bone.
					PDI->DrawLine(ShapeTransform.GetTranslation(), BoneTransform.GetTranslation(), FLinearColor::Green, SDPG_Foreground, AnimDynamicsNodeConstants::ShapeLineWidth);

					if (EditorAnimDynamicsNode->bShowLinearLimits)
					{
						DrawLinearLimits(PDI, ShapeTransform, BodyDefinition.ConstraintSetup);
					}

					if (EditorAnimDynamicsNode->bShowAngularLimits)
					{
						DrawAngularLimits(PDI, ShapeTransform, BodyDefinition.ConstraintSetup);
					}

					if (EditorAnimDynamicsNode->bShowCollisionSpheres && BodyDefinition.CollisionType != AnimPhysCollisionType::CoM)
					{
						const FAnimDynamicsViewportObjectReference ViewportObjectRef(EditorAnimDynamicsNode->GetUniqueID(), FAnimDynamicsViewportObjectType::SphericalColisionVolume, PhysicsBodyIndex);
						const bool bIsSelected = SelectedViewportObjects.Contains(ViewportObjectRef);
						const float LineWidth = (bIsSelected) ? AnimDynamicsNodeConstants::SelectedShapeLineWidth : AnimDynamicsNodeConstants::ShapeLineWidth;

						// Draw Collision Sphere.
						PDI->SetHitProxy(new HAnimDynamicsEditModeHitProxy(ViewportObjectRef));
						DrawWireSphere(PDI, ShapeTransform, FLinearColor(FColor::Cyan), BodyDefinition.SphereCollisionRadius, 24, SDPG_Foreground, LineWidth);
						PDI->SetHitProxy(nullptr);
					}
				}

				++PhysicsBodyIndex;
			}

			const float LimitPlaneSize = AnimDynamicsNodeConstants::LimitPlaneDrawSize;
			const FLinearColor LimitEdgeColor = AnimDynamicsNodeConstants::LimitLineDrawColor;


			// Draw the planar limits.
			if (EditorAnimDynamicsNode->bShowPlanarLimit && ActivePreviewNode->PlanarLimits.Num() > 0)
			{
				uint32 LimitIndex = 0;
				for (const FAnimPhysPlanarLimit& PlanarLimit : ActivePreviewNode->PlanarLimits)
				{
					FTransform LimitPlaneTransform = PlanarLimit.PlaneTransform;
					const int32 LimitDrivingBoneIdx = PreviewSkelMeshComp->GetBoneIndex(PlanarLimit.DrivingBone.BoneName);

					if (LimitDrivingBoneIdx != INDEX_NONE)
					{
						LimitPlaneTransform *= PreviewSkelMeshComp->GetComponentSpaceTransforms()[LimitDrivingBoneIdx];
					}

					const FMatrix PlaneTransform = LimitPlaneTransform.ToMatrixNoScale();

					const FAnimDynamicsViewportObjectReference ViewportObjectRef(EditorAnimDynamicsNode->GetUniqueID(), FAnimDynamicsViewportObjectType::PlaneLimit, LimitIndex);
					const bool bIsSelected = SelectedViewportObjects.Contains(ViewportObjectRef);
					const float LineWidth = (bIsSelected) ? AnimDynamicsNodeConstants::SelectedShapeLineWidth : AnimDynamicsNodeConstants::ShapeLineWidth;

					PDI->SetHitProxy(new HAnimDynamicsEditModeHitProxy(ViewportObjectRef));
					DrawPlane10x10(PDI, PlaneTransform, LimitPlaneSize, FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f), GEngine->ConstraintLimitMaterialPrismatic->GetRenderProxy(), SDPG_World);

					const FVector PlaneVertexPositionA(PlaneTransform.TransformPosition(FVector(-1.0, -1.0, 0.0) * LimitPlaneSize));
					const FVector PlaneVertexPositionB(PlaneTransform.TransformPosition(FVector(1.0, -1.0, 0.0) * LimitPlaneSize));
					const FVector PlaneVertexPositionC(PlaneTransform.TransformPosition(FVector(1.0, 1.0, 0.0) * LimitPlaneSize));
					const FVector PlaneVertexPositionD(PlaneTransform.TransformPosition(FVector(-1.0, 1.0, 0.0) * LimitPlaneSize));

					PDI->DrawLine(PlaneVertexPositionA, PlaneVertexPositionB, LimitEdgeColor, SDPG_Foreground, LineWidth);
					PDI->DrawLine(PlaneVertexPositionB, PlaneVertexPositionC, LimitEdgeColor, SDPG_Foreground, LineWidth);
					PDI->DrawLine(PlaneVertexPositionC, PlaneVertexPositionD, LimitEdgeColor, SDPG_Foreground, LineWidth);
					PDI->DrawLine(PlaneVertexPositionD, PlaneVertexPositionA, LimitEdgeColor, SDPG_Foreground, LineWidth);
					PDI->DrawLine(PlaneVertexPositionA, PlaneVertexPositionC, LimitEdgeColor, SDPG_Foreground, LineWidth);
					PDI->DrawLine(PlaneVertexPositionD, PlaneVertexPositionB, LimitEdgeColor, SDPG_Foreground, LineWidth);

					DrawDirectionalArrow(PDI, FRotationMatrix(FRotator(90.0f, 0.0f, 0.0f)) * LimitPlaneTransform.ToMatrixNoScale(), LimitEdgeColor, 50.0f, 20.0f, SDPG_Foreground, LineWidth);
					PDI->SetHitProxy(nullptr);

					++LimitIndex;
				}
			}

			// Draw Spherical Limits.
			if (EditorAnimDynamicsNode->bShowSphericalLimit && ActivePreviewNode->SphericalLimits.Num() > 0)
			{
				uint32 LimitIndex = 0;

				for (const FAnimPhysSphericalLimit& SphericalLimit : ActivePreviewNode->SphericalLimits)
				{
					FTransform SphereTransform = FTransform::Identity;
					SphereTransform.SetTranslation(SphericalLimit.SphereLocalOffset);

					const int32 DrivingBoneIdx = PreviewSkelMeshComp->GetBoneIndex(SphericalLimit.DrivingBone.BoneName);

					if (DrivingBoneIdx != INDEX_NONE)
					{
						SphereTransform *= PreviewSkelMeshComp->GetComponentSpaceTransforms()[DrivingBoneIdx];
					}

					const FAnimDynamicsViewportObjectReference ViewportObjectRef(EditorAnimDynamicsNode->GetUniqueID(), FAnimDynamicsViewportObjectType::SphericalLimit, LimitIndex);
					const bool bIsSelected = SelectedViewportObjects.Contains(ViewportObjectRef);
					const float LineWidth = (bIsSelected) ? AnimDynamicsNodeConstants::SelectedShapeLineWidth : AnimDynamicsNodeConstants::ShapeLineWidth;

					PDI->SetHitProxy(new HAnimDynamicsEditModeHitProxy(ViewportObjectRef));
					DrawSphere(PDI, SphereTransform.GetLocation(), FRotator::ZeroRotator, FVector(SphericalLimit.LimitRadius), 24, 6, GEngine->ConstraintLimitMaterialPrismatic->GetRenderProxy(), SDPG_World);
					DrawWireSphere(PDI, SphereTransform, LimitEdgeColor, SphericalLimit.LimitRadius, 24, SDPG_World, LineWidth);
					PDI->SetHitProxy(nullptr);

					++LimitIndex;
				}
			}
		}
	}
}

bool FAnimDynamicsEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	const FViewport* const Viewport = InViewportClient->Viewport;

	const bool bIsCtrlKeyDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	const bool bIsShiftKeyDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
	const bool bIsAltKeyDown = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);
	bool bHandled = false;

	const bool bModifySelection = bIsCtrlKeyDown || bIsShiftKeyDown;

	const HAnimDynamicsEditModeHitProxy* const AnimDynamicsProxy = HitProxyCast<HAnimDynamicsEditModeHitProxy>(HitProxy);

	if (AnimDynamicsProxy)
	{
		const FAnimDynamicsViewportObjectReference HitObjectRef = AnimDynamicsProxy->ViewportObjectRef;

		if (!bModifySelection)
		{
			SelectedViewportObjects.Reset();
		}

		if (bModifySelection && SelectedViewportObjects.Contains(HitObjectRef))
		{
			SelectedViewportObjects.Remove(HitObjectRef);
		}
		else
		{
			SelectedViewportObjects.Emplace(HitObjectRef);
		}

		bHandled = true;
	}

	if (!bHandled)
	{
		bHandled = FEdMode::HandleClick(InViewportClient, HitProxy, Click);
	}

	if (!bHandled && !bModifySelection)
	{
		SelectedViewportObjects.Reset(); // Clear selection if Click on empty space.
		bHandled = true;
	}

	return bHandled;
}

ECoordSystem FAnimDynamicsEditMode::GetWidgetCoordinateSystem() const
{
	return COORD_Local;
}

FVector FAnimDynamicsEditMode::GetWidgetLocation() const
{
	return GetActiveViewportObjectTransform().GetTranslation();
}

bool FAnimDynamicsEditMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	InMatrix = GetActiveViewportObjectTransform().ToMatrixNoScale().RemoveTranslation();
	return true;
}


const bool FAnimDynamicsEditMode::IsValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	return (InWidgetMode == UE::Widget::WM_Scale) || (InWidgetMode == UE::Widget::WM_Translate);
}

UE::Widget::EWidgetMode FAnimDynamicsEditMode::GetNextWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	UE::Widget::EWidgetMode NextMode = UE::Widget::WM_Translate;

	if (InWidgetMode == UE::Widget::WM_Translate)
	{
		NextMode = UE::Widget::WM_Scale;
	}

	check(IsValidWidgetMode(NextMode));

	return NextMode;
}

UE::Widget::EWidgetMode FAnimDynamicsEditMode::FindValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	UE::Widget::EWidgetMode ValidMode = InWidgetMode;

	if (!IsValidWidgetMode(ValidMode))
	{
		ValidMode = GetNextWidgetMode(ValidMode);
	}

	check(IsValidWidgetMode(ValidMode));

	return ValidMode;
}

UE::Widget::EWidgetMode FAnimDynamicsEditMode::GetWidgetMode() const
{
	const USkeletalMeshComponent* const SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	const UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = CastChecked<UAnimGraphNode_AnimDynamics>(GetActiveWidgetAnimNode());

	if (SkelComp && EditorAnimDynamicsNode)
	{
		const int32 BoneIndex = SkelComp->GetBoneIndex(EditorAnimDynamicsNode->Node.BoundBone.BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			CurWidgetMode = FindValidWidgetMode(CurWidgetMode);
			return CurWidgetMode;
		}
	}

	return UE::Widget::WM_None;
}

UE::Widget::EWidgetMode FAnimDynamicsEditMode::ChangeToNextWidgetMode(UE::Widget::EWidgetMode InCurWidgetMode)
{
	UE::Widget::EWidgetMode NextWidgetMode = GetNextWidgetMode(InCurWidgetMode);
	CurWidgetMode = FindValidWidgetMode(NextWidgetMode);

	return CurWidgetMode;
}

bool FAnimDynamicsEditMode::SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode)
{
	// if InWidgetMode is available 
	if (IsValidWidgetMode(InWidgetMode))
	{
		CurWidgetMode = InWidgetMode;
		return true;
	}

	return false;
}

FName FAnimDynamicsEditMode::GetSelectedBone() const
{
	FName SelectedBoneName;

	if (SelectedViewportObjects.Num() > 0)
	{
		const uint32 Index = SelectedViewportObjects.Num() - 1;
		const FAnimDynamicsViewportObjectReference& SelectedObjectRef = SelectedViewportObjects[Index];
		const UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = FindSelectedEditorAnimNode(SelectedObjectRef.EditorNodeUniqueId);
		const FAnimNode_AnimDynamics* const ActivePreviewNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode();

		{
			if ((SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::BoxExtents) || (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::SphericalColisionVolume))
			{
				if (ActivePreviewNode->PhysicsBodyDefinitions.IsValidIndex(SelectedObjectRef.Index))
				{
					SelectedBoneName = ActivePreviewNode->PhysicsBodyDefinitions[SelectedObjectRef.Index].BoundBone.BoneName;
				}
			}
			else if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::PlaneLimit)
			{
				if (ActivePreviewNode->PlanarLimits.IsValidIndex(SelectedObjectRef.Index))
				{
					SelectedBoneName = ActivePreviewNode->PlanarLimits[SelectedObjectRef.Index].DrivingBone.BoneName;
				}
			}
			else if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::SphericalLimit)
			{
				if (ActivePreviewNode->SphericalLimits.IsValidIndex(SelectedObjectRef.Index))
				{
					SelectedBoneName = ActivePreviewNode->SphericalLimits[SelectedObjectRef.Index].DrivingBone.BoneName;
				}
			}
		}
	}

	return SelectedBoneName;
}

const UAnimGraphNode_AnimDynamics* const FAnimDynamicsEditMode::FindSelectedEditorAnimNode(const int32 InEditorNodeId) const
{
	const FAnimNodeEditMode::EditorRuntimeNodePair* const FoundNodePair = SelectedAnimNodes.FindByPredicate([InEditorNodeId](const FAnimNodeEditMode::EditorRuntimeNodePair& Element) { return Element.EditorAnimNode && Element.EditorAnimNode->GetUniqueID() == InEditorNodeId; });

	if (FoundNodePair)
	{
		return CastChecked< UAnimGraphNode_AnimDynamics >(FoundNodePair->EditorAnimNode);
	}

	return nullptr;
}

UAnimGraphNode_AnimDynamics* const FAnimDynamicsEditMode::FindSelectedEditorAnimNode(const int32 InEditorNodeId)
{
	const FAnimNodeEditMode::EditorRuntimeNodePair* const FoundNodePair = SelectedAnimNodes.FindByPredicate([InEditorNodeId](const FAnimNodeEditMode::EditorRuntimeNodePair& Element) { return Element.EditorAnimNode && Element.EditorAnimNode->GetUniqueID() == InEditorNodeId; });

	if (FoundNodePair)
	{
		return CastChecked< UAnimGraphNode_AnimDynamics >(FoundNodePair->EditorAnimNode);
	}

	return nullptr;
}

const FTransform FAnimDynamicsEditMode::GetActiveViewportObjectTransform() const
{
	return GetViewportObjectTransform(GetActiveViewportObject());
}

const FTransform FAnimDynamicsEditMode::GetViewportObjectTransform(const FAnimDynamicsViewportObjectReference* const SelectedObjectRef) const
{
	FTransform Transform;
	Transform.SetIdentity();

	if (SelectedObjectRef)
	{
		Transform = GetViewportObjectLocalSpaceTransform(SelectedObjectRef);
		const UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = FindSelectedEditorAnimNode(SelectedObjectRef->EditorNodeUniqueId);

		FName BoneName;

		if (EditorAnimDynamicsNode)
		{
			const FAnimNode_AnimDynamics* const ActivePreviewNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode();

			if (ActivePreviewNode)
			{
				if ((SelectedObjectRef->Type == FAnimDynamicsViewportObjectType::BoxExtents) || (SelectedObjectRef->Type == FAnimDynamicsViewportObjectType::SphericalColisionVolume))
				{
					if (ActivePreviewNode->PhysicsBodyDefinitions.IsValidIndex(SelectedObjectRef->Index))
					{
						Transform.SetTranslation(Transform.GetTranslation() + Transform.GetRotation().RotateVector(ActivePreviewNode->PhysicsBodyDefinitions[SelectedObjectRef->Index].LocalJointOffset));
					}
				}
				else if (SelectedObjectRef->Type == FAnimDynamicsViewportObjectType::PlaneLimit)
				{
					if (ActivePreviewNode->PlanarLimits.IsValidIndex(SelectedObjectRef->Index))
					{
						Transform = ActivePreviewNode->PlanarLimits[SelectedObjectRef->Index].PlaneTransform * Transform;
					}
				}
				else if (SelectedObjectRef->Type == FAnimDynamicsViewportObjectType::SphericalLimit)
				{
					if (ActivePreviewNode->SphericalLimits.IsValidIndex(SelectedObjectRef->Index))
					{
						Transform.SetTranslation(Transform.GetTranslation() + Transform.GetRotation().RotateVector(ActivePreviewNode->SphericalLimits[SelectedObjectRef->Index].SphereLocalOffset));
					}
				}
			}
		}
	}


	return Transform;
}

const FTransform FAnimDynamicsEditMode::GetViewportObjectLocalSpaceTransform(const FAnimDynamicsViewportObjectReference* const SelectedObjectRef) const
{
	FTransform Transform;
	Transform.SetIdentity();

	const USkeletalMeshComponent* const SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	if (SelectedObjectRef && SkelComp)
	{
		const FName BoneName = GetSelectedBone();
		const int32 BoneIndex = SkelComp->GetBoneIndex(BoneName);

		if (BoneIndex != INDEX_NONE)
		{
			Transform = SkelComp->GetComponentSpaceTransforms()[BoneIndex];
		}
	}

	return Transform;
}

void FAnimDynamicsEditMode::DoTranslation(FVector& InTranslation)
{
	const USkeletalMeshComponent* const SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	if (SkelComp)
	{
		// Find widget translation in the space of the selected objects associated bone.
		const FTransform LocalSpaceTransform = GetViewportObjectLocalSpaceTransform(GetActiveViewportObject());
		const FVector LocalSpaceTranslation = LocalSpaceTransform.GetRotation().UnrotateVector(InTranslation);

		// Apply local space widget translation to all selected nodes. This means the collision volumes will each move in the same direction 
		// in their own space, potentially different directions in world space.
		for (const FAnimDynamicsViewportObjectReference& SelectedObjectRef : SelectedViewportObjects)
		{
			UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = FindSelectedEditorAnimNode(SelectedObjectRef.EditorNodeUniqueId);
			if (EditorAnimDynamicsNode)
			{
				FAnimNode_AnimDynamics* const RuntimeAnimDynamicsNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode();
				if (RuntimeAnimDynamicsNode)
				{

					if ((SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::BoxExtents) || (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::SphericalColisionVolume))
					{
						if (RuntimeAnimDynamicsNode->PhysicsBodyDefinitions.IsValidIndex(SelectedObjectRef.Index))
						{
							FVector& PhysicsBodyJointOffset = RuntimeAnimDynamicsNode->PhysicsBodyDefinitions[SelectedObjectRef.Index].LocalJointOffset;

							PhysicsBodyJointOffset += LocalSpaceTranslation;
							EditorAnimDynamicsNode->Node.PhysicsBodyDefinitions[SelectedObjectRef.Index].LocalJointOffset = PhysicsBodyJointOffset;
						}
					}
					else if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::PlaneLimit)
					{
						if (RuntimeAnimDynamicsNode->PlanarLimits.IsValidIndex(SelectedObjectRef.Index))
						{
							FTransform& LimitTransform = RuntimeAnimDynamicsNode->PlanarLimits[SelectedObjectRef.Index].PlaneTransform;
							LimitTransform.SetTranslation(LimitTransform.GetTranslation() + LocalSpaceTranslation);
							EditorAnimDynamicsNode->Node.PlanarLimits[SelectedObjectRef.Index].PlaneTransform = LimitTransform;
						}
					}
					else if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::SphericalLimit)
					{
						if (RuntimeAnimDynamicsNode->SphericalLimits.IsValidIndex(SelectedObjectRef.Index))
						{

							FVector& LimitOffset = RuntimeAnimDynamicsNode->SphericalLimits[SelectedObjectRef.Index].SphereLocalOffset;
							LimitOffset += LocalSpaceTranslation;
							EditorAnimDynamicsNode->Node.SphericalLimits[SelectedObjectRef.Index].SphereLocalOffset = LimitOffset;
						}
					}
				}
			}
		}
	}

	bIsInteractingWithWidget = true;
}

void FAnimDynamicsEditMode::DoRotation(FRotator& InRotation)
{
	const USkeletalMeshComponent* const SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	if (SkelComp)
	{
		// Transform rotation to the active object's local space.
		FQuat LocalSpaceRotation(FQuat::Identity);

		{
			FVector RotAxis;
			float RotAngle;
			InRotation.Quaternion().ToAxisAndAngle(RotAxis, RotAngle);
			const FTransform SelectedBoneTM = GetViewportObjectLocalSpaceTransform(GetActiveViewportObject());
			RotAxis = SelectedBoneTM.Inverse().TransformVector(RotAxis);
			FQuat DeltaQuat(RotAxis, RotAngle);
			DeltaQuat.Normalize();
			LocalSpaceRotation = DeltaQuat;
		}

		// Apply local rotation to all selected objects.
		for (const FAnimDynamicsViewportObjectReference& SelectedObjectRef : SelectedViewportObjects)
		{
			UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = FindSelectedEditorAnimNode(SelectedObjectRef.EditorNodeUniqueId);
			if (EditorAnimDynamicsNode)
			{
				FAnimNode_AnimDynamics* const RuntimeAnimDynamicsNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode();
				if (RuntimeAnimDynamicsNode)
				{
					if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::PlaneLimit)
					{
						if (RuntimeAnimDynamicsNode->PlanarLimits.IsValidIndex(SelectedObjectRef.Index))
						{
							FTransform& LimitTransform = RuntimeAnimDynamicsNode->PlanarLimits[SelectedObjectRef.Index].PlaneTransform;
							LimitTransform.SetRotation(LocalSpaceRotation * LimitTransform.GetRotation());
							EditorAnimDynamicsNode->Node.PlanarLimits[SelectedObjectRef.Index].PlaneTransform = LimitTransform;
						}
					}
				}
			}
		}
	}
}

void FAnimDynamicsEditMode::DoScale(FVector& InScale)
{
	const float Scale = static_cast<float>(InScale.X + InScale.Y + InScale.Z);

	for (const FAnimDynamicsViewportObjectReference& SelectedObjectRef : SelectedViewportObjects)
	{
		UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = FindSelectedEditorAnimNode(SelectedObjectRef.EditorNodeUniqueId);
		if (EditorAnimDynamicsNode)
		{
			FAnimNode_AnimDynamics* const RuntimeAnimDynamicsNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode();
			if (RuntimeAnimDynamicsNode)
			{
				if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::BoxExtents)
				{
					if (RuntimeAnimDynamicsNode->PhysicsBodyDefinitions.IsValidIndex(SelectedObjectRef.Index))
					{
						FVector& PhysicsBodyBoxExtents = RuntimeAnimDynamicsNode->PhysicsBodyDefinitions[SelectedObjectRef.Index].BoxExtents;
						PhysicsBodyBoxExtents += InScale;
						EditorAnimDynamicsNode->Node.PhysicsBodyDefinitions[SelectedObjectRef.Index].BoxExtents = PhysicsBodyBoxExtents;
					}
				}
				else if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::SphericalColisionVolume)
				{
					if (RuntimeAnimDynamicsNode->PhysicsBodyDefinitions.IsValidIndex(SelectedObjectRef.Index))
					{
						float& PhysicsBodySphereCollisionRadius = RuntimeAnimDynamicsNode->PhysicsBodyDefinitions[SelectedObjectRef.Index].SphereCollisionRadius;

						PhysicsBodySphereCollisionRadius += Scale;
						EditorAnimDynamicsNode->Node.PhysicsBodyDefinitions[SelectedObjectRef.Index].SphereCollisionRadius = PhysicsBodySphereCollisionRadius;
					}
				}
				else if (SelectedObjectRef.Type == FAnimDynamicsViewportObjectType::SphericalLimit)
				{
					if (RuntimeAnimDynamicsNode->SphericalLimits.IsValidIndex(SelectedObjectRef.Index))
					{
						const float LimitRadius = RuntimeAnimDynamicsNode->SphericalLimits[SelectedObjectRef.Index].LimitRadius + Scale;

						RuntimeAnimDynamicsNode->SphericalLimits[SelectedObjectRef.Index].LimitRadius = LimitRadius;
						EditorAnimDynamicsNode->Node.SphericalLimits[SelectedObjectRef.Index].LimitRadius = LimitRadius;
					}
				}
			}
		}
	}

	bIsInteractingWithWidget = true;
}

void FAnimDynamicsEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FAnimNodeEditMode::Tick(ViewportClient, DeltaTime);

	// Keep this flag set until the mouse is released after moving the translation widget. DoTranslation / Scale fns are only called when 
	// the widget actually moves but IsManipulatingWidget() rtns true whenever the input button is pressed. This logic prevents the physics 
	// simulation from stating/stopping unexpectedly when the widget is being manipulated but hasn't moved and avoids stopping the simulation 
	// every time a mouse button is pressed over the viewport.
	bIsInteractingWithWidget = IsManipulatingWidget() && bIsInteractingWithWidget;

	USkeletalMeshComponent* const PreviewSkelMeshComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	for (const EditorRuntimeNodePair& CurrentNodePair : SelectedAnimNodes)
	{
		const UAnimGraphNode_AnimDynamics* const EditorAnimDynamicsNode = CastChecked<UAnimGraphNode_AnimDynamics>(CurrentNodePair.EditorAnimNode);

		if (EditorAnimDynamicsNode)
		{
			FAnimNode_AnimDynamics* const ActivePreviewNode = EditorAnimDynamicsNode->GetPreviewDynamicsNode();

			if (ActivePreviewNode)
			{
				// Disable physics simulation when physics parameters are being edited via a widget, otherwise editing becomes 
				// very difficult as the objects are moving and the simulation is liable to become unstable.
				if (bIsInteractingWithWidget)
				{
					ActivePreviewNode->bDoPhysicsUpdateInEditor = false;
				}
				else if (!ActivePreviewNode->bDoPhysicsUpdateInEditor)
				{
					ActivePreviewNode->RequestInitialise(ETeleportType::ResetPhysics);
					ActivePreviewNode->bDoPhysicsUpdateInEditor = true;
				}
			}
		}
	}
}

bool FAnimDynamicsEditMode::ShouldDrawWidget() const
{
	return SelectedViewportObjects.Num() > 0; // Only draw a widget if at least one viewport object is selected.
}

const FAnimDynamicsViewportObjectReference* const FAnimDynamicsEditMode::GetActiveViewportObject() const
{
	if (SelectedViewportObjects.Num() > 0)
	{
		return &SelectedViewportObjects[SelectedViewportObjects.Num() - 1];
	}

	return nullptr;
}

