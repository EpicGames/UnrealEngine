// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeEditMode.h"
#include "Containers/Array.h"
#include "Math/Matrix.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/NameTypes.h"
#include "UnrealWidgetFwd.h"

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class HHitProxy;
struct FViewportClick;

enum class FAnimDynamicsViewportObjectType { PlaneLimit, SphericalLimit, SphericalColisionVolume, BoxExtents };

// FAnimDynamicsViewportObjectReference
//
// Used to identify a viewport editable structure (i.e. a physics object joint offset) in an AnimDynamics node.
struct FAnimDynamicsViewportObjectReference
{
	FAnimDynamicsViewportObjectReference(const uint32 InEditorNodeUniqueId, const FAnimDynamicsViewportObjectType InType, const uint32 Index);
	
	uint32 EditorNodeUniqueId;
	FAnimDynamicsViewportObjectType Type;
	uint32 Index;
};

const bool operator==(const FAnimDynamicsViewportObjectReference& Lhs, const FAnimDynamicsViewportObjectReference& Rhs);

class FAnimDynamicsEditMode : public FAnimNodeEditMode
{
public:
	FAnimDynamicsEditMode();

	/** IAnimNodeEditMode interface */
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual void ExitMode() override;
	virtual ECoordSystem GetWidgetCoordinateSystem() const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual UE::Widget::EWidgetMode ChangeToNextWidgetMode(UE::Widget::EWidgetMode InCurWidgetMode) override;
	virtual bool SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode) override;
	virtual FName GetSelectedBone() const override;
	virtual void DoTranslation(FVector& InTranslation) override;
	virtual void DoRotation(FRotator& InRotation) override;
	virtual void DoScale(FVector& InScale) override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual bool ShouldDrawWidget() const override;

private:
	UE::Widget::EWidgetMode FindValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const;
	const class UAnimGraphNode_AnimDynamics* const FindSelectedEditorAnimNode(const int32 InEditorNodeId) const;
	class UAnimGraphNode_AnimDynamics* const FindSelectedEditorAnimNode(const int32 InEditorNodeId);
	const bool IsValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const;
	UE::Widget::EWidgetMode GetNextWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const;
	
	const FTransform GetActiveViewportObjectTransform() const;
	const FTransform GetViewportObjectTransform(const FAnimDynamicsViewportObjectReference* const SelectedObjectRef) const;
	const FTransform GetViewportObjectLocalSpaceTransform(const FAnimDynamicsViewportObjectReference* const SelectedObjectRef) const;
	const FAnimDynamicsViewportObjectReference* const GetActiveViewportObject() const;

private:

	TArray< FAnimDynamicsViewportObjectReference > SelectedViewportObjects;

	mutable UE::Widget::EWidgetMode CurWidgetMode;

	// Set true every frame where DoTranslation fn is called, reset in Tick fn.
	bool bIsInteractingWithWidget;
};
