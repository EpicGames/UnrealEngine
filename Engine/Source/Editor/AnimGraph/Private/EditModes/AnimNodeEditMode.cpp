// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodeEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "Animation/BoneSocketReference.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/Skeleton.h"
#include "AssetEditorModeManager.h"
#include "BoneContainer.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "BoneIndices.h"
#include "BonePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/UnrealString.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Engine/SkeletalMesh.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "HAL/PlatformCrt.h"
#include "HitProxies.h"
#include "IPersonaPreviewScene.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Axis.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "ReferenceSkeleton.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UnrealNames.h"
#include "UnrealClient.h"

class FSceneView;
class FText;
struct FAnimNode_Base;

#define LOCTEXT_NAMESPACE "AnimNodeEditMode"

const bool operator==(const FAnimNodeEditMode::EditorRuntimeNodePair& Lhs, const FAnimNodeEditMode::EditorRuntimeNodePair& Rhs)
{
	return (Lhs.EditorAnimNode == Rhs.EditorAnimNode) && (Lhs.RuntimeAnimNode == Rhs.RuntimeAnimNode);
}

FAnimNodeEditMode::FAnimNodeEditMode()
	: bManipulating(false)
	, bInTransaction(false)
{
	// Disable grid drawing for this mode as the viewport handles this
	bDrawGrid = false;
}

bool FAnimNodeEditMode::GetCameraTarget(FSphere& OutTarget) const
{
	FVector Location(GetWidgetLocation());
	OutTarget.Center = Location;
	OutTarget.W = 50.0f;

	return true;
}

IPersonaPreviewScene& FAnimNodeEditMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FAnimNodeEditMode::GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const
{
	for (EditorRuntimeNodePair CurrentNodePair : SelectedAnimNodes)
	{
		if ((CurrentNodePair.EditorAnimNode != nullptr) && (CurrentNodePair.RuntimeAnimNode != nullptr))
		{
			CurrentNodePair.EditorAnimNode->GetOnScreenDebugInfo(OutDebugInfo, CurrentNodePair.RuntimeAnimNode, GetAnimPreviewScene().GetPreviewMeshComponent());
		}
	}
}

ECoordSystem FAnimNodeEditMode::GetWidgetCoordinateSystem() const
{
	UAnimGraphNode_SkeletalControlBase* SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	if (SkelControl != nullptr)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return (ECoordSystem)SkelControl->GetWidgetCoordinateSystem(GetAnimPreviewScene().GetPreviewMeshComponent());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return ECoordSystem::COORD_None;
}

UE::Widget::EWidgetMode FAnimNodeEditMode::GetWidgetMode() const
{
	UAnimGraphNode_SkeletalControlBase* SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	if (SkelControl != nullptr)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return (UE::Widget::EWidgetMode)SkelControl->GetWidgetMode(GetAnimPreviewScene().GetPreviewMeshComponent());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return UE::Widget::EWidgetMode::WM_None;
}

UE::Widget::EWidgetMode FAnimNodeEditMode::ChangeToNextWidgetMode(UE::Widget::EWidgetMode CurWidgetMode)
{
	UAnimGraphNode_SkeletalControlBase* SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	if (SkelControl != nullptr)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return (UE::Widget::EWidgetMode)SkelControl->ChangeToNextWidgetMode(GetAnimPreviewScene().GetPreviewMeshComponent(), CurWidgetMode);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return UE::Widget::EWidgetMode::WM_None;
}

bool FAnimNodeEditMode::SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode)
{
	UAnimGraphNode_SkeletalControlBase* SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	if (SkelControl != nullptr)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SkelControl->SetWidgetMode(GetAnimPreviewScene().GetPreviewMeshComponent(), InWidgetMode);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return false;
}

FName FAnimNodeEditMode::GetSelectedBone() const
{
	UAnimGraphNode_SkeletalControlBase* const SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	if (SkelControl != nullptr)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return SkelControl->FindSelectedBone();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return NAME_None;
}

void FAnimNodeEditMode::EnterMode(UAnimGraphNode_Base* InEditorNode, FAnimNode_Base* InRuntimeNode)
{
	check(InEditorNode && InRuntimeNode); // Expect valid Node ptrs.

	if (InEditorNode && InRuntimeNode)
	{
		SelectedAnimNodes.Add(EditorRuntimeNodePair(InEditorNode, InRuntimeNode));

		UAnimGraphNode_SkeletalControlBase* SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(InEditorNode);
		if (SkelControl != nullptr)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			SkelControl->MoveSelectActorLocation(GetAnimPreviewScene().GetPreviewMeshComponent(), static_cast<FAnimNode_SkeletalControlBase*>(InRuntimeNode));
			SkelControl->CopyNodeDataTo(InRuntimeNode);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	GetModeManager()->SetCoordSystem(GetWidgetCoordinateSystem());
	GetModeManager()->SetWidgetMode(GetWidgetMode());
}

void FAnimNodeEditMode::ExitMode()
{
	for (EditorRuntimeNodePair CurrentNodePair : SelectedAnimNodes)
	{
		if (CurrentNodePair.EditorAnimNode != nullptr)
		{
			UAnimGraphNode_SkeletalControlBase* SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(CurrentNodePair.EditorAnimNode);
			if (SkelControl != nullptr)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				SkelControl->DeselectActor(GetAnimPreviewScene().GetPreviewMeshComponent());
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}

	SelectedAnimNodes.Empty();
	PoseWatchedAnimNodes.Empty();
}

void FAnimNodeEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	USkeletalMeshComponent* const PreviewSkelMeshComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	check(View);
	check(PDI);
	check(PreviewSkelMeshComp);

	// Build a unique list of all selected or pose watched nodes, 0 = Node, 1 = IsSelected, 2 = IsPoseWatchEnabled. 
	using DrawParameters = TTuple<UAnimGraphNode_Base*, bool, bool>;

	TArray< DrawParameters > DrawParameterList;

	// Collect selected nodes.	
	for (const FAnimNodeEditMode::EditorRuntimeNodePair& CurrentNodePair : SelectedAnimNodes)
	{
		if (CurrentNodePair.EditorAnimNode != nullptr)
		{
			DrawParameterList.Add(DrawParameters(CurrentNodePair.EditorAnimNode, true, false));
		}
	}

	// Collect pose watched nodes.
	for (const FAnimNodeEditMode::EditorRuntimeNodePair& CurrentNodePair : PoseWatchedAnimNodes)
	{
		if (CurrentNodePair.EditorAnimNode != nullptr)
		{
			if (DrawParameters* Parameter = DrawParameterList.FindByPredicate([CurrentNodePair](DrawParameters& Other) { return Other.Get<0>() == CurrentNodePair.EditorAnimNode; }))
			{
				// Add pose watch flag to existing, selected node's draw parameters.
				Parameter->Get<2>() = true;
			}
			else
			{
				// Create a new draw parameter for this pose watched but not selected node.
				DrawParameterList.Add(DrawParameters(CurrentNodePair.EditorAnimNode, false, true));
			}
		}
	}

	// Draw all collected nodes.
	for (DrawParameters CurrentParameters : DrawParameterList)
	{
		CurrentParameters.Get<0>()->Draw(PDI, PreviewSkelMeshComp, CurrentParameters.Get<1>(), CurrentParameters.Get<2>());
	}
}

void FAnimNodeEditMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	for (EditorRuntimeNodePair CurrentNodePair : SelectedAnimNodes)
	{
		if (CurrentNodePair.EditorAnimNode != nullptr)
		{
			CurrentNodePair.EditorAnimNode->DrawCanvas(*Viewport, *const_cast<FSceneView*>(View), *Canvas, GetAnimPreviewScene().GetPreviewMeshComponent());
		}
	}
}

bool FAnimNodeEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (HitProxy != nullptr && HitProxy->IsA(HActor::StaticGetType()))
	{
		HActor* ActorHitProxy = static_cast<HActor*>(HitProxy);
		GetAnimPreviewScene().SetSelectedActor(ActorHitProxy->Actor);

		for (EditorRuntimeNodePair CurrentNodePair : SelectedAnimNodes)
		{
			UAnimGraphNode_SkeletalControlBase* SkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(CurrentNodePair.EditorAnimNode);
			if (SkelControl != nullptr)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				SkelControl->ProcessActorClick(ActorHitProxy);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		return true;
	}

	return false;
}

FVector FAnimNodeEditMode::GetWidgetLocation() const
{
	UAnimGraphNode_SkeletalControlBase* const EditorSkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	FAnimNode_SkeletalControlBase* const RuntimeSkelControl = static_cast<FAnimNode_SkeletalControlBase*>(GetActiveWidgetRuntimeAnimNode());
	if ((EditorSkelControl != nullptr) && (RuntimeSkelControl != nullptr))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return EditorSkelControl->GetWidgetLocation(GetAnimPreviewScene().GetPreviewMeshComponent(), RuntimeSkelControl);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return FVector::ZeroVector;
}

bool FAnimNodeEditMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	const EAxisList::Type CurrentAxis = InViewportClient ? InViewportClient->GetCurrentWidgetAxis() : EAxisList::None;
	const UE::Widget::EWidgetMode WidgetMode = InViewportClient ? InViewportClient->GetWidgetMode() : UE::Widget::WM_None;

	if (WidgetMode != UE::Widget::WM_None && CurrentAxis != EAxisList::None)
	{
		return HandleBeginTransform();
	}
	
	return IAnimNodeEditMode::StartTracking(InViewportClient, InViewport);
}

bool FAnimNodeEditMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (HandleEndTransform())
	{
		return true;
	}

	return IAnimNodeEditMode::EndTracking(InViewportClient, InViewport);
}

bool FAnimNodeEditMode::BeginTransform(const FGizmoState& InState)
{
	return HandleBeginTransform();
}

bool FAnimNodeEditMode::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform();
}

bool FAnimNodeEditMode::HandleBeginTransform()
{
	if (!bInTransaction)
	{
		GEditor->BeginTransaction(LOCTEXT("EditSkelControlNodeTransaction", "Edit Skeletal Control Node"));

		for (EditorRuntimeNodePair CurrentNodePair : SelectedAnimNodes)
		{
			if (CurrentNodePair.EditorAnimNode != nullptr)
			{
				CurrentNodePair.EditorAnimNode->SetFlags(RF_Transactional);
				CurrentNodePair.EditorAnimNode->Modify();
			}
		}

		bInTransaction = true;
	}

	bManipulating = true;

	return true;
}

bool FAnimNodeEditMode::HandleEndTransform()
{
	if (bManipulating)
	{
		bManipulating = false;

		if (bInTransaction)
		{
			GEditor->EndTransaction();
			bInTransaction = false;
		}

		return true;
	}
	return false;
}

bool FAnimNodeEditMode::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent)
{
	bool bHandled = false;

	// Handle switching modes - only allowed when not already manipulating
	if ((InEvent == IE_Pressed) && (InKey == EKeys::SpaceBar) && !bManipulating)
	{
		UE::Widget::EWidgetMode WidgetMode = (UE::Widget::EWidgetMode)ChangeToNextWidgetMode(GetModeManager()->GetWidgetMode());
		GetModeManager()->SetWidgetMode(WidgetMode);
		if (WidgetMode == UE::Widget::WM_Scale)
		{
			GetModeManager()->SetCoordSystem(COORD_Local);
		}
		else
		{
			GetModeManager()->SetCoordSystem(COORD_World);
		}

		bHandled = true;
		InViewportClient->Invalidate();
	}

	return bHandled;
}

bool FAnimNodeEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	const UE::Widget::EWidgetMode WidgetMode = InViewportClient->GetWidgetMode();

	bool bHandled = false;

	if (bManipulating && CurrentAxis != EAxisList::None)
	{
		bHandled = true;

		const bool bDoRotation = WidgetMode == UE::Widget::WM_Rotate || WidgetMode == UE::Widget::WM_TranslateRotateZ;
		const bool bDoTranslation = WidgetMode == UE::Widget::WM_Translate || WidgetMode == UE::Widget::WM_TranslateRotateZ;
		const bool bDoScale = WidgetMode == UE::Widget::WM_Scale;

		if (bDoRotation)
		{
			DoRotation(InRot);
		}

		if (bDoTranslation)
		{
			DoTranslation(InDrag);
		}

		if (bDoScale)
		{
			DoScale(InScale);
		}

		for (EditorRuntimeNodePair CurrentNodePair : SelectedAnimNodes)
		{
			if (CurrentNodePair.EditorAnimNode)
			{
				CurrentNodePair.EditorAnimNode->PostEditRefreshDebuggedComponent();
			}
		}

		InViewport->Invalidate();
	}

	return bHandled;
}

bool FAnimNodeEditMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	FName BoneName = GetSelectedBone();
	int32 BoneIndex = PreviewMeshComponent->GetBoneIndex(BoneName);
	if (BoneIndex != INDEX_NONE)
	{
		FTransform BoneMatrix = PreviewMeshComponent->GetBoneTransform(BoneIndex);
		InMatrix = BoneMatrix.ToMatrixNoScale().RemoveTranslation();
		return true;
	}

	return false;
}

bool FAnimNodeEditMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

bool FAnimNodeEditMode::ShouldDrawWidget() const
{
	return true;
}

void FAnimNodeEditMode::DoTranslation(FVector& InTranslation)
{
	UAnimGraphNode_SkeletalControlBase* const EditorSkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	FAnimNode_SkeletalControlBase* const RuntimeSkelControl = static_cast<FAnimNode_SkeletalControlBase*>(GetActiveWidgetRuntimeAnimNode());
	if ((EditorSkelControl != nullptr) && (RuntimeSkelControl != nullptr))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EditorSkelControl->DoTranslation(GetAnimPreviewScene().GetPreviewMeshComponent(), InTranslation, RuntimeSkelControl);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FAnimNodeEditMode::DoRotation(FRotator& InRotation)
{
	UAnimGraphNode_SkeletalControlBase* const EditorSkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	FAnimNode_SkeletalControlBase* const RuntimeSkelControl = static_cast<FAnimNode_SkeletalControlBase*>(GetActiveWidgetRuntimeAnimNode());
	if ((EditorSkelControl != nullptr) && (RuntimeSkelControl != nullptr))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EditorSkelControl->DoRotation(GetAnimPreviewScene().GetPreviewMeshComponent(), InRotation, RuntimeSkelControl);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FAnimNodeEditMode::DoScale(FVector& InScale)
{
	UAnimGraphNode_SkeletalControlBase* const EditorSkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	FAnimNode_SkeletalControlBase* const RuntimeSkelControl = static_cast<FAnimNode_SkeletalControlBase*>(GetActiveWidgetRuntimeAnimNode());
	if ((EditorSkelControl != nullptr) && (RuntimeSkelControl != nullptr))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EditorSkelControl->DoScale(GetAnimPreviewScene().GetPreviewMeshComponent(), InScale, RuntimeSkelControl);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FAnimNodeEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	IAnimNodeEditMode::Tick(ViewportClient, DeltaTime);

	// Keep actor location in sync with animation
	UAnimGraphNode_SkeletalControlBase* const EditorSkelControl = Cast<UAnimGraphNode_SkeletalControlBase>(GetActiveWidgetAnimNode());
	FAnimNode_SkeletalControlBase* const RuntimeSkelControl = static_cast<FAnimNode_SkeletalControlBase*>(GetActiveWidgetRuntimeAnimNode());
	if ((EditorSkelControl != nullptr) && (RuntimeSkelControl != nullptr))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EditorSkelControl->MoveSelectActorLocation(GetAnimPreviewScene().GetPreviewMeshComponent(), RuntimeSkelControl);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FAnimNodeEditMode::Exit()
{
	IAnimNodeEditMode::Exit();

	SelectedAnimNodes.Empty();
}

void FAnimNodeEditMode::RegisterPoseWatchedNode(UAnimGraphNode_Base* InEditorNode, FAnimNode_Base* InRuntimeNode)
{
	PoseWatchedAnimNodes.Add(FAnimNodeEditMode::EditorRuntimeNodePair(InEditorNode, InRuntimeNode));
}

UAnimGraphNode_Base* FAnimNodeEditMode::GetActiveWidgetAnimNode() const
{
	if (SelectedAnimNodes.Num() > 0)
	{
		return SelectedAnimNodes.Last().EditorAnimNode;
	}

	return nullptr;
}

FAnimNode_Base* FAnimNodeEditMode::GetActiveWidgetRuntimeAnimNode() const
{
	if (SelectedAnimNodes.Num() > 0)
	{
		return SelectedAnimNodes.Last().RuntimeAnimNode;
	}

	return nullptr;
}

void FAnimNodeEditMode::ConvertToComponentSpaceTransform(const USkeletalMeshComponent* SkelComp, const FTransform & InTransform, FTransform & OutCSTransform, int32 BoneIndex, EBoneControlSpace Space)
{
	USkeletalMesh* SkelMesh = SkelComp->GetSkeletalMeshAsset();
	USkeleton* Skeleton = SkelMesh->GetSkeleton();

	switch (Space)
	{
	case BCS_WorldSpace:
	{
		OutCSTransform = InTransform;
		OutCSTransform.SetToRelativeTransform(SkelComp->GetComponentTransform());
	}
	break;

	case BCS_ComponentSpace:
	{
		// Component Space, no change.
		OutCSTransform = InTransform;
	}
	break;

	case BCS_ParentBoneSpace:
		if (BoneIndex != INDEX_NONE)
		{
			const int32 ParentIndex = Skeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				const int32 MeshParentIndex = Skeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkelMesh, ParentIndex);
				if (MeshParentIndex != INDEX_NONE)
				{
					const FTransform ParentTM = SkelComp->GetBoneTransform(MeshParentIndex);
					OutCSTransform = InTransform * ParentTM;
				}
				else
				{
					OutCSTransform = InTransform;
				}
			}
		}
		break;

	case BCS_BoneSpace:
		if (BoneIndex != INDEX_NONE)
		{
			const int32 MeshBoneIndex = Skeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkelMesh, BoneIndex);
			if (MeshBoneIndex != INDEX_NONE)
			{
				const FTransform BoneTM = SkelComp->GetBoneTransform(MeshBoneIndex);
				OutCSTransform = InTransform * BoneTM;
			}
			else
			{
				OutCSTransform = InTransform;
			}
		}
		break;

	default:
		if (SkelMesh)
		{
			UE_LOGF(LogAnimation, Warning, "ConvertToComponentSpaceTransform: Unknown BoneSpace %d  for Mesh: %ls", (uint8)Space, *SkelMesh->GetFName().ToString());
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "ConvertToComponentSpaceTransform: Unknown BoneSpace %d  for Skeleton: %ls", (uint8)Space, *Skeleton->GetFName().ToString());
		}
		break;
	}
}


void FAnimNodeEditMode::ConvertToBoneSpaceTransform(const USkeletalMeshComponent* SkelComp, const FTransform & InCSTransform, FTransform & OutBSTransform, int32 BoneIndex, EBoneControlSpace Space)
{
	USkeletalMesh* SkelMesh = SkelComp->GetSkeletalMeshAsset();
	USkeleton* Skeleton = SkelMesh->GetSkeleton();

	switch(Space)
	{
		case BCS_WorldSpace:
		{
			OutBSTransform = InCSTransform * SkelComp->GetComponentTransform();
			break;
		}
		
		case BCS_ComponentSpace:
		{
			// Component Space, no change.
			OutBSTransform = InCSTransform;
			break;
		}

		case BCS_ParentBoneSpace:
		{
			if(BoneIndex != INDEX_NONE)
			{
				const int32 ParentIndex = Skeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);
				if(ParentIndex != INDEX_NONE)
				{
					const int32 MeshParentIndex = Skeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkelMesh, ParentIndex);
					if(MeshParentIndex != INDEX_NONE)
					{
						const FTransform ParentTM = SkelComp->GetBoneTransform(MeshParentIndex);
						OutBSTransform = InCSTransform.GetRelativeTransform(ParentTM);
					}
					else
					{
						OutBSTransform = InCSTransform;
					}
				}
			}
			break;
		}

		case BCS_BoneSpace:
		{
			if(BoneIndex != INDEX_NONE)
			{
				const int32 MeshBoneIndex = Skeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkelMesh, BoneIndex);
				if(MeshBoneIndex != INDEX_NONE)
				{
					FTransform BoneCSTransform = SkelComp->GetBoneTransform(MeshBoneIndex);
					OutBSTransform = InCSTransform.GetRelativeTransform(BoneCSTransform);
				}
				else
				{
					OutBSTransform = InCSTransform;
				}
			}
			break;
		}

		default:
		{
			UE_LOGF(LogAnimation, Warning, "ConvertToBoneSpaceTransform: Unknown BoneSpace %d  for Mesh: %ls", (int32)Space, *GetNameSafe(SkelMesh));
			break;
		}
	}
}

FVector FAnimNodeEditMode::ConvertCSVectorToBoneSpace(const USkeletalMeshComponent* SkelComp, FVector& InCSVector, FCSPose<FCompactHeapPose>& MeshBases, const FBoneSocketTarget& InTarget, const EBoneControlSpace Space)
{
	FVector OutVector = FVector::ZeroVector;

	if (MeshBases.GetPose().IsValid())
	{
		const FCompactPoseBoneIndex BoneIndex = InTarget.GetCompactPoseBoneIndex();

		switch (Space)
		{
			// World Space, no change in preview window
		case BCS_WorldSpace:
		case BCS_ComponentSpace:
			// Component Space, no change.
			OutVector = InCSVector;
			break;

		case BCS_ParentBoneSpace:
		{
			if (BoneIndex != INDEX_NONE)
			{
				const FCompactPoseBoneIndex ParentIndex = MeshBases.GetPose().GetParentBoneIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(ParentIndex);
					OutVector = ParentTM.InverseTransformVector(InCSVector);
				}
			}
		}
		break;

		case BCS_BoneSpace:
		{
			FTransform BoneTransform = InTarget.GetTargetTransform(FVector::ZeroVector, MeshBases, SkelComp->GetComponentToWorld());
			OutVector = BoneTransform.InverseTransformVector(InCSVector);
		}
		break;
		}
	}

	return OutVector;
}

FVector FAnimNodeEditMode::ConvertCSVectorToBoneSpace(const USkeletalMeshComponent* SkelComp, FVector& InCSVector, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space)
{
	FVector OutVector = FVector::ZeroVector;

	if (MeshBases.GetPose().IsValid())
	{
		const FMeshPoseBoneIndex MeshBoneIndex(SkelComp->GetBoneIndex(BoneName));
		const FCompactPoseBoneIndex BoneIndex = MeshBases.GetPose().GetBoneContainer().MakeCompactPoseIndex(MeshBoneIndex);

		switch (Space)
		{
			// World Space, no change in preview window
		case BCS_WorldSpace:
		case BCS_ComponentSpace:
			// Component Space, no change.
			OutVector = InCSVector;
			break;

		case BCS_ParentBoneSpace:
		{
			const FCompactPoseBoneIndex ParentIndex = MeshBases.GetPose().GetParentBoneIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(ParentIndex);
				OutVector = ParentTM.InverseTransformVector(InCSVector);
			}
		}
		break;

		case BCS_BoneSpace:
		{
			if (BoneIndex != INDEX_NONE)
			{
				const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(BoneIndex);
				OutVector = BoneTM.InverseTransformVector(InCSVector);
			}
		}
		break;
		}
	}

	return OutVector;
}

FQuat FAnimNodeEditMode::ConvertCSRotationToBoneSpace(const USkeletalMeshComponent* SkelComp, FRotator& InCSRotator, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space)
{
	FQuat OutQuat = FQuat::Identity;

	if (MeshBases.GetPose().IsValid())
	{
		const FMeshPoseBoneIndex MeshBoneIndex(SkelComp->GetBoneIndex(BoneName));
		const FCompactPoseBoneIndex BoneIndex = MeshBases.GetPose().GetBoneContainer().MakeCompactPoseIndex(MeshBoneIndex);

		FVector RotAxis;
		float RotAngle;
		InCSRotator.Quaternion().ToAxisAndAngle(RotAxis, RotAngle);

		switch (Space)
		{
			// World Space, no change in preview window
		case BCS_WorldSpace:
		case BCS_ComponentSpace:
			// Component Space, no change.
			OutQuat = InCSRotator.Quaternion();
			break;

		case BCS_ParentBoneSpace:
		{
			const FCompactPoseBoneIndex ParentIndex = MeshBases.GetPose().GetParentBoneIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(ParentIndex);
				FTransform InverseParentTM = ParentTM.Inverse();
				//Calculate the new delta rotation
				FVector4 BoneSpaceAxis = InverseParentTM.TransformVector(RotAxis);
				FQuat DeltaQuat(BoneSpaceAxis, RotAngle);
				DeltaQuat.Normalize();
				OutQuat = DeltaQuat;
			}
		}
		break;

		case BCS_BoneSpace:
		{
			const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(BoneIndex);
			FTransform InverseBoneTM = BoneTM.Inverse();
			FVector4 BoneSpaceAxis = InverseBoneTM.TransformVector(RotAxis);
			//Calculate the new delta rotation
			FQuat DeltaQuat(BoneSpaceAxis, RotAngle);
			DeltaQuat.Normalize();
			OutQuat = DeltaQuat;
		}
		break;
		}
	}

	return OutQuat;
}

FVector FAnimNodeEditMode::ConvertWidgetLocation(const USkeletalMeshComponent* InSkelComp, FCSPose<FCompactHeapPose>& InMeshBases, const FBoneSocketTarget& Target, const FVector& InLocation, const EBoneControlSpace Space)
{
	FVector WidgetLoc = FVector::ZeroVector;

	switch (Space)
	{
		// GetComponentTransform() must be Identity in preview window so same as ComponentSpace
	case BCS_WorldSpace:
	case BCS_ComponentSpace:
	{
		// Component Space, no change.
		WidgetLoc = InLocation;
	}
	break;

	case BCS_ParentBoneSpace:
	{
		if (InMeshBases.GetPose().IsValid())
		{
			const FCompactPoseBoneIndex CompactBoneIndex = Target.GetCompactPoseBoneIndex();

			if (CompactBoneIndex != INDEX_NONE)
			{
				if (ensure(InMeshBases.GetPose().IsValidIndex(CompactBoneIndex)))
				{
					const FCompactPoseBoneIndex CompactParentIndex = InMeshBases.GetPose().GetParentBoneIndex(CompactBoneIndex);
					if (CompactParentIndex != INDEX_NONE)
					{
						const FTransform& ParentTM = InMeshBases.GetComponentSpaceTransform(CompactParentIndex);
						WidgetLoc = ParentTM.TransformPosition(InLocation);
					}
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Using socket(%d), Socket name(%ls), Bone name(%ls)", 
						Target.bUseSocket, *Target.SocketReference.SocketName.ToString(), *Target.BoneReference.BoneName.ToString());
				}
			}
		}
	}
	break;

	case BCS_BoneSpace:
	{
		if (InMeshBases.GetPose().IsValid())
		{
			FTransform BoneTM = Target.GetTargetTransform(FVector::ZeroVector, InMeshBases, InSkelComp->GetComponentToWorld());
			WidgetLoc = BoneTM.TransformPosition(InLocation);
		}
	}
	break;
	}
	
	return WidgetLoc;
}
FVector FAnimNodeEditMode::ConvertWidgetLocation(const USkeletalMeshComponent* SkelComp, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const FVector& Location, const EBoneControlSpace Space)
{
	FVector WidgetLoc = FVector::ZeroVector;

	auto GetCompactBoneIndex = [](const USkeletalMeshComponent* InSkelComp, FCSPose<FCompactHeapPose>& InMeshBases, const FName& InBoneName)
	{
		if (InMeshBases.GetPose().IsValid())
		{
			USkeleton* Skeleton = InSkelComp->GetSkeletalMeshAsset()->GetSkeleton();
			const int32 MeshBoneIndex = InSkelComp->GetBoneIndex(InBoneName);
			if (MeshBoneIndex != INDEX_NONE)
			{
				return InMeshBases.GetPose().GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
			}
		}

		return FCompactPoseBoneIndex(INDEX_NONE);
	};

	switch (Space)
	{
		// GetComponentTransform() must be Identity in preview window so same as ComponentSpace
	case BCS_WorldSpace:
	case BCS_ComponentSpace:
	{
		// Component Space, no change.
		WidgetLoc = Location;
	}
	break;

	case BCS_ParentBoneSpace:
		{
			const FCompactPoseBoneIndex CompactBoneIndex = GetCompactBoneIndex(SkelComp, MeshBases, BoneName);
			if (CompactBoneIndex != INDEX_NONE)
			{
				const FCompactPoseBoneIndex CompactParentIndex = MeshBases.GetPose().GetParentBoneIndex(CompactBoneIndex);
				if (CompactParentIndex != INDEX_NONE)
				{
					const FTransform& ParentTM = MeshBases.GetComponentSpaceTransform(CompactParentIndex);
					WidgetLoc = ParentTM.TransformPosition(Location);
				}
			}
		}
		break;

	case BCS_BoneSpace:
		{
			const FCompactPoseBoneIndex CompactBoneIndex = GetCompactBoneIndex(SkelComp, MeshBases, BoneName);
			if (CompactBoneIndex != INDEX_NONE)
			{
				const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(CompactBoneIndex);
				WidgetLoc = BoneTM.TransformPosition(Location);
			}
		}
		break;
	}

	return WidgetLoc;
}

#undef LOCTEXT_NAMESPACE
