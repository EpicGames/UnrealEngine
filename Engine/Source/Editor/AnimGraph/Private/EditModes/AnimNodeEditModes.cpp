// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodeEditModes.h"

const FEditorModeID AnimNodeEditModes::AnimDynamics("AnimGraph.AnimDynamics");
const FEditorModeID AnimNodeEditModes::AnimNode("AnimGraph.AnimNode");
const FEditorModeID AnimNodeEditModes::TwoBoneIK("AnimGraph.SkeletalControl.TwoBoneIK");
const FEditorModeID AnimNodeEditModes::ObserveBone("AnimGraph.SkeletalControl.ObserveBone");
const FEditorModeID AnimNodeEditModes::ModifyBone("AnimGraph.SkeletalControl.ModifyBone");
const FEditorModeID AnimNodeEditModes::Fabrik("AnimGraph.SkeletalControl.Fabrik");
const FEditorModeID AnimNodeEditModes::PoseDriver("AnimGraph.AnimNode.PoseDriver");
const FEditorModeID AnimNodeEditModes::CCDIK("AnimGraph.SkeletalControl.CCDIK");
const FEditorModeID AnimNodeEditModes::SplineIK("AnimGraph.AnimNode.SplineIK");
const FEditorModeID AnimNodeEditModes::LookAt("AnimGraph.SkeletalControl.LookAt");
