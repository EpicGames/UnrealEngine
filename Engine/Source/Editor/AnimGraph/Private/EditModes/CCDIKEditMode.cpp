// Copyright Epic Games, Inc. All Rights Reserved.

#include "CCDIKEditMode.h"
#include "AnimGraphNode_CCDIK.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "PrimitiveDrawingUtils.h"

void FCCDIKEditMode::EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode)
{
	RuntimeNode = static_cast<FAnimNode_CCDIK*>(InRuntimeNode);
	GraphNode = CastChecked<UAnimGraphNode_CCDIK>(InEditorNode);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FCCDIKEditMode::ExitMode()
{
	RuntimeNode = nullptr;
	GraphNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

FVector FCCDIKEditMode::GetWidgetLocation() const
{
	EBoneControlSpace Space = RuntimeNode->EffectorLocationSpace;
	FVector Location = RuntimeNode->EffectorLocation;
	FBoneSocketTarget Target = RuntimeNode->EffectorTarget;

	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	return ConvertWidgetLocation(SkelComp, RuntimeNode->ForwardedPose, Target, Location, Space);
}

UE::Widget::EWidgetMode FCCDIKEditMode::GetWidgetMode() const
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	int32 TipBoneIndex = SkelComp->GetBoneIndex(RuntimeNode->TipBone.BoneName);
	int32 RootBoneIndex = SkelComp->GetBoneIndex(RuntimeNode->RootBone.BoneName);

	if (TipBoneIndex!= INDEX_NONE && RootBoneIndex != INDEX_NONE)
	{
		return UE::Widget::WM_Translate;
	}

	return UE::Widget::WM_None;
}

bool FCCDIKEditMode::UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const
{
	return InWidgetMode == UE::Widget::WM_Translate;
}

void FCCDIKEditMode::DoTranslation(FVector& InTranslation)
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	FVector Offset = ConvertCSVectorToBoneSpace(SkelComp, InTranslation, RuntimeNode->ForwardedPose, RuntimeNode->EffectorTarget, GraphNode->Node.EffectorLocationSpace);

	RuntimeNode->EffectorLocation += Offset;
	GraphNode->Node.EffectorLocation = RuntimeNode->EffectorLocation;
	GraphNode->SetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_CCDIK, EffectorLocation), RuntimeNode->EffectorLocation);
}

void FCCDIKEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// draw line from root bone to tip bone if available
	if (RuntimeNode && RuntimeNode->DebugLines.Num() > 0)
	{
		USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
		FTransform CompToWorld = SkelComp->GetComponentToWorld();

		// no component space
		for (int32 Index = 1; Index < RuntimeNode->DebugLines.Num(); ++Index)
		{
			FVector Start = CompToWorld.TransformPosition(RuntimeNode->DebugLines[Index - 1]);
			FVector End = CompToWorld.TransformPosition(RuntimeNode->DebugLines[Index]);

			PDI->DrawLine(Start, End, FLinearColor::Red, SDPG_Foreground);
		}
	}
#endif // #if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}