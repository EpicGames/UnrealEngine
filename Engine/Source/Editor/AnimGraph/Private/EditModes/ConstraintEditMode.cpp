// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_Constraint.h"
#include "BoneControllers/AnimNode_Constraint.h"
#include "Templates/Casts.h"

FConstraintEditMode::FConstraintEditMode()
{
	CurWidgetMode = UE::Widget::WM_Rotate;
}

void FConstraintEditMode::EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode)
{
	RuntimeNode = static_cast<FAnimNode_Constraint*>(InRuntimeNode);
	GraphNode = CastChecked<UAnimGraphNode_Constraint>(InEditorNode);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FConstraintEditMode::ExitMode()
{
	RuntimeNode = nullptr;
	GraphNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

