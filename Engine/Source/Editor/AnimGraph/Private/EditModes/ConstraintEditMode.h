// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeEditMode.h"
#include "UnrealWidgetFwd.h"

class FConstraintEditMode : public FAnimNodeEditMode
{
public:
	FConstraintEditMode();

	/** IAnimNodeEditMode interface */
	virtual void EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode) override;
	virtual void ExitMode() override;
private:

private:
	struct FAnimNode_Constraint* RuntimeNode;
	class UAnimGraphNode_Constraint* GraphNode;

	// storing current widget mode 
	mutable UE::Widget::EWidgetMode CurWidgetMode;
};
