// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModes/FabrikEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_Fabrik.h"
#include "Animation/AnimTypes.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneControllers/AnimNode_Fabrik.h"
#include "Containers/EnumAsByte.h"
#include "IPersonaPreviewScene.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Templates/Casts.h"

class USkeletalMeshComponent;
struct FBoneSocketTarget;

void FFabrikEditMode::EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode)
{
	RuntimeNode = static_cast<FAnimNode_Fabrik*>(InRuntimeNode);
	GraphNode = CastChecked<UAnimGraphNode_Fabrik>(InEditorNode);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FFabrikEditMode::ExitMode()
{
	RuntimeNode = nullptr;
	GraphNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

FVector FFabrikEditMode::GetWidgetLocation() const
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	FBoneSocketTarget& Target = RuntimeNode->EffectorTarget;
	FVector Location = RuntimeNode->EffectorTransform.GetLocation();
	EBoneControlSpace Space = RuntimeNode->EffectorTransformSpace;
	FVector WidgetLoc = ConvertWidgetLocation(SkelComp, RuntimeNode->ForwardedPose, Target, Location, Space);
	return WidgetLoc;
}

UE::Widget::EWidgetMode FFabrikEditMode::GetWidgetMode() const
{
	// allow translation all the time for effectot target
	return UE::Widget::WM_Translate;
}

bool FFabrikEditMode::UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const
{
	return InWidgetMode == UE::Widget::WM_Translate;
}

void FFabrikEditMode::DoTranslation(FVector& InTranslation)
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	FVector Offset = ConvertCSVectorToBoneSpace(SkelComp, InTranslation, RuntimeNode->ForwardedPose, RuntimeNode->EffectorTarget, RuntimeNode->EffectorTransformSpace);

	RuntimeNode->EffectorTransform.AddToTranslation(Offset);
	GraphNode->Node.EffectorTransform.SetTranslation(RuntimeNode->EffectorTransform.GetTranslation());
}
