// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeEditMode.h"
#include "Math/MathFwd.h"
#include "UnrealWidgetFwd.h"

class FFabrikEditMode : public FAnimNodeEditMode
{
public:
	/** IAnimNodeEditMode interface */
	virtual void EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode) override;
	virtual void ExitMode() override;
	virtual FVector GetWidgetLocation() const override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const override;
	virtual void DoTranslation(FVector& InTranslation) override;

private:
	struct FAnimNode_Fabrik* RuntimeNode;
	class UAnimGraphNode_Fabrik* GraphNode;
};
