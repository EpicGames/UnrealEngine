// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModes/LookAtEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_LookAt.h"
#include "Animation/AnimTypes.h"
#include "Animation/BoneSocketReference.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneControllers/AnimNode_LookAt.h"
#include "IPersonaPreviewScene.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"

class USkeletalMeshComponent;

void FLookAtEditMode::EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode)
{
	RuntimeNode = static_cast<FAnimNode_LookAt*>(InRuntimeNode);
	GraphNode = CastChecked<UAnimGraphNode_LookAt>(InEditorNode);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FLookAtEditMode::ExitMode()
{
	RuntimeNode = nullptr;
	GraphNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

FVector FLookAtEditMode::GetWidgetLocation() const
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	EBoneControlSpace Space = RuntimeNode->LookAtTarget.HasTargetSetup() ? BCS_BoneSpace : BCS_ComponentSpace;
	
	return ConvertWidgetLocation(SkelComp, RuntimeNode->ForwardedPose, RuntimeNode->LookAtTarget, RuntimeNode->LookAtLocation, Space);
}

ECoordSystem FLookAtEditMode::GetWidgetCoordinateSystem() const
{
	return RuntimeNode->LookAtTarget.HasTargetSetup() ? COORD_Local : COORD_World;
}

UE::Widget::EWidgetMode FLookAtEditMode::GetWidgetMode() const
{
	return UE::Widget::WM_Translate;
}

bool FLookAtEditMode::UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const
{
	return InWidgetMode == UE::Widget::WM_Translate;
}

FName FLookAtEditMode::GetSelectedBone() const
{
	return RuntimeNode->LookAtTarget.GetTargetSetup();
}

bool FLookAtEditMode::ShouldDrawWidget() const
{
	return true;
}

void FLookAtEditMode::DoTranslation(FVector& InTranslation)
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	EBoneControlSpace Space = RuntimeNode->LookAtTarget.HasTargetSetup() ? BCS_BoneSpace : BCS_ComponentSpace;
	FVector Offset = ConvertCSVectorToBoneSpace(SkelComp, InTranslation, RuntimeNode->ForwardedPose, RuntimeNode->LookAtTarget, Space);

	RuntimeNode->LookAtLocation += Offset;

	GraphNode->Node.LookAtLocation = RuntimeNode->LookAtLocation;
	GraphNode->SetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_LookAt, LookAtLocation), RuntimeNode->LookAtLocation);
}
