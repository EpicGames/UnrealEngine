// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModes/ModifyBoneEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_ModifyBone.h"
#include "Animation/AnimTypes.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneContainer.h"
#include "BoneIndices.h"
#include "BonePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/EnumAsByte.h"
#include "Engine/SkeletalMesh.h"
#include "IPersonaPreviewScene.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"

class USkeleton;

FModifyBoneEditMode::FModifyBoneEditMode()
{
	CurWidgetMode = UE::Widget::WM_Rotate;
}

void FModifyBoneEditMode::EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode)
{
	RuntimeNode = static_cast<FAnimNode_ModifyBone*>(InRuntimeNode);
	GraphNode = CastChecked<UAnimGraphNode_ModifyBone>(InEditorNode);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FModifyBoneEditMode::ExitMode()
{
	RuntimeNode = nullptr;
	GraphNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

ECoordSystem FModifyBoneEditMode::GetWidgetCoordinateSystem() const
{
	EBoneControlSpace Space = BCS_BoneSpace;
	switch (CurWidgetMode)
	{
	case UE::Widget::WM_Rotate:
		Space = GraphNode->Node.RotationSpace;
		break;
	case UE::Widget::WM_Translate:
		Space = GraphNode->Node.TranslationSpace;
		break;
	case UE::Widget::WM_Scale:
		Space = GraphNode->Node.ScaleSpace;
		break;
	}

	switch (Space)
	{
	default:
	case BCS_ParentBoneSpace:
		//@TODO: No good way of handling this one
		return COORD_World;
	case BCS_BoneSpace:
		return COORD_Local;
	case BCS_ComponentSpace:
	case BCS_WorldSpace:
		return COORD_World;
	}
}

FVector FModifyBoneEditMode::GetWidgetLocation() const
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	USkeleton* Skeleton = SkelComp->GetSkeletalMeshAsset()->GetSkeleton();
	FVector WidgetLoc = FVector::ZeroVector;

	// if the current widget mode is translate, then shows the widget according to translation space
	if (CurWidgetMode == UE::Widget::WM_Translate)
	{
		FCSPose<FCompactHeapPose>& MeshBases = RuntimeNode->ForwardedPose;
		WidgetLoc = ConvertWidgetLocation(SkelComp, MeshBases, GraphNode->Node.BoneToModify.BoneName, GraphNode->GetNodeValue(TEXT("Translation"), GraphNode->Node.Translation), GraphNode->Node.TranslationSpace);

		if (MeshBases.GetPose().IsValid() && GraphNode->Node.TranslationMode == BMM_Additive)
		{
			if (GraphNode->Node.TranslationSpace == EBoneControlSpace::BCS_WorldSpace ||
				GraphNode->Node.TranslationSpace == EBoneControlSpace::BCS_ComponentSpace)
			{
				const FMeshPoseBoneIndex MeshBoneIndex(SkelComp->GetBoneIndex(GraphNode->Node.BoneToModify.BoneName));
				const FCompactPoseBoneIndex BoneIndex = MeshBases.GetPose().GetBoneContainer().MakeCompactPoseIndex(MeshBoneIndex);

				if (BoneIndex != INDEX_NONE)
				{
					const FTransform& BoneTM = MeshBases.GetComponentSpaceTransform(BoneIndex);
					WidgetLoc += BoneTM.GetLocation();
				}
			}
		}
	}
	else // if the current widget mode is not translate mode, then show the widget on the bone to modify
	{
		int32 MeshBoneIndex = SkelComp->GetBoneIndex(GraphNode->Node.BoneToModify.BoneName);

		if (MeshBoneIndex != INDEX_NONE)
		{
			const FTransform BoneTM = SkelComp->GetBoneTransform(MeshBoneIndex);
			WidgetLoc = BoneTM.GetLocation();
		}
	}

	return WidgetLoc;
}

EBoneModificationMode FModifyBoneEditMode::GetBoneModificationMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	switch (InWidgetMode)
	{
	case UE::Widget::WM_Translate:
		if(!GraphNode->IsPinExposedAndLinked(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Translation)))
		{
			return GraphNode->Node.TranslationMode;
		}
		break;
	case UE::Widget::WM_Rotate:
		if(!GraphNode->IsPinExposedAndLinked(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Rotation)))
		{
			return GraphNode->Node.RotationMode;
		}
		break;
	case UE::Widget::WM_Scale:
		if(!GraphNode->IsPinExposedAndLinked(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Scale)))
		{
			return GraphNode->Node.ScaleMode;
		}
		break;
	case UE::Widget::WM_TranslateRotateZ:
	case UE::Widget::WM_2D:
		break;
	}

	return EBoneModificationMode::BMM_Ignore;
}

UE::Widget::EWidgetMode FModifyBoneEditMode::GetNextWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	UE::Widget::EWidgetMode InMode = InWidgetMode;
	switch (InMode)
	{
	case UE::Widget::WM_Translate:
		return UE::Widget::WM_Rotate;
	case UE::Widget::WM_Rotate:
		return UE::Widget::WM_Scale;
	case UE::Widget::WM_Scale:
		return UE::Widget::WM_Translate;
	case UE::Widget::WM_TranslateRotateZ:
	case UE::Widget::WM_2D:
		break;
	}

	return UE::Widget::WM_None;
}

UE::Widget::EWidgetMode FModifyBoneEditMode::FindValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	UE::Widget::EWidgetMode InMode = InWidgetMode;
	UE::Widget::EWidgetMode ValidMode = InMode;
	if (InMode == UE::Widget::WM_None)
	{	// starts from Rotate mode
		ValidMode = UE::Widget::WM_Rotate;
	}

	// find from current widget mode and loop 1 cycle until finding a valid mode
	for (int32 Index = 0; Index < 3; Index++)
	{
		if (GetBoneModificationMode(ValidMode) != BMM_Ignore)
		{
			return ValidMode;
		}

		ValidMode = GetNextWidgetMode(ValidMode);
	}

	// if couldn't find a valid mode, returns None
	ValidMode = UE::Widget::WM_None;

	return ValidMode;
}

UE::Widget::EWidgetMode FModifyBoneEditMode::GetWidgetMode() const
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	int32 BoneIndex = SkelComp->GetBoneIndex(GraphNode->Node.BoneToModify.BoneName);
	if (BoneIndex != INDEX_NONE)
	{
		CurWidgetMode = FindValidWidgetMode(CurWidgetMode);
		return CurWidgetMode;
	}

	return UE::Widget::WM_None;
}

UE::Widget::EWidgetMode FModifyBoneEditMode::ChangeToNextWidgetMode(UE::Widget::EWidgetMode InCurWidgetMode)
{
	UE::Widget::EWidgetMode NextWidgetMode = GetNextWidgetMode(InCurWidgetMode);
	CurWidgetMode = FindValidWidgetMode(NextWidgetMode);

	return CurWidgetMode;
}

bool FModifyBoneEditMode::SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode)
{
	// if InWidgetMode is available 
	if (FindValidWidgetMode(InWidgetMode) == InWidgetMode)
	{
		CurWidgetMode = InWidgetMode;
		return true;
	}

	return false;
}

bool FModifyBoneEditMode::UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const
{
	return FindValidWidgetMode(InWidgetMode) == InWidgetMode;
}

FName FModifyBoneEditMode::GetSelectedBone() const
{
	return GraphNode->Node.BoneToModify.BoneName;
}

void FModifyBoneEditMode::DoTranslation(FVector& InTranslation)
{
	if (GraphNode->Node.TranslationMode != EBoneModificationMode::BMM_Ignore)
	{
		USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
		FVector Offset = ConvertCSVectorToBoneSpace(SkelComp, InTranslation, RuntimeNode->ForwardedPose, GraphNode->Node.BoneToModify.BoneName, GraphNode->Node.TranslationSpace);

		RuntimeNode->Translation += Offset;

		GraphNode->Node.Translation = RuntimeNode->Translation;
		GraphNode->SetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Translation), RuntimeNode->Translation);
	}
}

void FModifyBoneEditMode::DoRotation(FRotator& InRotation)
{
	if (GraphNode->Node.RotationMode != EBoneModificationMode::BMM_Ignore)
	{
		USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
		FQuat DeltaQuat = ConvertCSRotationToBoneSpace(SkelComp, InRotation, RuntimeNode->ForwardedPose, GraphNode->Node.BoneToModify.BoneName, GraphNode->Node.RotationSpace);

		FQuat PrevQuat(RuntimeNode->Rotation);
		FQuat NewQuat = DeltaQuat * PrevQuat;
		RuntimeNode->Rotation = NewQuat.Rotator();
		GraphNode->Node.Rotation = RuntimeNode->Rotation;
		GraphNode->SetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Rotation), RuntimeNode->Rotation);
	}
}

void FModifyBoneEditMode::DoScale(FVector& InScale)
{
	if (GraphNode->Node.ScaleMode != EBoneModificationMode::BMM_Ignore)
	{
		FVector Offset = InScale;
		RuntimeNode->Scale += Offset;
		GraphNode->Node.Scale = RuntimeNode->Scale;
		GraphNode->SetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Scale), RuntimeNode->Scale);
	}
}

bool FModifyBoneEditMode::ShouldDrawWidget() const
{
	// Prevent us from using widgets if pins are linked
	if(CurWidgetMode == UE::Widget::WM_Translate && GraphNode->IsPinExposedAndLinked(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Translation)))
	{
		return false;
	}

	if(CurWidgetMode == UE::Widget::WM_Rotate && GraphNode->IsPinExposedAndLinked(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Rotation)))
	{
		return false;
	}

	if(CurWidgetMode == UE::Widget::WM_Scale && GraphNode->IsPinExposedAndLinked(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ModifyBone, Scale)))
	{
		return false;
	}

	return true;
}