// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeEditMode.h"
#include "BoneControllers/AnimNode_ModifyBone.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "UObject/NameTypes.h"
#include "UnrealWidgetFwd.h"

class FModifyBoneEditMode : public FAnimNodeEditMode
{
public:
	FModifyBoneEditMode();

	/** IAnimNodeEditMode interface */
	virtual void EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode) override;
	virtual void ExitMode() override;
	virtual ECoordSystem GetWidgetCoordinateSystem() const override;
	virtual FVector GetWidgetLocation() const override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual UE::Widget::EWidgetMode ChangeToNextWidgetMode(UE::Widget::EWidgetMode InCurWidgetMode) override;
	virtual bool SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode) override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const override;
	virtual FName GetSelectedBone() const override;
	virtual void DoTranslation(FVector& InTranslation) override;
	virtual void DoRotation(FRotator& InRotation) override;
	virtual void DoScale(FVector& InScale) override;
	virtual bool ShouldDrawWidget() const override;

private:
	// methods to find a valid widget mode for gizmo because doesn't need to show gizmo when the mode is "Ignore"
	UE::Widget::EWidgetMode FindValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const;
	EBoneModificationMode GetBoneModificationMode(UE::Widget::EWidgetMode InWidgetMode) const;
	UE::Widget::EWidgetMode GetNextWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const;

private:
	struct FAnimNode_ModifyBone* RuntimeNode;
	class UAnimGraphNode_ModifyBone* GraphNode;

	// storing current widget mode 
	mutable UE::Widget::EWidgetMode CurWidgetMode;
};
