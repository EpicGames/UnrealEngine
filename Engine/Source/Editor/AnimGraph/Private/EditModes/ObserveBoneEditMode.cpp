// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModes/ObserveBoneEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_ObserveBone.h"
#include "Animation/AnimTypes.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneContainer.h"
#include "BoneControllers/AnimNode_ObserveBone.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/EnumAsByte.h"
#include "Engine/SkeletalMesh.h"
#include "IPersonaPreviewScene.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"

class USkeleton;

void FObserveBoneEditMode::EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode)
{
	RuntimeNode = static_cast<FAnimNode_ObserveBone*>(InRuntimeNode);
	GraphNode = CastChecked<UAnimGraphNode_ObserveBone>(InEditorNode);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FObserveBoneEditMode::ExitMode()
{
	RuntimeNode = nullptr;
	GraphNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

ECoordSystem FObserveBoneEditMode::GetWidgetCoordinateSystem() const
{
	switch (GraphNode->Node.DisplaySpace)
	{
	default:
	case BCS_ParentBoneSpace:
		//@TODO: No good way of handling this one
		return COORD_World;
	case BCS_BoneSpace:
		return COORD_Local;
	case BCS_ComponentSpace:
	case BCS_WorldSpace:
		return COORD_World;
	}
}

FVector FObserveBoneEditMode::GetWidgetLocation() const
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	USkeleton* Skeleton = SkelComp->GetSkeletalMeshAsset()->GetSkeleton();
	FVector WidgetLoc = FVector::ZeroVector;

	int32 MeshBoneIndex = SkelComp->GetBoneIndex(GraphNode->Node.BoneToObserve.BoneName);

	if (MeshBoneIndex != INDEX_NONE)
	{
		const FTransform BoneTM = SkelComp->GetBoneTransform(MeshBoneIndex);
		WidgetLoc = BoneTM.GetLocation();
	}

	return WidgetLoc;
}

UE::Widget::EWidgetMode FObserveBoneEditMode::GetWidgetMode() const
{
	return UE::Widget::WM_Translate;
}

bool FObserveBoneEditMode::UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const
{
	return InWidgetMode == UE::Widget::WM_Translate;
}

FName FObserveBoneEditMode::GetSelectedBone() const
{
	return GraphNode->Node.BoneToObserve.BoneName;
}
