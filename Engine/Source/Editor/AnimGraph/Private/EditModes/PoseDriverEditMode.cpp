// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModes/PoseDriverEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_PoseDriver.h"
#include "AnimNodes/AnimNode_PoseDriver.h"
#include "Animation/AnimInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneContainer.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Engine/EngineTypes.h"
#include "GenericPlatform/ICursor.h"
#include "HitProxies.h"
#include "IPersonaPreviewScene.h"
#include "Math/Color.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/TranslationMatrix.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "RBF/RBFSolver.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

class FSceneView;
class FViewport;
struct FViewportClick;

#define LOCTEXT_NAMESPACE "A3Nodes"

void FPoseDriverEditMode::EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode)
{
	RuntimeNode = static_cast<FAnimNode_PoseDriver*>(InRuntimeNode);
	GraphNode = CastChecked<UAnimGraphNode_PoseDriver>(InEditorNode);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FPoseDriverEditMode::ExitMode()
{
	RuntimeNode = nullptr;
	GraphNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

/** Hit proxy for selecting targets */
struct HPDTargetHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY()

	int32 TargetIndex;

	HPDTargetHitProxy(int32 InTargetIndex)
		: HHitProxy(HPP_World)
		, TargetIndex(InTargetIndex)
	{
	}

	// HHitProxy interface
	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::Crosshairs; }
	// End of HHitProxy interface
};

IMPLEMENT_HIT_PROXY(HPDTargetHitProxy, HHitProxy)


void FPoseDriverEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	// Tell graph node last comp we were used on. A bit ugly, but no easy way to get from details customization to editor instance
	GraphNode->LastPreviewComponent = SkelComp;

	static const float DrawLineWidth = 0.1f;
	static const float DrawPosSize = 2.f;
	const float DrawAxisLength = GraphNode->AxisLength;
	const int32 DrawConeSubdivision = GraphNode->ConeSubdivision;
	const bool bDrawDebugCones = GraphNode->bDrawDebugCones;


	if (SkelComp->AnimScriptInstance)
	{
		const FBoneContainer& RequiredBones = SkelComp->AnimScriptInstance->GetRequiredBones();

		TArray<FRBFTarget> RBFTargets;
		RuntimeNode->GetRBFTargets(RBFTargets, &RequiredBones);

		// Iterate over each bone in the 'source bones' array
		for (int32 SourceIdx=0; SourceIdx< RuntimeNode->SourceBones.Num(); SourceIdx++)
		{
			const FBoneReference& SourceBoneRef = RuntimeNode->SourceBones[SourceIdx];

			// Get mesh bone index
			int32 BoneIndex = SkelComp->GetBoneIndex(SourceBoneRef.BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				// Get transform of driven bone, used as basis for drawing
				const FTransform BoneWorldTM = SkelComp->GetBoneTransform(BoneIndex);
				const FVector BonePos = BoneWorldTM.GetLocation();

				// Transform that we are evaluating pose in
				FTransform EvalSpaceTM = SkelComp->GetComponentToWorld();

				// If specifying space to eval in, get that space
				const int32 EvalSpaceBoneIndex = SkelComp->GetBoneIndex(RuntimeNode->EvalSpaceBone.BoneName);
				const FName ParentBoneName = SkelComp->GetParentBone(SourceBoneRef.BoneName);
				const int32 ParentBoneIndex = SkelComp->GetBoneIndex(ParentBoneName);
				if (EvalSpaceBoneIndex != INDEX_NONE)
				{
					EvalSpaceTM = SkelComp->GetBoneTransform(EvalSpaceBoneIndex);
				}
				// Otherwise, just use parent bone
				else if (ParentBoneIndex != INDEX_NONE)
				{
					EvalSpaceTM = SkelComp->GetBoneTransform(ParentBoneIndex);
				}

				// Get source bone TM from last frame
				if (RuntimeNode->SourceBoneTMs.IsValidIndex(SourceIdx))
				{
					const FTransform SourceBoneTM = RuntimeNode->SourceBoneTMs[SourceIdx];

					// Rotation drawing
					if (RuntimeNode->DriveSource == EPoseDriverSource::Rotation)
					{
						FVector LocalTwistVec = SourceBoneTM.TransformVectorNoScale(RuntimeNode->RBFParams.GetTwistAxisVector());
						FVector WorldTwistVec = EvalSpaceTM.TransformVectorNoScale(LocalTwistVec);
						PDI->DrawLine(BonePos, BonePos + (WorldTwistVec*DrawAxisLength), FLinearColor::Green, SDPG_Foreground, DrawLineWidth);

						// draw the median cones
						if (bDrawDebugCones)
						{
							if (RuntimeNode->RBFParams.NormalizeMethod == ERBFNormalizeMethod::NormalizeWithinMedian)
							{
								const FVector& MedianRot = RuntimeNode->RBFParams.MedianReference;
								LocalTwistVec = FRotator(MedianRot.X, MedianRot.Y, MedianRot.Z).RotateVector(RuntimeNode->RBFParams.GetTwistAxisVector());
								WorldTwistVec = EvalSpaceTM.TransformVectorNoScale(LocalTwistVec);

								const FVector LocalSwingVec = FVector::CrossProduct(LocalTwistVec, FVector(1, 1, 1));
								FVector WorldSwingVec = EvalSpaceTM.TransformVectorNoScale(LocalSwingVec);
								WorldSwingVec.Normalize();

								const FQuat WorldTwistQ(WorldTwistVec, UE_PI * 2.0f / float(DrawConeSubdivision));
								const FQuat WorldSwingMinQ(WorldSwingVec, FMath::DegreesToRadians(RuntimeNode->RBFParams.MedianMin));
								const FQuat WorldSwingMaxQ(WorldSwingVec, FMath::DegreesToRadians(RuntimeNode->RBFParams.MedianMax));

								const FVector FirstMinPositionOnCircle = WorldSwingMinQ.RotateVector(WorldTwistVec * DrawAxisLength);
								const FVector FirstMaxPositionOnCircle = WorldSwingMaxQ.RotateVector(WorldTwistVec * DrawAxisLength);
								FVector LastMinPositionOnCircle = FirstMinPositionOnCircle;
								FVector LastMaxPositionOnCircle = FirstMaxPositionOnCircle;

								const FLinearColor MinColor = FLinearColor::Yellow;
								const FLinearColor MaxColor = MinColor.Desaturate(0.5);

								for (int32 i = 0; i < DrawConeSubdivision; i++)
								{
									FVector NextMinPositionOnCircle = WorldTwistQ.RotateVector(LastMinPositionOnCircle);
									FVector NextMaxPositionOnCircle = WorldTwistQ.RotateVector(LastMaxPositionOnCircle);
									PDI->DrawLine(BonePos, BonePos + NextMinPositionOnCircle, MinColor, SDPG_Foreground, DrawLineWidth);
									PDI->DrawLine(BonePos, BonePos + NextMaxPositionOnCircle, MaxColor, SDPG_Foreground, DrawLineWidth);
									PDI->DrawLine(BonePos + LastMinPositionOnCircle, BonePos + NextMinPositionOnCircle, MinColor, SDPG_Foreground, DrawLineWidth);
									PDI->DrawLine(BonePos + LastMaxPositionOnCircle, BonePos + NextMaxPositionOnCircle, MaxColor, SDPG_Foreground, DrawLineWidth);
									PDI->DrawLine(BonePos + NextMinPositionOnCircle, BonePos + NextMaxPositionOnCircle, MaxColor, SDPG_Foreground, DrawLineWidth);

									LastMinPositionOnCircle = NextMinPositionOnCircle;
									LastMaxPositionOnCircle = NextMaxPositionOnCircle;
								}
								PDI->DrawLine(BonePos + LastMinPositionOnCircle, BonePos + FirstMinPositionOnCircle, MinColor, SDPG_Foreground, DrawLineWidth);
								PDI->DrawLine(BonePos + LastMaxPositionOnCircle, BonePos + FirstMaxPositionOnCircle, MaxColor, SDPG_Foreground, DrawLineWidth);
							}
						}
					}
					// Translation drawing
					else if (RuntimeNode->DriveSource == EPoseDriverSource::Translation)
					{
						const FVector LocalPos = SourceBoneTM.GetTranslation();
						FVector WorldPos = EvalSpaceTM.TransformPosition(LocalPos);
						DrawWireDiamond(PDI, FTranslationMatrix(WorldPos), DrawPosSize, FLinearColor::Green, SDPG_Foreground);

						// draw the median diamonds
						if (RuntimeNode->RBFParams.NormalizeMethod == ERBFNormalizeMethod::NormalizeWithinMedian)
						{
							WorldPos = EvalSpaceTM.TransformPosition(RuntimeNode->RBFParams.MedianReference);

							FLinearColor MinColor = FLinearColor::Yellow;
							FLinearColor MaxColor = MinColor.Desaturate(0.5);

							DrawWireDiamond(PDI, FTranslationMatrix(WorldPos), RuntimeNode->RBFParams.MedianMin, MinColor, SDPG_Foreground);
							DrawWireDiamond(PDI, FTranslationMatrix(WorldPos), RuntimeNode->RBFParams.MedianMax, MaxColor, SDPG_Foreground);
						}
					}

					// Build array of weight for every target
					float TotalWeight = 0.f;
					TArray<float> PerTargetWeights;
					PerTargetWeights.AddZeroed(RuntimeNode->PoseTargets.Num());
					for (const FRBFOutputWeight& Weight : RuntimeNode->OutputWeights)
					{
						TotalWeight += Weight.TargetWeight;
						PerTargetWeights[Weight.TargetIndex] = Weight.TargetWeight;
					}

					// Draw every target for this bone
					for (int32 TargetIdx = 0; TargetIdx < RuntimeNode->PoseTargets.Num(); TargetIdx++)
					{
						// Check we have a target transform for this bone
						const FPoseDriverTarget& PoseTarget = RuntimeNode->PoseTargets[TargetIdx];
						const FRBFTarget& RBFTarget = RBFTargets[TargetIdx];

						// skip hidden entries
						if (PoseTarget.bIsHidden)
						{
							continue;
						}

						if (bDrawDebugCones && PoseTarget.BoneTransforms.IsValidIndex(SourceIdx))
						{
							const FPoseDriverTransform& TargetTM = PoseTarget.BoneTransforms[SourceIdx];

							const bool bSelected = (GraphNode->SelectedTargetIndex == TargetIdx);
							const float AxisLength = bSelected ? (DrawAxisLength * 1.5f) : DrawAxisLength;
							const float LineWidth = bSelected ? (DrawLineWidth * 3.f) : DrawLineWidth;
							const float Radius = RuntimeNode->GetRadiusForTarget(RBFTarget);
							
							const FLinearColor Color = TotalWeight <= 0.f ? FLinearColor::Black : GraphNode->GetColorFromWeight(PerTargetWeights[TargetIdx]);

							PDI->SetHitProxy(new HPDTargetHitProxy(TargetIdx));

							// Rotation drawing
							if (RuntimeNode->DriveSource == EPoseDriverSource::Rotation)
							{
								const FVector LocalTwistVec = TargetTM.TargetRotation.RotateVector(RuntimeNode->RBFParams.GetTwistAxisVector());
								const FVector WorldTwistVec = EvalSpaceTM.TransformVectorNoScale(LocalTwistVec);

								const FVector LocalSwingVec = FVector::CrossProduct(LocalTwistVec, FVector(1, 1, 1));
								FVector WorldSwingVec = EvalSpaceTM.TransformVectorNoScale(LocalSwingVec);
								WorldSwingVec.Normalize();

								const FQuat WorldTwistQ(WorldTwistVec, UE_PI * 2.0f / float(DrawConeSubdivision));
								const FQuat WorldSwingQ(WorldSwingVec, FMath::DegreesToRadians(Radius));

								const FVector FirstPositionOnCircle = WorldSwingQ.RotateVector(WorldTwistVec * DrawAxisLength);
								FVector LastPositionOnCircle = FirstPositionOnCircle;

								for (int32 i = 0; i < DrawConeSubdivision; i++)
								{
									const FVector NextPositionOnCircle = WorldTwistQ.RotateVector(LastPositionOnCircle);
									PDI->DrawLine(BonePos, BonePos + NextPositionOnCircle, Color, SDPG_Foreground, DrawLineWidth);
									PDI->DrawLine(BonePos + LastPositionOnCircle, BonePos + NextPositionOnCircle, Color, SDPG_Foreground, DrawLineWidth);

									LastPositionOnCircle = NextPositionOnCircle;
								}
								PDI->DrawLine(BonePos + LastPositionOnCircle, BonePos + FirstPositionOnCircle, Color, SDPG_Foreground, DrawLineWidth);
							}
							// Translation drawing
							else if (RuntimeNode->DriveSource == EPoseDriverSource::Translation)
							{
								const FVector LocalPos = TargetTM.TargetTranslation;
								const FVector WorldPos = EvalSpaceTM.TransformPosition(LocalPos);
								DrawWireDiamond(PDI, FTranslationMatrix(WorldPos), Radius, Color, SDPG_Foreground, LineWidth);
							}

							PDI->SetHitProxy(nullptr);
						}
					}
				}
			}
		}
	}
}

bool FPoseDriverEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	bool bResult = FAnimNodeEditMode::HandleClick(InViewportClient, HitProxy, Click);

	if (HitProxy != nullptr && HitProxy->IsA(HPDTargetHitProxy::StaticGetType()))
	{
		const HPDTargetHitProxy* TargetHitProxy = static_cast<HPDTargetHitProxy*>(HitProxy);
		GraphNode->SelectedTargetIndex = TargetHitProxy->TargetIndex;
		GraphNode->SelectedTargetChangeDelegate.Broadcast();
		bResult = true;
	}

	return bResult;
}

#undef LOCTEXT_NAMESPACE
