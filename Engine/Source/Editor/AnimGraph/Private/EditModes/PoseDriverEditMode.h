// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeEditMode.h"

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class HHitProxy;
struct FViewportClick;

class FPoseDriverEditMode : public FAnimNodeEditMode
{
public:
	/** IAnimNodeEditMode interface */
	virtual void EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode) override;
	virtual void ExitMode() override;

	/** FEdMode interface */
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;

private:
	struct FAnimNode_PoseDriver* RuntimeNode;
	class UAnimGraphNode_PoseDriver* GraphNode;
};
