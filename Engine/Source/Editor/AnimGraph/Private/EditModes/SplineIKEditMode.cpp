// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineIKEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_SplineIK.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneControllers/AnimNode_SplineIK.h"
#include "Components/SplineComponent.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "GenericPlatform/ICursor.h"
#include "HitProxies.h"
#include "IPersonaPreviewScene.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/Color.h"
#include "Math/IntRect.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/TransformVectorized.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"

class FEditorViewportClient;
class FMaterialRenderProxy;
class FViewport;
struct FAnimNode_Base;
struct FViewportClick;

FSplineIKEditMode::FSplineIKEditMode()
	: SplineIKRuntimeNode(nullptr)
	, SplineIKGraphNode(nullptr)
	, SelectedSplinePoint(0)
	, WidgetMode(UE::Widget::WM_None)
{
}

void FSplineIKEditMode::EnterMode(UAnimGraphNode_Base* InEditorNode, FAnimNode_Base* InRuntimeNode)
{
	SplineIKRuntimeNode = static_cast<FAnimNode_SplineIK*>(InRuntimeNode);
	SplineIKGraphNode = CastChecked<UAnimGraphNode_SplineIK>(InEditorNode);

	WidgetMode = FindValidWidgetMode(UE::Widget::WM_None);

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FSplineIKEditMode::ExitMode()
{
	SplineIKGraphNode = nullptr;
	SplineIKRuntimeNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}

ECoordSystem FSplineIKEditMode::GetWidgetCoordinateSystem() const
{ 
	return COORD_Local;
}

struct HSplineHandleHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY()

	int32 SplineHandleIndex;

	HSplineHandleHitProxy(int32 InSplineHandleIndex)
		: HHitProxy(HPP_World)
		, SplineHandleIndex(InSplineHandleIndex)
	{
	}

	// HHitProxy interface
	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::CardinalCross; }
	// End of HHitProxy interface
};

IMPLEMENT_HIT_PROXY(HSplineHandleHitProxy, HHitProxy)

void FSplineIKEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	UDebugSkelMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	USplineComponent::Draw(PDI, View, SplineIKRuntimeNode->GetTransformedSplineCurves().Position, SkelComp->GetComponentTransform().ToMatrixWithScale(), FLinearColor::Yellow, SDPG_Foreground);

	for (int32 SplineHandleIndex = 0; SplineHandleIndex < SplineIKRuntimeNode->GetNumControlPoints(); SplineHandleIndex++)
	{
		const FMaterialRenderProxy* SphereMaterialProxy = SelectedSplinePoint == SplineHandleIndex ? GEngine->ArrowMaterialYellow->GetRenderProxy() : GEngine->ArrowMaterial->GetRenderProxy();

		PDI->SetHitProxy(new HSplineHandleHitProxy(SplineHandleIndex));
		FTransform StartTransform = SplineIKRuntimeNode->GetTransformedSplinePoint(SplineHandleIndex);
		const double Scale = View->WorldToScreen(StartTransform.GetLocation()).W * (4.0 / View->UnscaledViewRect.Width() / View->ViewMatrices.GetViewToClip().M[0][0]);
		DrawSphere(PDI, StartTransform.GetLocation(), FRotator::ZeroRotator, FVector(4.0) * Scale, 64, 64, SphereMaterialProxy, SDPG_Foreground);
		DrawCoordinateSystem(PDI, StartTransform.GetLocation(), StartTransform.GetRotation().Rotator(), static_cast<float>(30.0 * Scale), SDPG_Foreground);
	}

	PDI->SetHitProxy(nullptr);
}

FVector FSplineIKEditMode::GetWidgetLocation() const
{
	if (SelectedSplinePoint != INDEX_NONE)
	{
		FVector Location = SplineIKRuntimeNode->GetTransformedSplinePoint(SelectedSplinePoint).GetLocation();

		UDebugSkelMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();
		return SkelComp->GetComponentTransform().TransformPosition(Location);
	}

	return FVector::ZeroVector;
}

UE::Widget::EWidgetMode FSplineIKEditMode::GetWidgetMode() const
{
	return WidgetMode;
}

bool FSplineIKEditMode::IsModeValid(UE::Widget::EWidgetMode InWidgetMode) const
{
	// @TODO: when transforms are exposed as pin, deny editing via widget
	return true;
}


UE::Widget::EWidgetMode FSplineIKEditMode::GetNextWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	UE::Widget::EWidgetMode InMode = InWidgetMode;
	switch (InMode)
	{
	case UE::Widget::WM_Translate:
		return UE::Widget::WM_Rotate;
	case UE::Widget::WM_Rotate:
		return UE::Widget::WM_Scale;
	case UE::Widget::WM_Scale:
		return UE::Widget::WM_Translate;
	case UE::Widget::WM_TranslateRotateZ:
	case UE::Widget::WM_2D:
		break;
	}

	return UE::Widget::WM_None;
}

UE::Widget::EWidgetMode FSplineIKEditMode::FindValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const
{
	UE::Widget::EWidgetMode InMode = InWidgetMode;
	UE::Widget::EWidgetMode ValidMode = InMode;
	if (InMode == UE::Widget::WM_None)
	{	
		// starts from translate mode
		ValidMode = UE::Widget::WM_Translate;
	}

	// find from current widget mode and loop 1 cycle until finding a valid mode
	for (int32 Index = 0; Index < 3; Index++)
	{
		if (IsModeValid(ValidMode))
		{
			return ValidMode;
		}

		ValidMode = GetNextWidgetMode(ValidMode);
	}

	// if couldn't find a valid mode, returns None
	ValidMode = UE::Widget::WM_None;

	return ValidMode;
}

UE::Widget::EWidgetMode FSplineIKEditMode::ChangeToNextWidgetMode(UE::Widget::EWidgetMode CurWidgetMode)
{
	UE::Widget::EWidgetMode NextWidgetMode = GetNextWidgetMode(CurWidgetMode);
	WidgetMode = FindValidWidgetMode(NextWidgetMode);

	return WidgetMode;
}

bool FSplineIKEditMode::SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode)
{
	WidgetMode = InWidgetMode;
	return true;
}

bool FSplineIKEditMode::UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const
{
	return FindValidWidgetMode(InWidgetMode) == InWidgetMode;
}

FName FSplineIKEditMode::GetSelectedBone() const
{
	return NAME_None;
}

bool FSplineIKEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	bool bResult = FAnimNodeEditMode::HandleClick(InViewportClient, HitProxy, Click);

	if (HitProxy != nullptr && HitProxy->IsA(HSplineHandleHitProxy::StaticGetType()))
	{
		HSplineHandleHitProxy* HandleHitProxy = static_cast<HSplineHandleHitProxy*>(HitProxy);
		SelectedSplinePoint = HandleHitProxy->SplineHandleIndex;
		bResult = true;
	}

	return bResult;
}

bool FSplineIKEditMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	UDebugSkelMeshComponent* SkelMeshComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	if (SkelMeshComp)
	{
		if (SelectedSplinePoint != INDEX_NONE)
		{
			FTransform Transform = SplineIKRuntimeNode->GetTransformedSplinePoint(SelectedSplinePoint);
			FTransform WorldTransform = Transform * SkelMeshComp->GetComponentTransform();
			InMatrix = WorldTransform.ToMatrixNoScale().RemoveTranslation();
		}

		return true;
	}

	return false;
}

void FSplineIKEditMode::DoTranslation(FVector& InTranslation)
{
	if (SelectedSplinePoint != INDEX_NONE)
	{
		FVector NewLocation = SplineIKRuntimeNode->GetControlPoint(SelectedSplinePoint).GetLocation() + InTranslation;
		SplineIKRuntimeNode->SetControlPointLocation(SelectedSplinePoint, NewLocation);
		SplineIKGraphNode->Node.SetControlPointLocation(SelectedSplinePoint, NewLocation);
	}
}

void FSplineIKEditMode::DoRotation(FRotator& InRot)
{
	if (SelectedSplinePoint != INDEX_NONE)
	{
		FQuat NewRotation = SplineIKRuntimeNode->GetControlPoint(SelectedSplinePoint).GetRotation() * InRot.Quaternion();
		SplineIKRuntimeNode->SetControlPointRotation(SelectedSplinePoint, NewRotation);
		SplineIKGraphNode->Node.SetControlPointRotation(SelectedSplinePoint, NewRotation);
	}
}

void FSplineIKEditMode::DoScale(FVector& InScale)
{
	if (SelectedSplinePoint != INDEX_NONE)
	{
		FVector NewScale = SplineIKRuntimeNode->GetControlPoint(SelectedSplinePoint).GetScale3D() + InScale;
		SplineIKRuntimeNode->SetControlPointScale(SelectedSplinePoint, NewScale);
		SplineIKGraphNode->Node.SetControlPointScale(SelectedSplinePoint, NewScale);
	}
}
