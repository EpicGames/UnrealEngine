// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeEditMode.h"
#include "Math/Matrix.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/NameTypes.h"
#include "UnrealWidgetFwd.h"

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class HHitProxy;
class UAnimGraphNode_SplineIK;
struct FAnimNode_SplineIK;
struct FViewportClick;

class FSplineIKEditMode : public FAnimNodeEditMode
{
public:
	FSplineIKEditMode();

	/** IAnimNodeEditMode interface */
	virtual void EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode) override;
	virtual void ExitMode() override;
	ECoordSystem GetWidgetCoordinateSystem() const;
	virtual FVector GetWidgetLocation() const override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual UE::Widget::EWidgetMode ChangeToNextWidgetMode(UE::Widget::EWidgetMode CurWidgetMode) override;
	virtual bool SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode) override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const override;
	virtual FName GetSelectedBone() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual void DoTranslation(FVector& InTranslation) override;
	virtual void DoRotation(FRotator& InRot) override;
	virtual void DoScale(FVector& InScale) override;

	/** FEdMode interface */
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;

private:
	/** Mode helper functions */
	bool IsModeValid(UE::Widget::EWidgetMode InWidgetMode) const;
	UE::Widget::EWidgetMode GetNextWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const;
	UE::Widget::EWidgetMode FindValidWidgetMode(UE::Widget::EWidgetMode InWidgetMode) const;

private:
	/** Cache the typed nodes */
	FAnimNode_SplineIK* SplineIKRuntimeNode;
	UAnimGraphNode_SplineIK* SplineIKGraphNode;

	/** The currently selected spline point */
	int32 SelectedSplinePoint;

	/** Current widget mode */
	UE::Widget::EWidgetMode WidgetMode;
};