// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModes/TwoBoneIKEditMode.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_TwoBoneIK.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "BoneContainer.h"
#include "BoneControllers/AnimNode_TwoBoneIK.h"
#include "BonePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/EnumAsByte.h"
#include "Engine/EngineTypes.h"
#include "Engine/SkeletalMesh.h"
#include "GenericPlatform/ICursor.h"
#include "HitProxies.h"
#include "IPersonaPreviewScene.h"
#include "Math/Color.h"
#include "Math/Matrix.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"

class FSceneView;
class FViewport;
class USkeleton;
struct FAnimNode_Base;
struct FPropertyChangedEvent;

struct HTwoBoneIKProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	HTwoBoneIKProxy(FTwoBoneIKEditMode::BoneSelectModeType InBoneSelectMode)
		: HHitProxy(HPP_Wireframe)
		, BoneSelectMode(InBoneSelectMode)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}

	FTwoBoneIKEditMode::BoneSelectModeType BoneSelectMode;
};

IMPLEMENT_HIT_PROXY(HTwoBoneIKProxy, HHitProxy);

FTwoBoneIKEditMode::FTwoBoneIKEditMode()
	: TwoBoneIKRuntimeNode(nullptr)
	, TwoBoneIKGraphNode(nullptr)
	, BoneSelectMode(BSM_EndEffector)
	, PreviousBoneSpace(BCS_BoneSpace)
{
}


void FTwoBoneIKEditMode::EnterMode(UAnimGraphNode_Base* InEditorNode, FAnimNode_Base* InRuntimeNode)
{
	TwoBoneIKRuntimeNode = static_cast<FAnimNode_TwoBoneIK*>(InRuntimeNode);
	TwoBoneIKGraphNode = CastChecked<UAnimGraphNode_TwoBoneIK>(InEditorNode);

	PreviousBoneSpace = TwoBoneIKGraphNode->Node.EffectorLocationSpace;

	FAnimNodeEditMode::EnterMode(InEditorNode, InRuntimeNode);
}

void FTwoBoneIKEditMode::ExitMode()
{
	TwoBoneIKGraphNode = nullptr;
	TwoBoneIKRuntimeNode = nullptr;

	FAnimNodeEditMode::ExitMode();
}


void FTwoBoneIKEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	USkeletalMeshComponent* SkelMeshComp = GetAnimPreviewScene().GetPreviewMeshComponent();
	if (SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset() && SkelMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
	{
		USkeleton* Skeleton = SkelMeshComp->GetSkeletalMeshAsset()->GetSkeleton();

		PDI->SetHitProxy(new HTwoBoneIKProxy(BSM_EndEffector));
		DrawTargetLocation(PDI, BSM_EndEffector, FColor(255, 128, 128), FColor(180, 128, 128));
		PDI->SetHitProxy(new HTwoBoneIKProxy(BSM_JointTarget));
		DrawTargetLocation(PDI, BSM_JointTarget, FColor(128, 255, 128), FColor(128, 180, 128));
		PDI->SetHitProxy(nullptr);
	}

	FAnimNodeEditMode::Render(View, Viewport, PDI);
}

void FTwoBoneIKEditMode::DrawTargetLocation(FPrimitiveDrawInterface* PDI, BoneSelectModeType InBoneSelectMode, const FColor& TargetColor, const FColor& BoneColor) const
{
	EBoneControlSpace SpaceBase;
	if (InBoneSelectMode == BSM_EndEffector)
	{
		SpaceBase = TwoBoneIKRuntimeNode->EffectorLocationSpace;
	}
	else // BSM_JointTarget
	{
		SpaceBase = TwoBoneIKRuntimeNode->JointTargetLocationSpace;
	}

	const bool bInBoneSpace = (SpaceBase == BCS_ParentBoneSpace) || (SpaceBase == BCS_BoneSpace);
	FVector Location = GetWidgetLocation(InBoneSelectMode);
	FMatrix Matrix = FTransform(Location).ToMatrixNoScale();

	DrawCoordinateSystem( PDI, Location, FRotator::ZeroRotator, 20.f, SDPG_Foreground );
	DrawWireDiamond( PDI, Matrix, 4.f, bInBoneSpace ? BoneColor : TargetColor, SDPG_Foreground );
}

FVector FTwoBoneIKEditMode::GetWidgetLocation(BoneSelectModeType InBoneSelectMode) const
{
	EBoneControlSpace Space;
	FVector Location;
	FBoneSocketTarget Target;

	if (InBoneSelectMode == BSM_EndEffector)
	{
		Space = TwoBoneIKRuntimeNode->EffectorLocationSpace;
		Location = TwoBoneIKRuntimeNode->EffectorLocation;
		Target = TwoBoneIKRuntimeNode->EffectorTarget;
	}
	else // BSM_JointTarget
	{
		Space = TwoBoneIKRuntimeNode->JointTargetLocationSpace;
		Location = TwoBoneIKRuntimeNode->JointTargetLocation;
		Target = TwoBoneIKRuntimeNode->JointTarget;
	}

	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	// Make sure Node has had a chance to evaluate and cache a pose.
	if (TwoBoneIKRuntimeNode->ForwardedPose.GetPose().GetNumBones() > 0)
	{
		return ConvertWidgetLocation(SkelComp, TwoBoneIKRuntimeNode->ForwardedPose, Target, Location, Space);
	}
	else
	{
		return SkelComp->GetComponentTransform().GetLocation();
	}
}

FVector FTwoBoneIKEditMode::GetWidgetLocation() const
{
	return GetWidgetLocation(BoneSelectMode);
}

UE::Widget::EWidgetMode FTwoBoneIKEditMode::GetWidgetMode() const
{
	int32 BoneIndex = GetAnimPreviewScene().GetPreviewMeshComponent()->GetBoneIndex(TwoBoneIKGraphNode->Node.IKBone.BoneName);
	// Two bone IK node uses only Translate
	if (BoneIndex != INDEX_NONE)
	{
		return UE::Widget::WM_Translate;
	}

	return UE::Widget::WM_None;
}

bool FTwoBoneIKEditMode::UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const
{
	return InWidgetMode == UE::Widget::WM_Translate;
}

FBoneSocketTarget FTwoBoneIKEditMode::GetSelectedTarget() const
{
	// should return mesh bone index
	if (BoneSelectMode == BSM_EndEffector)
	{
		if (TwoBoneIKRuntimeNode->EffectorLocationSpace == EBoneControlSpace::BCS_BoneSpace ||
			TwoBoneIKRuntimeNode->EffectorLocationSpace == EBoneControlSpace::BCS_ParentBoneSpace)
		{
			return TwoBoneIKRuntimeNode->EffectorTarget;
		}

	}
	else if (BoneSelectMode == BSM_JointTarget)
	{
		if (TwoBoneIKRuntimeNode->JointTargetLocationSpace == EBoneControlSpace::BCS_BoneSpace ||
			TwoBoneIKRuntimeNode->JointTargetLocationSpace == EBoneControlSpace::BCS_ParentBoneSpace)
		{
			return TwoBoneIKRuntimeNode->JointTarget;
		}
	}

	return FBoneSocketTarget();
}

bool FTwoBoneIKEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	bool bResult = FAnimNodeEditMode::HandleClick(InViewportClient, HitProxy, Click);

	if (HitProxy != nullptr && HitProxy->IsA(HTwoBoneIKProxy::StaticGetType()))
	{
		HTwoBoneIKProxy* TwoBoneIKHitProxy = static_cast<HTwoBoneIKProxy*>(HitProxy);
		BoneSelectMode = TwoBoneIKHitProxy->BoneSelectMode;
		bResult = true;
	}

	return bResult;
}

void FTwoBoneIKEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FAnimNodeEditMode::Tick(ViewportClient, DeltaTime);

	// Cache the current bone space
	PreviousBoneSpace = TwoBoneIKGraphNode->Node.EffectorLocationSpace;
}

void FTwoBoneIKEditMode::OnExternalNodePropertyChange(FPropertyChangedEvent& InPropertyEvent)
{
	
}

void FTwoBoneIKEditMode::DoTranslation(FVector& InTranslation)
{
	USkeletalMeshComponent* SkelComp = GetAnimPreviewScene().GetPreviewMeshComponent();

	if (BoneSelectMode == BSM_EndEffector)
	{
		FVector Offset = ConvertCSVectorToBoneSpace(SkelComp, InTranslation, TwoBoneIKRuntimeNode->ForwardedPose, GetSelectedTarget(), TwoBoneIKGraphNode->Node.EffectorLocationSpace);

		TwoBoneIKRuntimeNode->EffectorLocation += Offset;
		TwoBoneIKGraphNode->Node.EffectorLocation = TwoBoneIKRuntimeNode->EffectorLocation;
		TwoBoneIKGraphNode->SetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, EffectorLocation), TwoBoneIKRuntimeNode->EffectorLocation);
	}
	else if (BoneSelectMode == BSM_JointTarget)
	{
		FVector Offset = ConvertCSVectorToBoneSpace(SkelComp, InTranslation, TwoBoneIKRuntimeNode->ForwardedPose, GetSelectedTarget(), TwoBoneIKGraphNode->Node.JointTargetLocationSpace);

		TwoBoneIKRuntimeNode->JointTargetLocation += Offset;
		TwoBoneIKGraphNode->Node.JointTargetLocation = TwoBoneIKRuntimeNode->JointTargetLocation;
		TwoBoneIKGraphNode->SetDefaultValue(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_TwoBoneIK, JointTargetLocation), TwoBoneIKRuntimeNode->JointTargetLocation);
	}
}
