// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeEditMode.h"
#include "Animation/AnimTypes.h"
#include "Animation/BoneSocketReference.h"
#include "Delegates/IDelegateInstance.h"
#include "Math/MathFwd.h"
#include "UnrealWidgetFwd.h"

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class HHitProxy;
class UAnimGraphNode_TwoBoneIK;
struct FColor;
struct FPropertyChangedEvent;
struct FViewportClick;

class FTwoBoneIKEditMode : public FAnimNodeEditMode
{
public:
	/** Bone selection mode */
	enum BoneSelectModeType
	{
		BSM_EndEffector,
		BSM_JointTarget,
		BSM_Max
	};

	FTwoBoneIKEditMode();

	/** IAnimNodeEditMode interface */
	virtual void EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode) override;
	virtual void ExitMode() override;
	virtual FVector GetWidgetLocation() const override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode InWidgetMode) const override;
	FBoneSocketTarget GetSelectedTarget() const;
	virtual void DoTranslation(FVector& InTranslation) override;

	/** FEdMode interface */
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;

protected:
	void OnExternalNodePropertyChange(FPropertyChangedEvent& InPropertyEvent);
	FDelegateHandle NodePropertyDelegateHandle;

private:
	/** Helper function for Render() */
	void DrawTargetLocation(FPrimitiveDrawInterface* PDI, BoneSelectModeType InBoneSelectMode, const FColor& TargetColor, const FColor& BoneColor) const;

	/** Helper funciton for GetWidgetLocation() and joint rendering */
	FVector GetWidgetLocation(BoneSelectModeType InBoneSelectMode) const;

private:
	/** Cache the typed nodes */
	struct FAnimNode_TwoBoneIK* TwoBoneIKRuntimeNode;
	UAnimGraphNode_TwoBoneIK* TwoBoneIKGraphNode;

	/** The current bone selection mode */
	BoneSelectModeType BoneSelectMode;

	/** The bone space we last saw for the current node */
	EBoneControlSpace PreviousBoneSpace;
};
