// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AnimGetter.h"

#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_StateMachine.h"
#include "AnimStateAliasNode.h"
#include "AnimStateNode.h"
#include "AnimStateNodeBase.h"
#include "AnimStateTransitionNode.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimationAsset.h"
#include "AnimationCustomTransitionSchema.h"
#include "AnimationGraphSchema.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationTransitionGraph.h"
#include "AnimationTransitionSchema.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/Script.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AnimGetter)

class UBlueprint;

#define LOCTEXT_NAMESPACE "AnimGetter"

void UK2Node_AnimGetter::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		RegisterDelegates();
	}
}

void UK2Node_AnimGetter::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	RegisterDelegates();
}

void UK2Node_AnimGetter::RegisterDelegates()
{
	if (!HasAnyFlags(RF_Transient) && HasValidBlueprint())
	{
		if (UAnimBlueprint* ABP = CastChecked<UAnimBlueprint>(GetBlueprint()))
		{
			GraphRenameHandle = ABP->OnGraphRenamedEvent().AddLambda([this](UEdGraph*, FName, FName)
			{
				UpdateCachedTitle();
			});
		}
	}
}

void UK2Node_AnimGetter::UnregisterDelegates()
{
	if (!HasAnyFlags(RF_Transient) && HasValidBlueprint())
	{
		if (UAnimBlueprint* ABP = CastChecked<UAnimBlueprint>(GetBlueprint()))
		{
			ABP->OnGraphRenamedEvent().Remove(GraphRenameHandle);
		}
	}
}

void UK2Node_AnimGetter::BeginDestroy()
{
	Super::BeginDestroy();

	UnregisterDelegates();	
}

void UK2Node_AnimGetter::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::FixBrokenStateMachineReferencesInTransitionGetters)
	{
		RestoreStateMachineNode();
	}
}

void UK2Node_AnimGetter::PostPasteNode()
{
	Super::PostPasteNode();

	SourceAnimBlueprint = Cast<UAnimBlueprint>(GetBlueprint());
	RestoreStateMachineState();
	RestoreStateMachineNode();
	UpdateCachedTitle();
}

void UK2Node_AnimGetter::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	TArray<UEdGraphPin*> PinsToHide;
	TArray<FString> PinNames;

	// TODO: Find a nicer way to maybe pull these down from the instance class and allow 
	// projects to add new parameters from derived instances
	PinNames.Add(TEXT("CurrentTime"));
	PinNames.Add(TEXT("AssetPlayerIndex"));
	PinNames.Add(TEXT("MachineIndex"));
	PinNames.Add(TEXT("StateIndex"));
	PinNames.Add(TEXT("TransitionIndex"));

	for(FString& PinName : PinNames)
	{
		if(UEdGraphPin* FoundPin = FindPin(PinName))
		{
			PinsToHide.Add(FoundPin);
		}
	}

	for(UEdGraphPin* Pin : PinsToHide)
	{
		Pin->bHidden = true;
	}
}

FText UK2Node_AnimGetter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return CachedTitle;
}

bool UK2Node_AnimGetter::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Cast<UAnimationGraphSchema>(Schema) != NULL || Cast<UAnimationTransitionSchema>(Schema) != NULL;
}

void UK2Node_AnimGetter::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// First cache the available functions for getters
	UClass* ActionKey = GetClass();
	const UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(ActionRegistrar.GetActionKeyFilter());
	if(AnimBlueprint && ActionRegistrar.IsOpenForRegistration(AnimBlueprint))
	{
		UClass* BPClass = *AnimBlueprint->ParentClass;
		while(BPClass && !BPClass->HasAnyClassFlags(CLASS_Native))
		{
			BPClass = BPClass->GetSuperClass();
		}

		if(BPClass)
		{
			TArray<UFunction*> AnimGetters;
			for(TFieldIterator<UFunction> FuncIter(BPClass) ; FuncIter ; ++FuncIter)
			{
				UFunction* Func = *FuncIter;

				if(Func->HasMetaData(TEXT("AnimGetter")) && Func->HasAnyFunctionFlags(FUNC_Native))
				{
					AnimGetters.Add(Func);
				}
			}

			auto UiSpecOverride = [](const FBlueprintActionContext& /*Context*/, const IBlueprintNodeBinder::FBindingSet& Bindings, FBlueprintActionUiSpec* UiSpecOut, FText Title)
			{
				UiSpecOut->MenuName = Title;
			};

			TArray<UAnimGraphNode_AssetPlayerBase*> AssetPlayerNodes;
			TArray<UAnimGraphNode_StateMachine*> MachineNodes;
			TArray<UAnimStateNode*> StateNodes;
			TArray<UAnimStateTransitionNode*> TransitionNodes;
			TArray<UAnimStateAliasNode*> AliasNodes;
			
			FBlueprintEditorUtils::GetAllNodesOfClass(AnimBlueprint, AssetPlayerNodes);
			FBlueprintEditorUtils::GetAllNodesOfClass(AnimBlueprint, MachineNodes);
			FBlueprintEditorUtils::GetAllNodesOfClass(AnimBlueprint, StateNodes);
			FBlueprintEditorUtils::GetAllNodesOfClass(AnimBlueprint, TransitionNodes);
			FBlueprintEditorUtils::GetAllNodesOfClass(AnimBlueprint, AliasNodes);

			for(UFunction* Getter : AnimGetters)
			{
				FNodeSpawnData Params;
				Params.AnimInstanceClass = BPClass;
				Params.Getter = Getter;
				Params.SourceBlueprint = AnimBlueprint;
				Params.GetterContextString = Getter->GetMetaData(TEXT("GetterContext"));

				if(GetterRequiresParameter(Getter, TEXT("AssetPlayerIndex")))
				{
					for(UAnimGraphNode_Base* AssetNode : AssetPlayerNodes)
					{
						// Should always succeed
						if(UAnimationAsset* NodeAsset = AssetNode->GetAnimationAsset())
						{
							Params.SourceNode = AssetNode;
							UpdateCachedTitle(Params);

							UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(UK2Node_AnimGetter::StaticClass(), /*AssetNode->GetGraph()*/nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateUObject(const_cast<UK2Node_AnimGetter*>(this), &UK2Node_AnimGetter::PostSpawnNodeSetup, Params));
							Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, Params.CachedTitle);
							ActionRegistrar.AddBlueprintAction(AnimBlueprint, Spawner);
						}
					}
				}
				else if(GetterRequiresParameter(Getter, TEXT("MachineIndex")))
				{
					if(GetterRequiresParameter(Getter, TEXT("StateIndex")))
					{
						for(UAnimStateNode* StateNode : StateNodes)
						{
							// Get the state machine node from the outer chain
							UAnimationStateMachineGraph* Graph = Cast<UAnimationStateMachineGraph>(StateNode->GetOuter());
							if(Graph)
							{
								if(UAnimGraphNode_StateMachine* MachineNode = Cast<UAnimGraphNode_StateMachine>(Graph->GetOuter()))
								{
									Params.SourceNode = MachineNode;
								}
							}

							Params.SourceStateNode = StateNode;
							UpdateCachedTitle(Params);

							UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(UK2Node_AnimGetter::StaticClass(), /*StateNode->GetGraph()*/nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateUObject(const_cast<UK2Node_AnimGetter*>(this), &UK2Node_AnimGetter::PostSpawnNodeSetup, Params));
							Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, Params.CachedTitle);
							ActionRegistrar.AddBlueprintAction(AnimBlueprint, Spawner);
						}

						for (UAnimStateAliasNode* AliasNode : AliasNodes)
						{
							// Get the state machine node from the outer chain
							UAnimationStateMachineGraph* Graph = Cast<UAnimationStateMachineGraph>(AliasNode->GetOuter());
							if (Graph)
							{
								if (UAnimGraphNode_StateMachine* MachineNode = Cast<UAnimGraphNode_StateMachine>(Graph->GetOuter()))
								{
									Params.SourceNode = MachineNode;
								}
							}

							Params.SourceStateNode = AliasNode;
							UpdateCachedTitle(Params);

							UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(UK2Node_AnimGetter::StaticClass(), /*AliasNode->GetGraph()*/nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateUObject(const_cast<UK2Node_AnimGetter*>(this), &UK2Node_AnimGetter::PostSpawnNodeSetup, Params));
							Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, Params.CachedTitle);
							ActionRegistrar.AddBlueprintAction(AnimBlueprint, Spawner);
						}
					}
					else if(GetterRequiresParameter(Getter, TEXT("TransitionIndex")))
					{
						for(UAnimStateTransitionNode* TransitionNode : TransitionNodes)
						{
							UAnimationStateMachineGraph* Graph = Cast<UAnimationStateMachineGraph>(TransitionNode->GetOuter());
							if(Graph)
							{
								if(UAnimGraphNode_StateMachine* MachineNode = Cast<UAnimGraphNode_StateMachine>(Graph->GetOuter()))
								{
									Params.SourceNode = MachineNode;
								}
							}

							Params.SourceStateNode = TransitionNode;
							UpdateCachedTitle(Params);

							UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(UK2Node_AnimGetter::StaticClass(), /*TransitionNode->GetGraph()*/nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateUObject(const_cast<UK2Node_AnimGetter*>(this), &UK2Node_AnimGetter::PostSpawnNodeSetup, Params));
							Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, Params.CachedTitle);
							ActionRegistrar.AddBlueprintAction(AnimBlueprint, Spawner);
						}
					}
					else
					{
						// Only requires the state machine
						for(UAnimGraphNode_StateMachine* MachineNode : MachineNodes)
						{
							Params.SourceNode = MachineNode;
							UpdateCachedTitle(Params);

							UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(UK2Node_AnimGetter::StaticClass(), /*MachineNode*/nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateUObject(const_cast<UK2Node_AnimGetter*>(this), &UK2Node_AnimGetter::PostSpawnNodeSetup, Params));
							Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, Params.CachedTitle);
							ActionRegistrar.AddBlueprintAction(AnimBlueprint, Spawner);
						}
					}
				}
				else
				{
					// Doesn't operate on a node, only need one entry
					UpdateCachedTitle(Params);

					UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(UK2Node_AnimGetter::StaticClass(), nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateUObject(const_cast<UK2Node_AnimGetter*>(this), &UK2Node_AnimGetter::PostSpawnNodeSetup, Params));
					Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, Params.CachedTitle);
					ActionRegistrar.AddBlueprintAction(AnimBlueprint, Spawner);
				}
			}
		}
	}
}

void UK2Node_AnimGetter::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	const bool bAssetPlayerIndexRequired = GetterRequiresParameter(GetTargetFunction(), TEXT("AssetPlayerIndex"));
	const bool bMachineIndexRequired = GetterRequiresParameter(GetTargetFunction(), TEXT("MachineIndex"));
	const bool bTransitionIndexRequired = GetterRequiresParameter(GetTargetFunction(), TEXT("TransitionIndex"));
	const bool bStateIndexRequired = GetterRequiresParameter(GetTargetFunction(), TEXT("StateIndex"));

	const bool bSourceNodeRequired = bAssetPlayerIndexRequired || bMachineIndexRequired || bTransitionIndexRequired || bStateIndexRequired;

	if (bSourceNodeRequired && SourceNode == nullptr)
	{
		MessageLog.Error(TEXT("@@ contains invalid data. Please delete and recreate the node."), this);
	}
}

bool UK2Node_AnimGetter::IsActionFilteredOut(FBlueprintActionFilter const& Filter)
{
	if(Filter.Context.Graphs.Num() > 0)
	{
		UEdGraph* CurrGraph = Filter.Context.Graphs[0];

		if(CurrGraph && Filter.Context.Blueprints.Num() > 0)
		{
			UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Filter.Context.Blueprints[0]);
			check(AnimBlueprint);

			if(SourceAnimBlueprint == AnimBlueprint)
			{
				// Get the native anim instance derived class
				UClass* NativeInstanceClass = AnimBlueprint->ParentClass;
				while(NativeInstanceClass && !NativeInstanceClass->HasAnyClassFlags(CLASS_Native))
				{
					NativeInstanceClass = NativeInstanceClass->GetSuperClass();
				}

				if(GetterClass != NativeInstanceClass)
				{
					// If the anim instance containing the getter is not the class we're currently using then bail
					return true;
				}

				const UEdGraphSchema* Schema = CurrGraph->GetSchema();
				
				// Bail if we aren't meant for this graph at all
				if(!IsContextValidForSchema(Schema))
				{
					return true;
				}

				if(Cast<UAnimationTransitionSchema>(Schema) || Cast<UAnimationCustomTransitionSchema>(Schema))
				{
					if(!SourceNode && !SourceStateNode)
					{
						// No dependancies, always allow
						return false;
					}

					// Inside a transition graph
					if(SourceNode)
					{
						if(UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(CurrGraph->GetOuter()))
						{
							if(SourceStateNode)
							{
								if(UAnimStateTransitionNode* SourceTransitionNode = Cast<UAnimStateTransitionNode>(SourceStateNode))
								{
									// if we have a transition node, make sure it's the same as the one we're in
									if(SourceTransitionNode == TransitionNode)
									{
										return false;
									}
								}
								else if(UAnimStateNode* PreviousStateNode = Cast<UAnimStateNode>(TransitionNode->GetPreviousState()))
								{
									// Only allow actions using states that are referencing the previous state
									if(SourceStateNode == PreviousStateNode)
									{
										return false;
									}
								}
								else if (UAnimStateAliasNode* PreviousAliasNode = Cast<UAnimStateAliasNode>(TransitionNode->GetPreviousState()))
								{
									// Only allow actions using states that are referencing the previous alias
									if (SourceStateNode == PreviousAliasNode)
									{
										return false;
									}
								}
							}
							else if(UAnimGraphNode_StateMachine* MachineNode = Cast<UAnimGraphNode_StateMachine>(SourceNode))
							{
								// Available everywhere
								return false;
							}
							else if(UAnimStateNode* PrevStateNode = Cast<UAnimStateNode>(TransitionNode->GetPreviousState()))
							{
								// Make sure the attached asset node is in the source graph
								if(SourceNode && SourceNode->GetGraph() == PrevStateNode->BoundGraph)
								{
									return false;
								}
							}
						}
					}
				}
				else if(Cast<UAnimationGraphSchema>(Schema))
				{
					// Inside normal anim graph
					if(SourceStateNode)
					{
						for(UBlueprint* Blueprint : Filter.Context.Blueprints)
						{
							TArray<UAnimStateNode*> StateNodes;
							FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, StateNodes);

							if(StateNodes.Contains(SourceStateNode))
							{
								return false;
							}
						}
					}
				}
			}
		}
	}

	return true;
}

void UK2Node_AnimGetter::RestoreStateMachineState()
{
	if (SourceStateNode)
	{
		if (UAnimationTransitionGraph* TransitionGraph = Cast<UAnimationTransitionGraph>(GetOuter()))
		{
			if (UAnimStateTransitionNode* StateTransitionNode = Cast<UAnimStateTransitionNode>(TransitionGraph->GetOuter()))
			{
				SourceStateNode = StateTransitionNode->GetPreviousState();
			}
		}
	}
}

void UK2Node_AnimGetter::RestoreStateMachineNode()
{
	if (SourceStateNode)
	{
		UAnimationStateMachineGraph* Graph = Cast<UAnimationStateMachineGraph>(SourceStateNode->GetOuter());
		if (Graph)
		{
			if (UAnimGraphNode_StateMachine* MachineNode = Cast<UAnimGraphNode_StateMachine>(Graph->GetOuter()))
			{
				SourceNode = MachineNode;
			}
		}
	}
}

bool UK2Node_AnimGetter::GetterRequiresParameter(const UFunction* Getter, FString ParamName)
{
	bool bRequiresParameter = false;

	for(TFieldIterator<FProperty> PropIt(Getter); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* Prop = *PropIt;
		
		if(Prop->GetName() == ParamName)
		{
			bRequiresParameter = true;
			break;
		}
	}

	return bRequiresParameter;
}


void UK2Node_AnimGetter::PostSpawnNodeSetup(UEdGraphNode* NewNode, bool bIsTemplateNode, FNodeSpawnData SpawnData)
{
	UK2Node_AnimGetter* TypedNode = CastChecked<UK2Node_AnimGetter>(NewNode);

	// Apply parameters
	TypedNode->SourceNode = SpawnData.SourceNode;
	TypedNode->SourceStateNode = SpawnData.SourceStateNode;
	TypedNode->GetterClass = SpawnData.AnimInstanceClass;
	TypedNode->SourceAnimBlueprint = SpawnData.SourceBlueprint;
	TypedNode->SetFromFunction((UFunction*)SpawnData.Getter);
	TypedNode->CachedTitle = SpawnData.CachedTitle;
	
	SpawnData.GetterContextString.ParseIntoArray(TypedNode->Contexts, TEXT("|"), 1);
}

bool UK2Node_AnimGetter::IsContextValidForSchema(const UEdGraphSchema* Schema) const
{
	if(Contexts.Num() == 0)
	{
		// Valid in all graphs
		return true;
	}

	const FString& Context = Contexts[0];

	UClass* ClassToCheck = nullptr;
	if(Context == TEXT("CustomBlend"))
	{
		ClassToCheck = UAnimationCustomTransitionSchema::StaticClass();
	}
	else if(Context == TEXT("Transition"))
	{
		ClassToCheck = UAnimationTransitionSchema::StaticClass();
	}
	else if(Context == TEXT("AnimGraph"))
	{
		ClassToCheck = UAnimationGraphSchema::StaticClass();
	}

	return Schema->GetClass() == ClassToCheck;
}

void UK2Node_AnimGetter::UpdateCachedTitle(FNodeSpawnData& SpawnData)
{
	SpawnData.CachedTitle = GenerateTitle((UFunction*)SpawnData.Getter, SpawnData.SourceStateNode, SpawnData.SourceNode);
}

FText UK2Node_AnimGetter::GenerateTitle(UFunction* Getter, UAnimStateNodeBase* SourceStateNode, UAnimGraphNode_Base* SourceNode)
{
	static const FText InvalidNodeTitle = LOCTEXT("NodeTitleInvalid", "{0} (Invalid node)");

	if (GetterRequiresParameter(Getter, TEXT("AssetPlayerIndex")))
	{
		if (!SourceNode)
		{
			return FText::Format(InvalidNodeTitle, Getter->GetDisplayNameText());
		}
		// Should always succeed
		if (UAnimationAsset* NodeAsset = SourceNode->GetAnimationAsset())
		{
			return FText::Format(LOCTEXT("NodeTitle", "{0} ({1})"), Getter->GetDisplayNameText(), FText::FromString(*NodeAsset->GetName()));
		}
	}
	else if (GetterRequiresParameter(Getter, TEXT("MachineIndex")))
	{
		if (GetterRequiresParameter(Getter, TEXT("StateIndex")) || GetterRequiresParameter(Getter, TEXT("TransitionIndex")))
		{
			if (!SourceStateNode)
			{
				return FText::Format(InvalidNodeTitle, Getter->GetDisplayNameText());
			}
			return  FText::Format(LOCTEXT("NodeTitle", "{0} ({1})"), Getter->GetDisplayNameText(), SourceStateNode->GetNodeTitle(ENodeTitleType::ListView));
		}
		else
		{
			if (!SourceNode)
			{
				return FText::Format(InvalidNodeTitle, Getter->GetDisplayNameText());
			}
			// Only requires the state machine
			return FText::Format(LOCTEXT("NodeTitle", "{0} ({1})"), Getter->GetDisplayNameText(), SourceNode->GetNodeTitle(ENodeTitleType::ListView));
		}
	}

	// Doesn't operate on a node, only need one entry
	return Getter->GetDisplayNameText();
}

void UK2Node_AnimGetter::UpdateCachedTitle()
{
	// Make sure we have a valid Blueprint owner before proceeding. Otherwise we will assert trying to resolve the target function below.
	if (!ensureMsgf(HasValidBlueprint(), TEXT("UK2Node_AnimGetter::UpdateCachedTitle(): Attempting to update cached title for [%s] without a valid Blueprint owner (this is unexpected)."), *GetFullName()))
	{
		return;
	}

	if (UFunction* Function = GetTargetFunction())
	{
		CachedTitle = GenerateTitle(Function, SourceStateNode, SourceNode);
	}
}

FNodeSpawnData::FNodeSpawnData()
	: SourceNode(nullptr)
	, SourceStateNode(nullptr)
	, AnimInstanceClass(nullptr)
	, SourceBlueprint(nullptr)
	, Getter(nullptr)
{

}

#undef LOCTEXT_NAMESPACE
