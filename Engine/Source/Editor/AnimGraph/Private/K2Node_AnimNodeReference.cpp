// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AnimNodeReference.h"

#include "AnimBlueprintCompiler.h"
#include "AnimBlueprintExtension.h"
#include "AnimBlueprintExtension_Tag.h"
#include "AnimExecutionContextLibrary.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimNodeReference.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "FindInBlueprintManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AnimNodeReference)

class UEdGraph;
struct FSearchTagDataPair;

#define LOCTEXT_NAMESPACE "K2Node_AnimNodeReference"

void UK2Node_AnimNodeReference::ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	// Expand to function call
	UK2Node_CallFunction* CallFunctionNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, InSourceGraph);
	CallFunctionNode->SetFromFunction(GetDefault<UAnimExecutionContextLibrary>()->FindFunctionChecked(GET_FUNCTION_NAME_CHECKED(UAnimExecutionContextLibrary, GetAnimNodeReference)));
	CallFunctionNode->AllocateDefaultPins();

	UEdGraphPin* ValuePin = FindPinChecked(TEXT("Value"), EGPD_Output);

	if (UEdGraphPin* ReturnValuePin = CallFunctionNode->FindPin(TEXT("ReturnValue"), EGPD_Output))
	{
		InCompilerContext.MovePinLinksToIntermediate(*ValuePin, *ReturnValuePin);
	}

	UAnimBlueprintExtension_Tag* Extension = UAnimBlueprintExtension::FindExtension<UAnimBlueprintExtension_Tag>(Cast<UAnimBlueprint>(GetBlueprint()));
	if (Extension)
	{
		Extension->RequestTaggedNode(this, Tag, CallFunctionNode);
	}
}

void UK2Node_AnimNodeReference::AllocateDefaultPins()
{
	FEdGraphPinType PinType;
	PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
	PinType.PinSubCategoryObject = FAnimNodeReference::StaticStruct();

	CreatePin(EGPD_Output, PinType, TEXT("Value"));
}

FText UK2Node_AnimNodeReference::GetLabelText() const
{
	return FText::FromName(Tag);
}

FText UK2Node_AnimNodeReference::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("NodeTitleFormat", "Node Reference: {0}"), FText::FromName(Tag));
}

void UK2Node_AnimNodeReference::AddSearchMetaDataInfo(TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	if(Tag != NAME_None)
	{
		OutTaggedMetaData.Emplace(LOCTEXT("Tag", "Tag"), FText::FromName(Tag));
	}
}

void UK2Node_AnimNodeReference::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* NodeClass = GetClass();
	const UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(ActionRegistrar.GetActionKeyFilter());
	if (AnimBlueprint && ActionRegistrar.IsOpenForRegistration(AnimBlueprint))
	{
		// Let us spawn a node for any labelled anim graph node we find
		TArray<UAnimGraphNode_Base*> AnimGraphNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass(AnimBlueprint, AnimGraphNodes);

		for(UAnimGraphNode_Base* AnimGraphNodeToReference : AnimGraphNodes)
		{
			if(AnimGraphNodeToReference->GetTag() != NAME_None)
			{
				UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(NodeClass);
				check(NodeSpawner);
				NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda([WeakAnimGraphNodeToReference = TWeakObjectPtr<UAnimGraphNode_Base>(AnimGraphNodeToReference)](UEdGraphNode* InNode, bool bInIsTemplate)
				{
					UK2Node_AnimNodeReference* AnimNodeReference = CastChecked<UK2Node_AnimNodeReference>(InNode);
					AnimNodeReference->Tag = WeakAnimGraphNodeToReference->GetTag();
				});
				ActionRegistrar.AddBlueprintAction(AnimBlueprint, NodeSpawner);
			}
		}
	}
}

bool UK2Node_AnimNodeReference::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	if(Tag != NAME_None)
	{
		for (UBlueprint* Blueprint : FilterContext.Blueprints)
		{
			if(!Blueprint->IsA<UAnimBlueprint>())
			{
				// Not an animation Blueprint, cannot use
				bIsFilteredOut = true;
				break;
			}
		}
	}
	else
	{
		bIsFilteredOut = true;
	}
	return bIsFilteredOut;
}

FText UK2Node_AnimNodeReference::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Variables);
}

bool UK2Node_AnimNodeReference::IsCompatibleWithGraph(UEdGraph const* TargetGraph) const
{
	return Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph)) != nullptr;
}

FText UK2Node_AnimNodeReference::GetTooltipText() const
{
	return LOCTEXT("NodeReferenceTooltip", "Gets a reference to an anim graph node in this Animation Blueprint");
}

#undef LOCTEXT_NAMESPACE
