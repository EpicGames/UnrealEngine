// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_PlayMontage.h"

#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "PlayMontageCallbackProxy.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_PlayMontage)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_PlayMontage::UK2Node_PlayMontage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UPlayMontageCallbackProxy, CreateProxyObjectForPlayMontage);
	ProxyFactoryClass = UPlayMontageCallbackProxy::StaticClass();
	ProxyClass = UPlayMontageCallbackProxy::StaticClass();
}

FText UK2Node_PlayMontage::GetTooltipText() const
{
	return LOCTEXT("K2Node_PlayMontage_Tooltip", "Plays a Montage on a SkeletalMeshComponent");
}

FText UK2Node_PlayMontage::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("PlayMontage", "Play Montage");
}

FText UK2Node_PlayMontage::GetMenuCategory() const
{
	return LOCTEXT("PlayMontageCategory", "Animation|Montage");
}

void UK2Node_PlayMontage::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	Super::GetPinHoverText(Pin, HoverTextOut);

	static const FName NAME_OnNotifyBegin = FName(TEXT("OnNotifyBegin"));
	static const FName NAME_OnNotifyEnd = FName(TEXT("OnNotifyEnd"));

	if (Pin.PinName == NAME_OnNotifyBegin)
	{
		FText ToolTipText = LOCTEXT("K2Node_PlayMontage_OnNotifyBegin_Tooltip", "Event called when using a PlayMontageNotify or PlayMontageNotifyWindow Notify in a Montage.");
		HoverTextOut = FString::Printf(TEXT("%s\n%s"), *ToolTipText.ToString(), *HoverTextOut);
	}
	else if (Pin.PinName == NAME_OnNotifyEnd)
	{
		FText ToolTipText = LOCTEXT("K2Node_PlayMontage_OnNotifyEnd_Tooltip", "Event called when using a PlayMontageNotifyWindow Notify in a Montage.");
		HoverTextOut = FString::Printf(TEXT("%s\n%s"), *ToolTipText.ToString(), *HoverTextOut);
	}
}

#undef LOCTEXT_NAMESPACE
