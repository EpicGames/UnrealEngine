// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_TransitionRuleGetter.h"

#include "AnimGraphNode_Base.h"
#include "AnimStateNode.h"
#include "AnimStateTransitionNode.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimationAsset.h"
#include "AnimationGraphSchema.h"
#include "AnimationTransitionSchema.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintActionFilter.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"


/////////////////////////////////////////////////////
// UK2Node_TransitionRuleGetter

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_TransitionRuleGetter)

#define LOCTEXT_NAMESPACE "TransitionRuleGetter"

UK2Node_TransitionRuleGetter::UK2Node_TransitionRuleGetter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_TransitionRuleGetter::AllocateDefaultPins()
{
	UEdGraphPin* OutputPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Float, TEXT("Output"));

	OutputPin->PinFriendlyName = GetFriendlyName(GetterType);

	PreloadObject(AssociatedAnimAssetPlayerNode);
	PreloadObject(AssociatedStateNode);

	Super::AllocateDefaultPins();
}

void UK2Node_TransitionRuleGetter::PreloadRequiredAssets()
{
	PreloadObject(AssociatedAnimAssetPlayerNode);
	PreloadObject(AssociatedStateNode);
	Super::PreloadRequiredAssets();
}

FText UK2Node_TransitionRuleGetter::GetFriendlyName(ETransitionGetter::Type TypeID)
{
	FText FriendlyName;
	switch (TypeID)
	{
	case ETransitionGetter::AnimationAsset_GetCurrentTime:
		FriendlyName = LOCTEXT("AnimationAssetTimeElapsed", "CurrentTime");
		break;
	case ETransitionGetter::AnimationAsset_GetLength:
		FriendlyName = LOCTEXT("AnimationAssetSequenceLength", "Length");
		break;
	case ETransitionGetter::AnimationAsset_GetCurrentTimeFraction:
		FriendlyName = LOCTEXT("AnimationAssetFractionalTimeElapsed", "CurrentTime (ratio)");
		break;
	case ETransitionGetter::AnimationAsset_GetTimeFromEnd:
		FriendlyName = LOCTEXT("AnimationAssetTimeRemaining", "TimeRemaining");
		break;
	case ETransitionGetter::AnimationAsset_GetTimeFromEndFraction:
		FriendlyName = LOCTEXT("AnimationAssetFractionalTimeRemaining", "TimeRemaining (ratio)");
		break;
	case ETransitionGetter::CurrentState_ElapsedTime:
		FriendlyName = LOCTEXT("CurrentStateElapsedTime", "Elapsed State Time");
		break;
	case ETransitionGetter::CurrentState_GetBlendWeight:
		FriendlyName = LOCTEXT("CurrentStateBlendWeight", "State's Blend Weight");
		break;
	case ETransitionGetter::ArbitraryState_GetBlendWeight:
		FriendlyName = LOCTEXT("ArbitraryStateBlendWeight", "BlendWeight");
		break;
	case ETransitionGetter::CurrentTransitionDuration:
		FriendlyName = LOCTEXT("CrossfadeDuration", "Crossfade Duration");
		break;
	default:
		ensure(false);
		break;
	}

	return FriendlyName;
}

FText UK2Node_TransitionRuleGetter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	// @TODO: FText::Format() is slow... consider caching this tooltip like 
	//        we do for a lot of the BP nodes now (unfamiliar with this 
	//        node's purpose, so hesitant to muck with this at this time).

	if (AssociatedAnimAssetPlayerNode != NULL)
	{
		UAnimationAsset* BoundAsset = AssociatedAnimAssetPlayerNode->GetAnimationAsset();
		if (BoundAsset)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("BoundAsset"), FText::FromString(BoundAsset->GetName()));
			return  FText::Format(LOCTEXT("AnimationAssetInfoGetterTitle", "{BoundAsset} Asset"), Args);
		}
		else
		{
			return LOCTEXT("AnimationAssetInfoGetterTitle_None", "Animation Asset");
		}
	}
	else if (AssociatedStateNode != NULL)
	{
		if (UAnimStateNode* State = AssociatedStateNode)
		{
			const FString OwnerName = State->GetOuter()->GetName();

			FFormatNamedArguments Args;
			Args.Add(TEXT("OwnerName"), FText::FromString(OwnerName));
			Args.Add(TEXT("StateName"), FText::FromString(State->GetStateName()));
			return FText::Format(LOCTEXT("StateInfoGetterTitle", "{OwnerName}.{StateName} State"), Args);
		}
		else
		{
			return LOCTEXT("StateInfoGetterTitle_None", "State");
		}
	}
	else if (GetterType == ETransitionGetter::CurrentTransitionDuration)
	{
		return LOCTEXT("TransitionDuration", "Transition");
	}
	else if (GetterType == ETransitionGetter::CurrentState_ElapsedTime ||
			 GetterType == ETransitionGetter::CurrentState_GetBlendWeight)
	{
		return LOCTEXT("CurrentState", "Current State");
	}

	return Super::GetNodeTitle(TitleType);
}

void UK2Node_TransitionRuleGetter::GetStateSpecificAnimGraphSchemaMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const UAnimBlueprint* AnimBlueprint, UAnimStateNode* StateNode) const
{
	auto UiSpecOverride = [](const FBlueprintActionContext& /*Context*/, const IBlueprintNodeBinder::FBindingSet& Bindings, FBlueprintActionUiSpec* UiSpecOut, UAnimStateNode* AnimStateNode)
	{
		const FString OwnerName = AnimStateNode->GetOuter()->GetName();
		UiSpecOut->MenuName = FText::Format(LOCTEXT("TransitionRuleGetterTitle", "Current {0} for state '{1}.{2}'"), 
			UK2Node_TransitionRuleGetter::GetFriendlyName(ETransitionGetter::ArbitraryState_GetBlendWeight), 
			FText::FromString(OwnerName), 
			FText::FromString(AnimStateNode->GetStateName()));
	};

	auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, UAnimStateNode* AnimStateNode)
	{
		UK2Node_TransitionRuleGetter* NewNodeTyped = CastChecked<UK2Node_TransitionRuleGetter>(NewNode);
		NewNodeTyped->AssociatedStateNode = AnimStateNode;
		NewNodeTyped->GetterType = ETransitionGetter::ArbitraryState_GetBlendWeight;
	};

	UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create( UK2Node_TransitionRuleGetter::StaticClass(), nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda, StateNode) );
	Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, StateNode);
	ActionRegistrar.AddBlueprintAction( AnimBlueprint, Spawner );
}

void UK2Node_TransitionRuleGetter::GetStateSpecificAnimTransitionSchemaMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const UAnimBlueprint* AnimBlueprint, UAnimStateNode* StateNode) const
{
	// Offer options from the source state

	// Sequence player positions
	ETransitionGetter::Type SequenceSpecificGetters[] =
	{
		ETransitionGetter::AnimationAsset_GetCurrentTime,
		ETransitionGetter::AnimationAsset_GetLength,
		ETransitionGetter::AnimationAsset_GetCurrentTimeFraction,
		ETransitionGetter::AnimationAsset_GetTimeFromEnd,
		ETransitionGetter::AnimationAsset_GetTimeFromEndFraction
	};

	// Using the State Machine's graph, find all asset players nodes
	TArray<UK2Node*> AssetPlayers;
	StateNode->BoundGraph->GetNodesOfClassEx<UAnimGraphNode_Base, UK2Node>(/*out*/ AssetPlayers);

	for (int32 TypeIndex = 0; TypeIndex < UE_ARRAY_COUNT(SequenceSpecificGetters); ++TypeIndex)
	{
		for (auto NodeIt = AssetPlayers.CreateConstIterator(); NodeIt; ++NodeIt)
		{
			UAnimGraphNode_Base* AnimNode = CastChecked<UAnimGraphNode_Base>(*NodeIt);

			if (AnimNode->DoesSupportTimeForTransitionGetter())
			{
				UAnimationAsset * AnimAsset = AnimNode->GetAnimationAsset();
				if (AnimAsset)
				{
					auto UiSpecOverride = [](const FBlueprintActionContext& /*Context*/, const IBlueprintNodeBinder::FBindingSet& Bindings, FBlueprintActionUiSpec* UiSpecOut, FString AssetName, TEnumAsByte<ETransitionGetter::Type> InGetterType)
					{
						UiSpecOut->Category = LOCTEXT("AssetPlayer", "Asset Player");

						FFormatNamedArguments Args;
						Args.Add(TEXT("NodeName"), UK2Node_TransitionRuleGetter::GetFriendlyName(InGetterType));
						Args.Add(TEXT("AssetName"), FText::FromString(AssetName));
						FText Title = FText::Format(LOCTEXT("TransitionFor", "{NodeName} for '{AssetName}'"), Args);
						UiSpecOut->MenuName = Title;
					};

					auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, UAnimGraphNode_Base* InAssociatedAnimAssetPlayerNode, TEnumAsByte<ETransitionGetter::Type> InGetterType)
					{
						UK2Node_TransitionRuleGetter* NewNodeTyped = CastChecked<UK2Node_TransitionRuleGetter>(NewNode);
						NewNodeTyped->AssociatedAnimAssetPlayerNode = InAssociatedAnimAssetPlayerNode;
						NewNodeTyped->GetterType = InGetterType;
					};

					// Prepare the node spawner
					UAnimGraphNode_Base* AssociatedAnimNode = AnimNode;
					const FString AssetName = AnimAsset->GetName();

					TEnumAsByte<ETransitionGetter::Type> TransitionGetterType = SequenceSpecificGetters[TypeIndex];

					UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create( UK2Node_TransitionRuleGetter::StaticClass(), nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda, AssociatedAnimNode, TransitionGetterType) );
					Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, AssetName, TransitionGetterType);
					ActionRegistrar.AddBlueprintAction( AnimBlueprint, Spawner );
				}
			}
		}
	}
}

void UK2Node_TransitionRuleGetter::GetStateSpecificMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const UAnimBlueprint* AnimBlueprint) const
{
	TArray<UAnimStateNode*> States;
	FBlueprintEditorUtils::GetAllNodesOfClass(AnimBlueprint, /*out*/ States);

	// Go through all states to generate possible menu actions
	for (auto StateIt = States.CreateIterator(); StateIt; ++StateIt)
	{
		UAnimStateNode* StateNode = *StateIt;

		GetStateSpecificAnimGraphSchemaMenuActions(ActionRegistrar, AnimBlueprint, StateNode);
		GetStateSpecificAnimTransitionSchemaMenuActions(ActionRegistrar, AnimBlueprint, StateNode);
	}
}

void UK2Node_TransitionRuleGetter::GetNonStateSpecificMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		// Non-sequence specific ones
		ETransitionGetter::Type NonSpecificGetters[] =
		{
			ETransitionGetter::CurrentTransitionDuration,
			ETransitionGetter::CurrentState_ElapsedTime,
			ETransitionGetter::CurrentState_GetBlendWeight
		};

		for (int32 TypeIndex = 0; TypeIndex < UE_ARRAY_COUNT(NonSpecificGetters); ++TypeIndex)
		{
			auto UiSpecOverride = [](const FBlueprintActionContext& /*Context*/, const IBlueprintNodeBinder::FBindingSet& Bindings, FBlueprintActionUiSpec* UiSpecOut, TEnumAsByte<ETransitionGetter::Type> InGetterType)
			{
				UiSpecOut->Category = LOCTEXT("Transition", "Transition");
				UiSpecOut->MenuName = UK2Node_TransitionRuleGetter::GetFriendlyName(InGetterType);
			};

			auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TEnumAsByte<ETransitionGetter::Type> InGetterType)
			{
				UK2Node_TransitionRuleGetter* NewNodeTyped = CastChecked<UK2Node_TransitionRuleGetter>(NewNode);
				NewNodeTyped->GetterType = InGetterType;
			};

			TEnumAsByte<ETransitionGetter::Type> TransitionGetterType = NonSpecificGetters[TypeIndex];
			UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create( UK2Node_TransitionRuleGetter::StaticClass(), nullptr, UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda, TransitionGetterType) );
			Spawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride, TransitionGetterType);
			ActionRegistrar.AddBlueprintAction( ActionKey, Spawner );
		}
	}
}

void UK2Node_TransitionRuleGetter::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{

}

FText UK2Node_TransitionRuleGetter::GetTooltipText() const
{
	return GetNodeTitle(ENodeTitleType::FullTitle);
}

bool UK2Node_TransitionRuleGetter::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Cast<UAnimationGraphSchema>(Schema) != NULL || Cast<UAnimationTransitionSchema>(Schema) != NULL;
}

UAnimStateTransitionNode* GetTransitionNodeFromGraph(const FAnimBlueprintDebugData& DebugData, const UEdGraph* Graph)
{
	if (const TWeakObjectPtr<UAnimStateTransitionNode>* TransNodePtr = DebugData.TransitionGraphToNodeMap.Find(Graph))
	{
		return TransNodePtr->Get();
	}

	if (const TWeakObjectPtr<UAnimStateTransitionNode>* TransNodePtr = DebugData.TransitionBlendGraphToNodeMap.Find(Graph))
	{
		return TransNodePtr->Get();
	}

	return NULL;
}

bool UK2Node_TransitionRuleGetter::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	if(Filter.Context.Graphs.Num())
	{
		const UEdGraphSchema* Schema = Filter.Context.Graphs[0]->GetSchema();
		if(Cast<UAnimationGraphSchema>(Schema))
		{
			if(GetterType == ETransitionGetter::ArbitraryState_GetBlendWeight)
			{
				// only show the transition nodes if the associated state node is in every graph:
				if( AssociatedStateNode )
				{
					for( auto Blueprint : Filter.Context.Blueprints )
					{
						TArray<UAnimStateNode*> States;
						FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, /*out*/ States);

						if( !States.Contains( AssociatedStateNode ) )
						{
							return true;
						}
					}
					return false;
				}
			}
			return true;
		}
		else if(Cast<UAnimationTransitionSchema>(Schema))
		{
			// Non-sequence specific TransitionRuleGetter nodes have no associated nodes assigned, they are always allowed in AnimationTransitionSchema graphs
			if( AssociatedStateNode == nullptr && AssociatedAnimAssetPlayerNode == nullptr)
			{
				return false;
			}
			else if(AssociatedAnimAssetPlayerNode)
			{
				if(Filter.Context.Blueprints.Num())
				{
					UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Filter.Context.Blueprints[0]);
					check(AnimBlueprint);

					if (UAnimBlueprintGeneratedClass* AnimBlueprintClass = AnimBlueprint->GetAnimBlueprintSkeletonClass())
					{
						// Local function to find the transition node using the context graph and AnimBlueprintDebugData
						auto GetTransitionNodeFromGraphLambda = [](const FAnimBlueprintDebugData& DebugData, const UEdGraph* Graph) -> UAnimStateTransitionNode*
						{
							if (const TWeakObjectPtr<UAnimStateTransitionNode>* TransNodePtr = DebugData.TransitionGraphToNodeMap.Find(Graph))
							{
								return TransNodePtr->Get();
							}

							if (const TWeakObjectPtr<UAnimStateTransitionNode>* TransNodePtr = DebugData.TransitionBlendGraphToNodeMap.Find(Graph))
							{
								return TransNodePtr->Get();
							}

							return NULL;
						};

						// Check if the TransitionNode can be found in the AninBlueprint's debug data
						if (UAnimStateTransitionNode* TransNode = GetTransitionNodeFromGraph(AnimBlueprintClass->GetAnimBlueprintDebugData(), Filter.Context.Graphs[0]))
						{
							if (UAnimStateNode* SourceStateNode = Cast<UAnimStateNode>(TransNode->GetPreviousState()))
							{
								// Check if the AnimAssetPlayerNode's graph is the state machine's bound graph
								if(AssociatedAnimAssetPlayerNode->GetGraph() == SourceStateNode->BoundGraph)
								{
									return false;
								}
							}
						}
					}
				}
			}
		}
	}
	return true;
}

UEdGraphPin* UK2Node_TransitionRuleGetter::GetOutputPin() const
{
	return FindPinChecked(TEXT("Output"));
}

#undef LOCTEXT_NAMESPACE
