// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseDriverDetails.h"

#include "AnimGraphNode_PoseDriver.h"
#include "AnimNodes/AnimNode_PoseDriver.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/PoseAsset.h"
#include "Animation/Skeleton.h"
#include "Animation/SmartName.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "RBF/RBFSolver.h"
#include "SCurveEditor.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
struct FGeometry;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "PoseDriverDetails"

static const FName ColumnId_Target("Target");
TArray< TSharedPtr<FString> > SPDD_TargetRow::DistanceMethodOptions;
TArray< TSharedPtr<FString> > SPDD_TargetRow::FunctionTypeOptions;

class FSoloToggleButton : public SButton
{
public:
	SLATE_BEGIN_ARGS(FSoloToggleButton)
	{}
		SLATE_EVENT(FSimpleDelegate, OnSoloStartAction)
		SLATE_EVENT(FSimpleDelegate, OnSoloEndAction)
		SLATE_ATTRIBUTE(bool, SoloState)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnSoloStartAction = InArgs._OnSoloStartAction;
		OnSoloEndAction = InArgs._OnSoloEndAction;
		SoloState = InArgs._SoloState;

		SButton::Construct(
			SButton::FArguments()
			.Text(FText::FromString(TEXT("S")))
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
			.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
			.ContentPadding(4.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.OnPressed(this, &FSoloToggleButton::OnButtonPressed)
			.OnReleased(this, &FSoloToggleButton::OnButtonReleased)
			.IsFocusable(false)
		);

		SetAppearPressed(SoloState);
	}

	FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		OnSoloStartAction.ExecuteIfBound();
		return FReply::Handled();
	}

private:
	void OnButtonPressed()
	{
		OnSoloStartAction.ExecuteIfBound();
	}

	void OnButtonReleased()
	{
		OnSoloEndAction.ExecuteIfBound();	
	}

	FSimpleDelegate OnSoloStartAction;
	FSimpleDelegate OnSoloEndAction;
	TAttribute< bool > SoloState;
};


void SPDD_TargetRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	TargetInfoPtr = InArgs._TargetInfo;
	PoseDriverDetailsPtr = InArgs._PoseDriverDetails;

	if (DistanceMethodOptions.Num() == 0)
	{
		UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/AnimGraphRuntime.ERBFDistanceMethod"));
		for (int32 i = 0; i < Enum->NumEnums() - 1; i++)
		{
			DistanceMethodOptions.Add(MakeShareable(new FString(Enum->GetDisplayNameTextByIndex(i).ToString())));
		}
	}

	if (FunctionTypeOptions.Num() == 0)
	{
		UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/AnimGraphRuntime.ERBFFunctionType"));
		for (int32 i = 0; i < Enum->NumEnums() - 1; i++)
		{
			FunctionTypeOptions.Add(MakeShareable(new FString(Enum->GetDisplayNameTextByIndex(i).ToString())));
		}
	}

	// Register delegate so TargetInfo can trigger UI expansion
	TSharedPtr<FPDD_TargetInfo> TargetInfo = TargetInfoPtr.Pin();
	TargetInfo->ExpandTargetDelegate.AddSP(this, &SPDD_TargetRow::ExpandTargetInfo);

	SMultiColumnTableRow< TSharedPtr<FPDD_TargetInfo> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef< SWidget > SPDD_TargetRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();

	TSharedPtr<SVerticalBox> TargetEntryVertBox;

	TSharedRef<SWidget> RowWidget = SNew(SBox)
	.Padding(2.f)
	[
		SNew(SBorder)
		.Padding(0.f)
		.ForegroundColor(FLinearColor::White)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SAssignNew(ExpandArea, SExpandableArea)
			.Padding(0.f)
			.InitiallyCollapsed(true)
			.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f))
			.OnAreaExpansionChanged(this, &SPDD_TargetRow::OnTargetExpansionChanged)
			.HeaderContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SPDD_TargetRow::GetTargetTitleText)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.Padding(0,3)
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(150.f)
					.Content()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.FillWidth(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateLambda([this] { return 1.f - this->GetTargetWeight(); })))
						[
							SNew(SSpacer)
						]

						+ SHorizontalBox::Slot()
						.FillWidth(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateLambda([this] { return this->GetTargetWeight(); })))
						[
							SNew(SImage)
							.ColorAndOpacity(this, &SPDD_TargetRow::GetWeightBarColor)
							.Image(FAppStyle::GetBrush("WhiteBrush"))
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(3, 0))
				[
					SNew(SBox)
					.MinDesiredWidth(40.f)
					.MaxDesiredWidth(40.f)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SPDD_TargetRow::GetTargetWeightText)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(3, 0, 6, 0))
				[
					SNew(FSoloToggleButton)
					.SoloState(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this] { return this->IsSoloTarget(); })))
					.OnSoloStartAction(FSimpleDelegate::CreateSP(this, &SPDD_TargetRow::SoloTargetStart))
					.OnSoloEndAction(FSimpleDelegate::CreateSP(this, &SPDD_TargetRow::SoloTargetEnd))
					.ToolTipText(LOCTEXT("SoloTarget", "Hold to solo temporarily. Doube-click to keep solo enabled."))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &SPDD_TargetRow::RemoveTarget), LOCTEXT("RemoveTarget", "Remove Target"))
				]
			]
			.BodyContent()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(TargetEntryVertBox, SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					.VAlign(VAlign_Fill)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(0, 0, 3, 0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Scale", "Scale:"))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.MinDesiredWidth(150.f)
							.Content()
							[
								SNew(SNumericEntryBox<float>)
								.MinSliderValue(0.f)
								.MaxSliderValue(1.f)
								.Value(this, &SPDD_TargetRow::GetScale)
								.OnValueChanged(this, &SPDD_TargetRow::SetScale)
								.AllowSpin(true)
							]
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DrivenName","Drive:"))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SSearchableComboBox)
							.OptionsSource(&PoseDriverDetails->DrivenNameOptions)
							.OnGenerateWidget(this, &SPDD_TargetRow::MakeDrivenNameWidget)
							.OnSelectionChanged(this, &SPDD_TargetRow::OnDrivenNameChanged)
							.Content()						
							[
								SNew(STextBlock)
								.Text(this, &SPDD_TargetRow::GetDrivenNameText)
							]
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("IsHidden", "Hidden:"))
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(SCheckBox)
							.IsChecked_Lambda([this]() { return IsHidden() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.OnCheckStateChanged(this, &SPDD_TargetRow::OnIsHiddenChanged)
							.Padding(FMargin(4.0f, 0.0f))
							.ToolTipText(LOCTEXT("IsHiddenToolTip", "Define if this target should be hidden from debug drawing."))
						]

						+ SHorizontalBox::Slot()
						.FillWidth(1)
						[
							SNew(SSpacer)
						]

					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					.VAlign(VAlign_Fill)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Override", "Override:"))
						]
					
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(SSearchableComboBox)
							.OptionsSource(&DistanceMethodOptions)
							.OnGenerateWidget(this, &SPDD_TargetRow::MakeDrivenNameWidget)
							.OnSelectionChanged(this, &SPDD_TargetRow::OnDistanceMethodChanged)
							.IsEnabled_Lambda([this]() { return IsOverrideEnabled(); })
							.Content()
							[
								SNew(STextBlock)
								.Text(this, &SPDD_TargetRow::GetDistanceMethodAsText)
							]
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(SSearchableComboBox)
							.Visibility_Lambda([this]() { return IsCustomCurveEnabled() ? EVisibility::Collapsed : EVisibility::Visible;  })
							.OptionsSource(&FunctionTypeOptions)
							.OnGenerateWidget(this, &SPDD_TargetRow::MakeDrivenNameWidget)
							.OnSelectionChanged(this, &SPDD_TargetRow::OnFunctionTypeChanged)
							.IsEnabled_Lambda([this]() { return IsOverrideEnabled(); })
							.Content()
							[
								SNew(STextBlock)
								.Text(this, &SPDD_TargetRow::GetFunctionTypeAsText)
							]
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("CustomCurve", "Curve:"))
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(6, 0, 3, 0))
						[
							SNew(SCheckBox)
							.IsChecked_Lambda([this]() { return IsCustomCurveEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.OnCheckStateChanged(this, &SPDD_TargetRow::OnApplyCustomCurveChanged)
							.Padding(FMargin(4.0f, 0.0f))
							.ToolTipText(LOCTEXT("CustomCurveTooltip", "Define a custom response curve for this target."))
						]

						+ SHorizontalBox::Slot()
						.FillWidth(1)
						[
							SNew(SSpacer)
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					.VAlign(VAlign_Fill)
					[
						SNew(SBox)
						.Visibility_Lambda([this]() { return IsCustomCurveEnabled() ? EVisibility::Visible : EVisibility::Collapsed;  })
						.IsEnabled_Lambda([this]() { return IsOverrideEnabled(); })
						.Content()
						[
							SAssignNew(CurveEditor, SCurveEditor)
							.ViewMinInput(0.f)
							.ViewMaxInput(1.f)
							.ViewMinOutput(0.f)
							.ViewMaxOutput(1.f)
							.TimelineLength(1.f)
							.DesiredSize(FVector2D(512, 128))
							.HideUI(true)
						]
					]
				]
			]
		]
	];

	CurveEditor->SetCurveOwner(this);

	// Find number of bones we are reading, which gives the number of entry boxes we need
	int32 NumSourceBones = PoseDriverDetails->GetFirstSelectedPoseDriver()->Node.SourceBones.Num();

	for (int32 BoneIndex = 0; BoneIndex < NumSourceBones; BoneIndex++)
	{
		TargetEntryVertBox->AddSlot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.MaxDesiredWidth(800.f)
			.Content()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex(this, &SPDD_TargetRow::GetTransRotWidgetIndex)

					+SWidgetSwitcher::Slot()
					[
						SNew(SVectorInputBox)
						.AllowSpin(true)
						.X(this, &SPDD_TargetRow::GetTranslation, BoneIndex, EAxis::X)
						.OnXChanged(this, &SPDD_TargetRow::SetTranslation, BoneIndex, EAxis::X)
						.Y(this, &SPDD_TargetRow::GetTranslation, BoneIndex, EAxis::Y)
						.OnYChanged(this, &SPDD_TargetRow::SetTranslation, BoneIndex, EAxis::Y)
						.Z(this, &SPDD_TargetRow::GetTranslation, BoneIndex, EAxis::Z)
						.OnZChanged(this, &SPDD_TargetRow::SetTranslation, BoneIndex, EAxis::Z)
					]

					+ SWidgetSwitcher::Slot()
					[
						SNew(SRotatorInputBox)
						.Roll(this, &SPDD_TargetRow::GetRotation, BoneIndex, EAxis::X)
						.OnRollChanged(this, &SPDD_TargetRow::SetRotation, BoneIndex, EAxis::X)
						.Pitch(this, &SPDD_TargetRow::GetRotation, BoneIndex, EAxis::Y)
						.OnPitchChanged(this, &SPDD_TargetRow::SetRotation, BoneIndex, EAxis::Y)
						.Yaw(this, &SPDD_TargetRow::GetRotation, BoneIndex, EAxis::Z)
						.OnYawChanged(this, &SPDD_TargetRow::SetRotation, BoneIndex, EAxis::Z)
					]
				]
			]
		];
	}

	return RowWidget;
}

FPoseDriverTarget* SPDD_TargetRow::GetTarget() const
{
	FPoseDriverTarget* Target = nullptr;

	UAnimGraphNode_PoseDriver* Driver = GetPoseDriverGraphNode();
	if (Driver)
	{
		Target = &(Driver->Node.PoseTargets[GetTargetIndex()]);
	}

	return Target;
}

UAnimGraphNode_PoseDriver* SPDD_TargetRow::GetPoseDriverGraphNode() const 
{
	UAnimGraphNode_PoseDriver* Driver = nullptr;
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();
	if (PoseDriverDetails.IsValid())
	{
		Driver = PoseDriverDetails->GetFirstSelectedPoseDriver();
	}

	return Driver;
}

bool SPDD_TargetRow::IsEditingRotation() const
{
	UAnimGraphNode_PoseDriver* Driver = GetPoseDriverGraphNode();
	if (Driver)
	{
		return (Driver->Node.DriveSource == EPoseDriverSource::Rotation);
	}
	return false;
}

void SPDD_TargetRow::NotifyTargetChanged()
{
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();
	if (PoseDriverDetails.IsValid())
	{
		PoseDriverDetails->NodePropHandle->NotifyPostChange(EPropertyChangeType::ValueSet); // Will push change to preview node instance
	}
}

int32 SPDD_TargetRow::GetTargetIndex() const
{
	TSharedPtr<FPDD_TargetInfo> TargetInfo = TargetInfoPtr.Pin();
	return TargetInfo->TargetIndex;
}

bool SPDD_TargetRow::IsOverrideEnabled() const
{
	bool bOverrideEnabled = true;
	UAnimGraphNode_PoseDriver* Driver = GetPoseDriverGraphNode();
	if (Driver)
	{
		bOverrideEnabled = Driver->Node.RBFParams.SolverType == ERBFSolverType::Additive;
	}

	return bOverrideEnabled;
}

float SPDD_TargetRow::GetTargetWeight() const
{
	float OutWeight = 0.f;

	UAnimGraphNode_PoseDriver* Driver = GetPoseDriverGraphNode();
	if (Driver)
	{
		FAnimNode_PoseDriver* PreviewNode = Driver->GetPreviewPoseDriverNode();
		if (PreviewNode)
		{
			for (const FRBFOutputWeight& Weight : PreviewNode->OutputWeights)
			{
				if (Weight.TargetIndex == GetTargetIndex())
				{
					OutWeight = Weight.TargetWeight;
				}
			}
		}
	}


	return OutWeight;
}


int32 SPDD_TargetRow::GetTransRotWidgetIndex() const
{
	return IsEditingRotation() ? 1 : 0;
}


TOptional<float> SPDD_TargetRow::GetTranslation(int32 BoneIndex, EAxis::Type Axis) const
{
	float Translation = 0.f;
	const FPoseDriverTarget* Target = GetTarget();
	if (Target && Target->BoneTransforms.IsValidIndex(BoneIndex))
	{
		Translation = static_cast<float>(Target->BoneTransforms[BoneIndex].TargetTranslation.GetComponentForAxis(Axis));
	}

	return Translation;
}

TOptional<float> SPDD_TargetRow::GetRotation(int32 BoneIndex, EAxis::Type Axis) const
{
	float Rotation = 0.f;
	const FPoseDriverTarget* Target = GetTarget();
	if (Target && Target->BoneTransforms.IsValidIndex(BoneIndex))
	{
		Rotation = static_cast<float>(Target->BoneTransforms[BoneIndex].TargetRotation.GetComponentForAxis(Axis));
	}

	return Rotation;
}

TOptional<float> SPDD_TargetRow::GetScale() const
{
	const FPoseDriverTarget* Target = GetTarget();
	return (Target) ? Target->TargetScale : 0.f;
}

bool SPDD_TargetRow::IsCustomCurveEnabled() const
{
	FPoseDriverTarget* Target = GetTarget();
	return (Target && Target->bApplyCustomCurve);
}


void SPDD_TargetRow::OnApplyCustomCurveChanged(const ECheckBoxState NewCheckState)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		Target->bApplyCustomCurve = (NewCheckState == ECheckBoxState::Checked);

		// As a convenience, if curve is empty, add linear mapping here
		if (Target->bApplyCustomCurve && Target->CustomCurve.GetNumKeys() == 0)
		{
			Target->CustomCurve.AddKey(0.f, 0.f);
			Target->CustomCurve.AddKey(1.f, 1.f);
			CurveEditor->ZoomToFitHorizontal(true);
		}

		// Push value to preview
		NotifyTargetChanged();
	}
}

bool SPDD_TargetRow::IsHidden() const
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		return Target->bIsHidden;
	}
	return false;
}


void SPDD_TargetRow::OnIsHiddenChanged(const ECheckBoxState NewCheckState)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		Target->bIsHidden= (NewCheckState == ECheckBoxState::Checked);
		NotifyTargetChanged();
	}
}

FText SPDD_TargetRow::GetDistanceMethodAsText() const
{
	const FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		return FText::FromString(*DistanceMethodOptions[(int32)Target->DistanceMethod]);
	}
	return FText();
}

void SPDD_TargetRow::OnDistanceMethodChanged(TSharedPtr<FString> InItem, ESelectInfo::Type SelectionType)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		for (int32 i = 0; i < DistanceMethodOptions.Num(); i++)
		{
			if (InItem->Compare(*DistanceMethodOptions[i]) == 0)
			{
				Target->DistanceMethod = (ERBFDistanceMethod)i;
				NotifyTargetChanged();
				return;
			}
		}
	}
}

FText SPDD_TargetRow::GetFunctionTypeAsText() const
{
	const FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		return FText::FromString(*FunctionTypeOptions[(int32)Target->FunctionType]);
	}
	return FText();
}

void SPDD_TargetRow::OnFunctionTypeChanged(TSharedPtr<FString> InItem, ESelectInfo::Type SelectionType)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		for (int32 i = 0; i < FunctionTypeOptions.Num(); i++)
		{
			if (InItem->Compare(*FunctionTypeOptions[i]) == 0)
			{
				Target->FunctionType = (ERBFFunctionType)i;
				NotifyTargetChanged();
				return;
			}
		}
	}
}

FText SPDD_TargetRow::GetDrivenNameText() const
{
	FPoseDriverTarget* Target = GetTarget();
	return (Target) ? FText::FromName(Target->DrivenName) : FText::GetEmpty();
}

void SPDD_TargetRow::OnDrivenNameChanged(TSharedPtr<FString> NewName, ESelectInfo::Type SelectInfo)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target && SelectInfo != ESelectInfo::Direct)
	{
		Target->DrivenName = FName(*NewName.Get());
		NotifyTargetChanged();
	}
}

TSharedRef<SWidget> SPDD_TargetRow::MakeDrivenNameWidget(TSharedPtr<FString> InItem)
{
	return 
		SNew(STextBlock)
		.Text(FText::FromString(*InItem));
}

FText SPDD_TargetRow::GetTargetTitleText() const
{
	FPoseDriverTarget* Target = GetTarget();
	FString DrivenName = (Target) ? Target->DrivenName.ToString() : TEXT("");
	FString Title = FString::Printf(TEXT("%d - %s"), GetTargetIndex(), *DrivenName);
	return FText::FromString(Title);
}

FText SPDD_TargetRow::GetTargetWeightText() const
{
	FString Title = FString::Printf(TEXT("%2.1f"), GetTargetWeight() * 100.f);
	return FText::FromString(Title);
}

FSlateColor SPDD_TargetRow::GetWeightBarColor() const
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetPoseDriverGraphNode();
	if (PoseDriver)
	{
		return PoseDriver->GetColorFromWeight(GetTargetWeight());
	}
	return FLinearColor::Black;
}

void SPDD_TargetRow::SetTranslation(float NewTrans, int32 BoneIndex, EAxis::Type Axis)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target && Target->BoneTransforms.IsValidIndex(BoneIndex))
	{
		Target->BoneTransforms[BoneIndex].TargetTranslation.SetComponentForAxis(Axis, NewTrans);
	}

	NotifyTargetChanged();
}

void SPDD_TargetRow::SetRotation(float NewRot, int32 BoneIndex, EAxis::Type Axis)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target && Target->BoneTransforms.IsValidIndex(BoneIndex))
	{
		Target->BoneTransforms[BoneIndex].TargetRotation.SetComponentForAxis(Axis, NewRot);
	}

	NotifyTargetChanged();
}

void SPDD_TargetRow::SetScale(float NewScale)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		Target->TargetScale = NewScale;
		NotifyTargetChanged();
	}
}

void SPDD_TargetRow::SetDrivenNameText(const FText& NewText, ETextCommit::Type CommitType)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		Target->DrivenName = FName(*NewText.ToString());
		NotifyTargetChanged();
	}
}



void SPDD_TargetRow::RemoveTarget()
{
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();
	if (PoseDriverDetails.IsValid())
	{
		PoseDriverDetails->RemoveTarget(GetTargetIndex()); // This will remove me
	}
}

void SPDD_TargetRow::SoloTargetStart()
{
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();
	if (PoseDriverDetails.IsValid())
	{
		PoseDriverDetails->SetSoloTarget(GetTargetIndex()); 
	}
}

void SPDD_TargetRow::SoloTargetEnd()
{
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();
	if (PoseDriverDetails.IsValid())
	{
		PoseDriverDetails->SetSoloTarget(INDEX_NONE); 
	}
}

bool SPDD_TargetRow::IsSoloTarget() const
{
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();
	if (PoseDriverDetails.IsValid())
	{
		return PoseDriverDetails->GetSoloTarget() == GetTargetIndex();
	}

	return false;
}

void SPDD_TargetRow::ExpandTargetInfo()
{
	ExpandArea->SetExpanded(true); // This fires OnTargetExpansionChanged which causes item to get selected as well, which is fine
}

void SPDD_TargetRow::OnTargetExpansionChanged(bool bExpanded)
{
	TSharedPtr<FPoseDriverDetails> PoseDriverDetails = PoseDriverDetailsPtr.Pin();
	if (PoseDriverDetails.IsValid())
	{
		PoseDriverDetails->SelectTarget(GetTargetIndex(), false);
	}
}

////// curve editor interface

TArray<FRichCurveEditInfoConst> SPDD_TargetRow::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;

	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		Curves.Add(&(Target->CustomCurve));
	}

	return Curves;
}

void SPDD_TargetRow::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	if (FPoseDriverTarget* Target = GetTarget())
	{
		Curves.Add(&(Target->CustomCurve));
	}
}

TArray<FRichCurveEditInfo> SPDD_TargetRow::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;

	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		Curves.Add(&(Target->CustomCurve));
	}

	return Curves;
}

void SPDD_TargetRow::ModifyOwner()
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetPoseDriverGraphNode();
	if (PoseDriver)
	{
		PoseDriver->Modify();
	}
}

TArray<const UObject*> SPDD_TargetRow::GetOwners() const
{
	TArray<const UObject*> Owners;

	UAnimGraphNode_PoseDriver* PoseDriver = GetPoseDriverGraphNode();
	if (PoseDriver)
	{
		Owners.Add(GetPoseDriverGraphNode());
	}

	return Owners;
}

void SPDD_TargetRow::MakeTransactional()
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetPoseDriverGraphNode();
	if (PoseDriver)
	{
		PoseDriver->SetFlags(PoseDriver->GetFlags() | RF_Transactional);
	}
}

bool SPDD_TargetRow::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	FPoseDriverTarget* Target = GetTarget();
	if (Target)
	{
		return CurveInfo.CurveToEdit == &(Target->CustomCurve);
	}
	else
	{
		return false;
	}
}

void SPDD_TargetRow::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	NotifyTargetChanged();
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FPoseDriverDetails::MakeInstance()
{
	return MakeShareable(new FPoseDriverDetails);
}

TSharedRef<SWidget> FPoseDriverDetails::GetToolsMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyFromPoseAsset", "Copy All From PoseAsset"), 
		LOCTEXT("CopyFromPoseAssetTooltip", "Copy target positions from PoseAsset. Will overwrite any existing targets."), 
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPoseDriverDetails::ClickedOnCopyFromPoseAsset),
			FCanExecuteAction::CreateRaw(this, &FPoseDriverDetails::CopyFromPoseAssetIsEnabled)
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AutoTargetScale", "Auto Scale"),
		LOCTEXT("AutoTargetScaleTooltip", "Automatically set all Scale factors, based on distance to nearest neighbour targets."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPoseDriverDetails::ClickedOnAutoScaleFactors),
			FCanExecuteAction::CreateRaw(this, &FPoseDriverDetails::AutoScaleFactorsIsEnabled)
		)
	);

	return MenuBuilder.MakeWidget();
}

FSlateColor FPoseDriverDetails::GetToolsForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");
	static const FName DefaultForegroundName("DefaultForeground");

	return ToolsButton->IsHovered() ? FAppStyle::GetSlateColor(InvertedForegroundName) : FAppStyle::GetSlateColor(DefaultForegroundName);
}

void FPoseDriverDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& PoseTargetsCategory = DetailBuilder.EditCategory("PoseTargets");

	// Get a property handle because we might need to to invoke NotifyPostChange
	NodePropHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_PoseDriver, Node));

	// Bind delegate to PoseAsset changing
	TSharedRef<IPropertyHandle> PoseAssetPropHandle = DetailBuilder.GetProperty("Node.PoseAsset");
	PoseAssetPropHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FPoseDriverDetails::OnPoseAssetChanged));

	// Bind delegate to source bones changing
	TSharedRef<IPropertyHandle> SourceBonesPropHandle = DetailBuilder.GetProperty("Node.SourceBones");
	SourceBonesPropHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FPoseDriverDetails::OnSourceBonesChanged));

	// Cache set of selected things
	SelectedObjectsList = DetailBuilder.GetSelectedObjects();

	// Create list of driven names
	UpdateDrivenNameOptions();

	TSharedPtr<IPropertyHandle> PoseTargetsProperty = DetailBuilder.GetProperty("Node.PoseTargets", UAnimGraphNode_PoseDriver::StaticClass());
	// Update target infos when resetting the property 
	PoseTargetsProperty->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FPoseDriverDetails::UpdateTargetInfosList));

	IDetailPropertyRow& PoseTargetsRow = PoseTargetsCategory.AddProperty(PoseTargetsProperty);
	PoseTargetsRow.ShowPropertyButtons(false);
	FDetailWidgetRow& PoseTargetRowWidget = PoseTargetsRow.CustomWidget();

	TSharedRef<SWidget> PoseTargetsHeaderWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "RoundButton")
			.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
			.ContentPadding(FMargin(2, 0))
			.OnClicked(this, &FPoseDriverDetails::ClickedAddTarget)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0, 1))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Plus"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(2, 0, 0, 0))
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(LOCTEXT("AddTarget", "Add Target"))
					.ShadowOffset(FVector2D(1, 1))
				]
			]
		];

	PoseTargetsCategory.HeaderContent(PoseTargetsHeaderWidget);

	static const FName DefaultForegroundName("DefaultForeground");

	PoseTargetRowWidget.WholeRowContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SAssignNew(TargetListWidget, SPDD_TargetListType)
			.ListItemsSource(&TargetInfos)
			.OnGenerateRow(this, &FPoseDriverDetails::GenerateTargetRow)
			.SelectionMode(ESelectionMode::SingleToggle)
			.OnSelectionChanged(this, &FPoseDriverDetails::OnTargetSelectionChanged)
			.HeaderRow
			(
				SNew(SHeaderRow)
				.Visibility(EVisibility::Collapsed)
				+ SHeaderRow::Column(ColumnId_Target)
			)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(6, 0, 3, 0))
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return IsSoloDrivenOnly() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged(this, &FPoseDriverDetails::OnSoloDrivenOnlyChanged)
				.Padding(FMargin(4.0f, 0.0f))
				.ToolTipText(LOCTEXT("SoloDrivenOnlyHelp", "Only solo the driven poses or curves and leave the source joint(s) in place."))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SoloDrivenOnly", "Solo Driven Pose/Curve Only"))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(2, 2)
			.AutoWidth()
			[
				SAssignNew(ToolsButton, SComboButton)
				.ContentPadding(3.f)
				.ForegroundColor(this, &FPoseDriverDetails::GetToolsForegroundColor)
				.ButtonStyle(FAppStyle::Get(), "ToggleButton") // Use the tool bar item style for this button
				.OnGetMenuContent(this, &FPoseDriverDetails::GetToolsMenuContent)
				.ButtonContent()
				[
					SNew(STextBlock).Text(LOCTEXT("ViewButton", "Tools "))
				]
			]
		]
	];

	// Update target list from selected pose driver node
	UpdateTargetInfosList();

	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();

	PoseDriver->SelectedTargetChangeDelegate.AddSP(this, &FPoseDriverDetails::SelectedTargetChanged);
}

void FPoseDriverDetails::SelectedTargetChanged()
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		SelectTarget(PoseDriver->SelectedTargetIndex, true);
	}
}

TSharedRef<ITableRow> FPoseDriverDetails::GenerateTargetRow(TSharedPtr<FPDD_TargetInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InInfo.IsValid());

	return
		SNew(SPDD_TargetRow, OwnerTable)
		.TargetInfo(InInfo)
		.PoseDriverDetails(SharedThis(this));
}

void FPoseDriverDetails::OnTargetSelectionChanged(TSharedPtr<FPDD_TargetInfo> InInfo, ESelectInfo::Type SelectInfo)
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		if (InInfo.IsValid())
		{
			PoseDriver->SelectedTargetIndex = InInfo->TargetIndex;
		}
		else
		{
			PoseDriver->SelectedTargetIndex = INDEX_NONE;
		}
	}
}

void FPoseDriverDetails::OnPoseAssetChanged()
{
	UpdateDrivenNameOptions();
}

void FPoseDriverDetails::OnSourceBonesChanged()
{
	for (const TWeakObjectPtr<UObject>& Object : SelectedObjectsList)
	{
		UAnimGraphNode_PoseDriver* PoseDriver = Cast<UAnimGraphNode_PoseDriver>(Object.Get());
		if(PoseDriver)
		{
			PoseDriver->ReserveTargetTransforms();
		}
	}

	UpdateTargetInfosList();
}

void FPoseDriverDetails::OnSoloDrivenOnlyChanged(const ECheckBoxState NewCheckState)
{	
	bool bDrivenOnly = (NewCheckState == ECheckBoxState::Checked);

	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver&& PoseDriver->Node.bSoloDrivenOnly != bDrivenOnly)
	{
		PoseDriver->Node.bSoloDrivenOnly = bDrivenOnly;

		// Set this as an interactive change to avoid triggering the "node needs recompile"
		// message, since it won't be needed anyway.
		NodePropHandle->NotifyPostChange(EPropertyChangeType::Interactive);
	}
}

UAnimGraphNode_PoseDriver* FPoseDriverDetails::GetFirstSelectedPoseDriver() const
{
	for (const TWeakObjectPtr<UObject>& Object : SelectedObjectsList)
	{
		UAnimGraphNode_PoseDriver* TestPoseDriver = Cast<UAnimGraphNode_PoseDriver>(Object.Get());
		if (TestPoseDriver != nullptr && !TestPoseDriver->IsTemplate())
		{
			return TestPoseDriver;
		}
	}

	return nullptr;
}

void FPoseDriverDetails::UpdateTargetInfosList()
{
	TargetInfos.Empty();
	
	if(SelectedObjectsList.Num() == 1)
	{
		UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
		if (PoseDriver)
		{
			for (int32 i = 0; i < PoseDriver->Node.PoseTargets.Num(); i++)
			{
				TargetInfos.Add( FPDD_TargetInfo::Make(i) );
			}
		}
	}

	TargetListWidget->RequestListRefresh();
}

void FPoseDriverDetails::UpdateDrivenNameOptions()
{
	DrivenNameOptions.Empty();

	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		// None is always an option
		DrivenNameOptions.Add(MakeShareable(new FString()));

		// Compile list of all curves in Skeleton
		if (PoseDriver->Node.DriveOutput == EPoseDriverOutput::DriveCurves)
		{
			USkeleton* Skeleton = PoseDriver->GetAnimBlueprint()->TargetSkeleton;
			if (Skeleton)
			{
				TArray<FName> NameArray;
				Skeleton->GetCurveMetaDataNames(NameArray);
				NameArray.Sort(FNameLexicalLess());

				for (const FName& CurveName : NameArray)
				{
					DrivenNameOptions.Add(MakeShareable(new FString(CurveName.ToString())));
				}
			}
		}
		// Compile list of all poses in PoseAsset
		else
		{
			if (PoseDriver->Node.PoseAsset)
			{
				for (const FName& Name : PoseDriver->Node.PoseAsset->GetPoseFNames())
				{
					DrivenNameOptions.Add(MakeShareable(new FString(Name.ToString())));
				}
			}
		}
	}
}


void FPoseDriverDetails::ClickedOnCopyFromPoseAsset()
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		PoseDriver->CopyTargetsFromPoseAsset();

		// Also update radius/scales
		float MaxDist;
		PoseDriver->AutoSetTargetScales(MaxDist);
		PoseDriver->Node.RBFParams.Radius = 0.5f * MaxDist; // reasonable default radius

		UpdateTargetInfosList();
		NodePropHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

bool FPoseDriverDetails::CopyFromPoseAssetIsEnabled() const
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	return(PoseDriver && PoseDriver->Node.PoseAsset);
}


void FPoseDriverDetails::ClickedOnAutoScaleFactors()
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		float MaxDist;
		PoseDriver->AutoSetTargetScales(MaxDist);
		PoseDriver->Node.RBFParams.Radius = 0.5f * MaxDist; // reasonable default radius
		NodePropHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

bool FPoseDriverDetails::AutoScaleFactorsIsEnabled() const
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	return(PoseDriver && PoseDriver->Node.PoseTargets.Num() > 1);
}

FReply FPoseDriverDetails::ClickedAddTarget()
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		PoseDriver->AddNewTarget();
		UpdateTargetInfosList();
		NodePropHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd); // will push changes to preview node instance
	}
	return FReply::Handled();
}

void FPoseDriverDetails::RemoveTarget(int32 TargetIndex)
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver && PoseDriver->Node.PoseTargets.IsValidIndex(TargetIndex))
	{
		PoseDriver->Node.PoseTargets.RemoveAt(TargetIndex);
		UpdateTargetInfosList();
		NodePropHandle->NotifyPostChange(EPropertyChangeType::ArrayRemove); // will push changes to preview node instance
	}
}

void FPoseDriverDetails::SetSoloTarget(int32 TargetIndex)
{
	UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		FAnimNode_PoseDriver& Node = PoseDriver->Node;

		if (PoseDriver->Node.PoseTargets.IsValidIndex(TargetIndex))
		{
			Node.SoloTargetIndex = TargetIndex;
		}
		else
		{
			Node.SoloTargetIndex = INDEX_NONE;
		}

		// Set this as an interactive change to avoid triggering the "node needs recompile"
		// message, since it won't be needed anyway.
		NodePropHandle->NotifyPostChange(EPropertyChangeType::Interactive);
	}
}

int32 FPoseDriverDetails::GetSoloTarget() const
{
	const UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();
	if (PoseDriver)
	{
		return PoseDriver->Node.SoloTargetIndex;
	}

	return INDEX_NONE;
}

bool FPoseDriverDetails::IsSoloDrivenOnly() const
{
	const UAnimGraphNode_PoseDriver* PoseDriver = GetFirstSelectedPoseDriver();

	return PoseDriver ? PoseDriver->Node.bSoloDrivenOnly : true;
}

void FPoseDriverDetails::SelectTarget(int32 TargetIndex, bool bExpandTarget)
{
	if (TargetInfos.IsValidIndex(TargetIndex))
	{
		TargetListWidget->SetSelection(TargetInfos[TargetIndex]);

		if (bExpandTarget)
		{
			TargetInfos[TargetIndex]->ExpandTargetDelegate.Broadcast();
		}
	}
}

#undef LOCTEXT_NAMESPACE

