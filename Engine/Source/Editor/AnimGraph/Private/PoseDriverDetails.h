// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Curves/CurveOwnerInterface.h"
#include "Curves/RichCurve.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Axis.h"
#include "Misc/Optional.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FPoseDriverDetails;
class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class ITableRow;
class SComboButton;
class SCurveEditor;
class SExpandableArea;
class STableViewBase;
class SWidget;
class UAnimGraphNode_PoseDriver;
class UObject;
struct FPoseDriverTarget;

/** Entry in backing list for target list widget */
struct FPDD_TargetInfo 
{
	int32 TargetIndex;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FPDD_TargetInfo> Make(int32 InTargetIndex)
	{
		return MakeShareable(new FPDD_TargetInfo(InTargetIndex));
	}

	/** Executed when we want to expand this target info UI */
	FSimpleMulticastDelegate ExpandTargetDelegate;

protected:
	/** Hidden constructor, always use Make above */
	FPDD_TargetInfo(int32 InTargetIndex)
		: TargetIndex(InTargetIndex)
	{}
};

/** Type of target list widget */
typedef SListView< TSharedPtr<FPDD_TargetInfo> > SPDD_TargetListType;


/** Widget for displaying info on a paricular target */
class SPDD_TargetRow : public SMultiColumnTableRow< TSharedPtr<FPDD_TargetInfo> > ,
	public FCurveOwnerInterface
{
public:

	SLATE_BEGIN_ARGS(SPDD_TargetRow) {}
		SLATE_ARGUMENT(TWeakPtr<FPDD_TargetInfo>, TargetInfo)
		SLATE_ARGUMENT(TWeakPtr<FPoseDriverDetails>, PoseDriverDetails)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	/** Return underlying FPoseDriverTarget this widget represents */
	FPoseDriverTarget* GetTarget() const;
	/** Get the pose drive node we are editing */
	UAnimGraphNode_PoseDriver* GetPoseDriverGraphNode() const;
	/** If we are editing rotation or translation */
	bool IsEditingRotation() const;
	/** Call when target is modified, so change is propagated to preview instance */
	void NotifyTargetChanged();
	/** Get current weight of this target in preview */
	float GetTargetWeight() const;

	/** Get index of target this represents on pose driver */
	int32 GetTargetIndex() const;

	/** If we should enable the display of override controls */
	bool IsOverrideEnabled() const;

	int32 GetTransRotWidgetIndex() const;
	TOptional<float> GetTranslation(int32 BoneIndex, EAxis::Type Axis) const;
	TOptional<float> GetRotation(int32 BoneIndex, EAxis::Type Axis) const;
	TOptional<float> GetScale() const;
	FText GetTargetTitleText() const;
	FText GetTargetWeightText() const;
	FSlateColor GetWeightBarColor() const;

	void SetTranslation(float NewTrans, int32 BoneIndex, EAxis::Type Axis);
	void SetRotation(float NewRot, int32 BoneIndex, EAxis::Type Axis);
	void SetScale(float NewScale);
	void SetDrivenNameText(const FText& NewText, ETextCommit::Type CommitType);

	void ExpandTargetInfo();
	void OnTargetExpansionChanged(bool bExpanded);
	FText GetDrivenNameText() const;
	void OnDrivenNameChanged(TSharedPtr<FString> NewName, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> MakeDrivenNameWidget(TSharedPtr<FString> InItem);

	bool IsCustomCurveEnabled() const;
	void OnApplyCustomCurveChanged(const ECheckBoxState NewCheckState);

	bool IsHidden() const;
	void OnIsHiddenChanged(const ECheckBoxState NewCheckState);

	FText GetDistanceMethodAsText() const;
	void OnDistanceMethodChanged(TSharedPtr<FString> InItem, ESelectInfo::Type SelectionType);
	FText GetFunctionTypeAsText() const;
	void OnFunctionTypeChanged(TSharedPtr<FString> InItem, ESelectInfo::Type SelectionType);

	/** Remove this target from  */
	void RemoveTarget();

	void SoloTargetStart();
	void SoloTargetEnd();

	// Returns true if the target is currently set to solo.
	bool IsSoloTarget() const;

	/** FCurveOwnerInterface interface */
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	virtual TArray<FRichCurveEditInfo> GetCurves() override;
	virtual void ModifyOwner() override;
	virtual TArray<const UObject*> GetOwners() const override;
	virtual void MakeTransactional() override;
	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;

	/** Expandable area used for this widget */
	TSharedPtr<SExpandableArea> ExpandArea;

	/** Curve editor for custom curves */
	TSharedPtr<SCurveEditor> CurveEditor;

	/** Pointer back to owning customization */
	TWeakPtr<FPoseDriverDetails> PoseDriverDetailsPtr;

	/** Info that this widget represents */
	TWeakPtr<FPDD_TargetInfo> TargetInfoPtr;

	static TArray< TSharedPtr<FString> > DistanceMethodOptions;
	static TArray< TSharedPtr<FString> > FunctionTypeOptions;
};


/** Details customization for PoseDriver node */
class FPoseDriverDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

	void ClickedOnCopyFromPoseAsset();
	bool CopyFromPoseAssetIsEnabled() const;
	void ClickedOnAutoScaleFactors();
	bool AutoScaleFactorsIsEnabled() const;
	FReply ClickedAddTarget();
	TSharedRef<ITableRow> GenerateTargetRow(TSharedPtr<FPDD_TargetInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	void OnTargetSelectionChanged(TSharedPtr<FPDD_TargetInfo> InInfo, ESelectInfo::Type SelectInfo);
	void OnPoseAssetChanged();
	void OnSourceBonesChanged();
	void OnSoloDrivenOnlyChanged(const ECheckBoxState NewCheckState);
	void SelectedTargetChanged();

	/** Get tools popup menu content */
	TSharedRef<SWidget> GetToolsMenuContent();
	FSlateColor GetToolsForegroundColor() const;

	/** Remove a target from node */
	void RemoveTarget(int32 TargetIndex);

	/** Set this target to show at 100% and others at 0% until the next compile, unless
	    it is already a solo target, in which case it will get un-soloed and the normal weight
		computation applies. */
	void SetSoloTarget(int32 TargetIndex);

	/** Returns the current solo target index, or INDEX_NONE if no target is set to solo */
	int32 GetSoloTarget() const;

	/** Returns true if the solo is for driven poses/curves only. */
	bool IsSoloDrivenOnly() const;

	/** Set the currently selected Target */
	void SelectTarget(int32 TargetIndex, bool bExpandTarget);

	/** Util to get the first selected PoseDriver node */
	UAnimGraphNode_PoseDriver* GetFirstSelectedPoseDriver() const;

	/**  Refresh list of TargetInfos, mirroring PoseTargets list on node */
	void UpdateTargetInfosList();

	/** Update list of options for targets to drive (use by combo box) */
	void UpdateDrivenNameOptions();


	/** Cached array of selected objects */
	TArray< TWeakObjectPtr<UObject> > SelectedObjectsList;
	/** Array source for target list */
	TArray< TSharedPtr<FPDD_TargetInfo> > TargetInfos;
	/** List of things a target can drive (curves or morphs), used by combo box */
	TArray< TSharedPtr<FString> > DrivenNameOptions;
	/** Target list widget */
	TSharedPtr<SPDD_TargetListType> TargetListWidget;
	/** Property handle to node */
	TSharedPtr<IPropertyHandle> NodePropHandle;
	/** Pointer to Tools menu button */
	TSharedPtr<SComboButton> ToolsButton;
};
