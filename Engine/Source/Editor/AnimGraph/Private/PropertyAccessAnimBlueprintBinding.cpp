// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAccessAnimBlueprintBinding.h"

#include "AnimBlueprintExtension_PropertyAccess.h"
#include "Animation/AnimBlueprint.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "ScopedTransaction.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "FPropertyAccessAnimBlueprintBinding"

bool FPropertyAccessAnimBlueprintBinding::CanBindToContext(const FContext& InContext) const
{
	if(InContext.Blueprint)
	{
		return InContext.Blueprint->IsA<UAnimBlueprint>();
	}

	return false;
}

TSharedPtr<FExtender> FPropertyAccessAnimBlueprintBinding::MakeBindingMenuExtender(const FContext& InContext, const FBindingMenuArgs& InArgs) const
{
	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();
	Extender->AddMenuExtension("BindingActions", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([InArgs](FMenuBuilder& InMenuBuilder)
	{
		InMenuBuilder.BeginSection("CallSite", LOCTEXT("CallSite", "Call Site"));
		{
			InMenuBuilder.AddSubMenu(
				LOCTEXT("CallSite", "Call Site"),
				LOCTEXT("CallSiteTooltip", "Choose when this property access value will be called & cached for later use when this graph is executed."),
				FNewMenuDelegate::CreateLambda([InArgs](FMenuBuilder& InSubMenuBuilder)
				{
					InSubMenuBuilder.BeginSection("CallSiteSubMenu", LOCTEXT("CallSite", "Call Site"));
					{
						auto AddContextEntry = [InArgs, &InSubMenuBuilder](const FName& InContextId, const FText& InLabel, const FText& InToolTip)
						{
							InSubMenuBuilder.AddMenuEntry(
								InLabel,
								InToolTip,
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([InArgs, InContextId]()
									{
										FScopedTransaction Transaction(LOCTEXT("SetCallSiteTransaction", "Set Call Site"));
										InArgs.OnSetPropertyAccessContextId.ExecuteIfBound(InContextId);
									}),
									FCanExecuteAction::CreateLambda([InArgs, InContextId]()
									{
										if(InArgs.OnCanSetPropertyAccessContextId.IsBound())
										{
											return InArgs.OnCanSetPropertyAccessContextId.Execute(InContextId);
										}
										return false;
									}),
									FIsActionChecked::CreateLambda([InArgs, InContextId]()
									{
										const FName CurrentId = InArgs.OnGetPropertyAccessContextId.IsBound() ? InArgs.OnGetPropertyAccessContextId.Execute() : NAME_None;
										return CurrentId == InContextId;
									})),
								NAME_None,
								EUserInterfaceActionType::RadioButton
							);
						};
						
						AddContextEntry(UAnimBlueprintExtension_PropertyAccess::ContextId_Automatic,
							LOCTEXT("CallSiteAutomatic", "Automatic"),
							LOCTEXT("CallSiteAutomatic_ToolTip", "Automatically determine the call site for this property access based on context and thread safety"));

						AddContextEntry(UAnimBlueprintExtension_PropertyAccess::ContextId_UnBatched_ThreadSafe,
							LOCTEXT("CallSiteThreadSafe", "Thread Safe"),
							LOCTEXT("CallSiteThreadSafe_ToolTip", "Can safely be executed on worker threads"));

						AddContextEntry(UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_GameThreadPreEventGraph,
							LOCTEXT("CallSiteGameThreadPreEventGraph", "Game Thread (Pre-Event Graph)"),
							LOCTEXT("CallSiteGameThreadPreEventGraph_ToolTip", "Executed on the game thread before the event graph is run"));

						AddContextEntry(UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_GameThreadPostEventGraph,
							LOCTEXT("CallSiteGameThreadPostEventGraph", "Game Thread (Post-Event Graph)"),
							LOCTEXT("CallSiteGameThreadPostEventGraph_ToolTip", "Executed on the game thread after the event graph is run"));

						AddContextEntry(UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_WorkerThreadPreEventGraph,
							LOCTEXT("CallSiteWorkerThreadPreEventGraph", "Worker Thread (Pre-Event Graph)"),
							LOCTEXT("CallSiteWorkerThreadPreEventGraph_ToolTip", "Executed on a worker thread before the thread safe event graph is run"));

						AddContextEntry(UAnimBlueprintExtension_PropertyAccess::ContextId_Batched_WorkerThreadPostEventGraph,
							LOCTEXT("CallSiteWorkerThreadPostEventGraph", "Worker Thread (Post-Event Graph)"),
							LOCTEXT("CallSiteWorkerThreadPostEventGraph_ToolTip", "Executed on a worker thread after the thread safe event graph is run"));
					}
					InSubMenuBuilder.EndSection();
				})
			);
		}
		InMenuBuilder.EndSection();
	}));
	return Extender;
}

#undef LOCTEXT_NAMESPACE
