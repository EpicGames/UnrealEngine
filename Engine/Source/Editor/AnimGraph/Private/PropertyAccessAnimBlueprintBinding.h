// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyAccessBlueprintBinding.h"
#include "Templates/SharedPointer.h"

class FExtender;

class FPropertyAccessAnimBlueprintBinding : public IPropertyAccessBlueprintBinding
{
public:
	// IPropertyAccessBlueprintBinding interface
	virtual bool CanBindToContext(const FContext& InContext) const override;
	virtual TSharedPtr<FExtender> MakeBindingMenuExtender(const FContext& InContext, const FBindingMenuArgs& InArgs) const override;
};
