// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blueprint/BlueprintExtension.h"

#include "AnimBlueprintExtension.generated.h"

#define UE_API ANIMGRAPH_API

class UAnimBlueprint;
class IAnimBlueprintCompilationContext;
class IAnimBlueprintCopyTermDefaultsContext;
class IAnimBlueprintCompilationBracketContext;
class IAnimBlueprintExtensionCopyTermDefaultsContext;
class IAnimBlueprintPostExpansionStepContext;
class IAnimBlueprintGeneratedClassCompiledData;
class IAnimBlueprintCompilerCreationContext;
class UAnimGraphNode_Base;
class UEdGraph;

/** Extension that allows per-system data to be held on the anim blueprint, and per-system logic to be executed during compilation */
UCLASS(MinimalAPI)
class UAnimBlueprintExtension : public UBlueprintExtension
{
	GENERATED_BODY()

public:
	/** Request an anim blueprint extension for an anim blueprint. It is illegal to perform this operation once compilation has commenced, use GetExtension instead. */
	template<typename ExtensionType>
	static ExtensionType* RequestExtension(UAnimBlueprint* InAnimBlueprint)
	{
		return CastChecked<ExtensionType>(RequestExtension(InAnimBlueprint, ExtensionType::StaticClass()));
	}
	
	/** Request an anim blueprint extension for an anim blueprint. It is illegal to perform this operation once compilation has commenced, use GetExtension instead. */
	static UE_API UAnimBlueprintExtension* RequestExtension(UAnimBlueprint* InAnimBlueprint, TSubclassOf<UAnimBlueprintExtension> InExtensionType);

	/** Get an already-requested extension for an anim blueprint. Will assert if the extension is not found. */
	template<typename ExtensionType>
	static ExtensionType* GetExtension(UAnimBlueprint* InAnimBlueprint)
	{
		return CastChecked<ExtensionType>(GetExtension(InAnimBlueprint, ExtensionType::StaticClass()));
	}

	/** Get an already-requested extension for an anim blueprint. @return nullptr if the extension is not found. */
	template<typename ExtensionType>
	static ExtensionType* FindExtension(UAnimBlueprint* InAnimBlueprint)
	{
		return Cast<ExtensionType>(GetExtension(InAnimBlueprint, ExtensionType::StaticClass()));
	}
	
	/** Get an already-requested an anim blueprint extension for an anim blueprint. */
	static UE_API UAnimBlueprintExtension* GetExtension(UAnimBlueprint* InAnimBlueprint, TSubclassOf<UAnimBlueprintExtension> InExtensionType);
	
	/** Iterate over all registered UAnimBlueprintExtensions in an anim BP */
	static UE_API void ForEachExtension(UAnimBlueprint* InAnimBlueprint, TFunctionRef<void(UAnimBlueprintExtension*)> InFunction);

	/** Get all subsystems currently present on an anim blueprint */
	static UE_API TArray<UAnimBlueprintExtension*> GetExtensions(UAnimBlueprint* InAnimBlueprint);

	/** Request all extensions that a node needs */
	static UE_API void RequestExtensionsForNode(UAnimGraphNode_Base* InAnimGraphNode);
	
	/** Refresh all extensions according to nodes present in an anim BP */
	static UE_API void RefreshExtensions(UAnimBlueprint* InAnimBlueprint);
	
	/** Get the structure that will be added to any BP-derived UAnimInstance */
	UE_API const UScriptStruct* GetInstanceDataType() const;

	/** Get the structure that will be added to any anim BP class */
	UE_API const UScriptStruct* GetClassDataType() const;

	/** Get the property of the instance data */
	UE_API const FStructProperty* GetInstanceDataProperty() const;

	/** Get the property of the class data */
	UE_API const FStructProperty* GetClassDataProperty() const;

	/** Get the defaults that will be held on the class */
	template<typename ClassDataType>
	ClassDataType& GetClassData()
	{
		return *static_cast<ClassDataType*>(GetClassDataInternal());
	}

	/** Get the defaults that will be held on the instance */
	template<typename InstanceDataType>
    InstanceDataType& GetInstanceData()
	{
		return *static_cast<InstanceDataType*>(GetInstanceDataInternal());
	}

protected:
	/** Get the anim blueprint that hosts this extension */
	UE_API UAnimBlueprint* GetAnimBlueprint() const;
	
private:
	friend class FAnimBlueprintCompilerContext;
	
	/** 
	 * Override point called when a compiler context is created for the anim blueprint
	 * @param	InCreationContext	The compiler context for the current compilation
	 */
	virtual void HandleBeginCompilation(IAnimBlueprintCompilerCreationContext& InCreationContext) {}

	/** 
	 * Override point called when the class starts compiling. The class may be new or recycled.
	 * @param	InClass			The class that is being compiled. This could be a newly created class or a recycled class
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */
	virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}

	/** 
	 * Override point called before all animation nodes are processed
	 * @param	InAnimNodes		The set of anim nodes that should be processed. Note that these nodes have not yet been pruned for connectivity
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */
	virtual void HandlePreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}
	
	/** 
	 * Override point called after all animation nodes are processed
	 * @param	InAnimNodes		The set of anim nodes that were processed. Note that these nodes were not pruned for connectivity (they will be the same set passed to PreProcessAnimationNodes)
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */	
	virtual void HandlePostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}

	/** 
	 * Override point called post graph expansion
	 * @param	InGraph		The graph that was just expanded
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */	
	virtual void HandlePostExpansionStep(const UEdGraph* InGraph, IAnimBlueprintPostExpansionStepContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}

	/** 
	 * Override point called when the class has finished compiling
	 * @param	InClass			The class that was compiled
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */
	virtual void HandleFinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}
	
	/** 
	 * Override point called when data is being copied to the CDO
	 * @param	InDefaultObject		The CDO for the just-compiled class
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	InPerExtensionContext	The extension context for the current compilation
	 */
	virtual void HandleCopyTermDefaultsToDefaultObject(UObject* InDefaultObject, IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext) {}

	/** 
	 * Override point called when data is being copied to the sparse class data
	 * @param	InDefaultObject		The CDO for the just-compiled class
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	InPerExtensionContext	The extension context for the current compilation
	 */
	virtual void HandleCopyTermDefaultsToSparseClassData(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext) {}	

	/** Override point called when a compiler context is destroyed for the anim blueprint. Can be used to clean up resources. */
	virtual void HandleEndCompilation() {}

private:
	/** 
	 * Called when a compiler context is created for the anim blueprint
	 * @param	InCreationContext	The compiler context for the current compilation
	 */	
	UE_API void BeginCompilation(IAnimBlueprintCompilerCreationContext& InCreationContext);

	/** 
	 * Called when the class starts compiling. The class may be new or recycled.
	 * @param	InClass			The class that is being compiled. This could be a newly created class or a recycled class
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */
	UE_API void StartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	/** 
	 * Called before all animation nodes are processed
	 * @param	InAnimNodes		The set of anim nodes that should be processed. Note that these nodes have not yet been pruned for connectivity
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */
	UE_API void PreProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	/** 
	 * Called after all animation nodes are processed
	 * @param	InAnimNodes		The set of anim nodes that were processed. Note that these nodes were not pruned for connectivity (they will be the same set passed to PreProcessAnimationNodes)
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */	
	UE_API void PostProcessAnimationNodes(TArrayView<UAnimGraphNode_Base*> InAnimNodes, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	/** 
	 * Called post graph expansion
	 * @param	InGraph		The graph that was just expanded
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */		
	UE_API void PostExpansionStep(const UEdGraph* InGraph, IAnimBlueprintPostExpansionStepContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	/** 
	 * Called when the class has finished compiling
	 * @param	InClass			The class that was compiled
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	OutCompiledData	The compiled data that this handler can write to
	 */
	UE_API void FinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);
	
	/** 
	 * Called when data is being copied to the CDO
	 * @param	InDefaultObject		The CDO for the just-compiled class
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	InPerExtensionContext	The extension context for the current compilation
	 */
	UE_API void CopyTermDefaultsToDefaultObject(UObject* InDefaultObject, IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext);

	/** 
	 * Called when data is being copied to the sparse class data
	 * @param	InDefaultObject		The CDO for the just-compiled class
	 * @param	InCompilerContext	The compiler context for the current compilation
	 * @param	InPerExtensionContext	The extension context for the current compilation
	 */
	UE_API void CopyTermDefaultsToSparseClassData(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintExtensionCopyTermDefaultsContext& InPerExtensionContext);

	/** Called when a compiler context is destroyed for the anim blueprint. Can be used to clean up resources. */
	UE_API void EndCompilation();
	
	/** Get the defaults that will be held on the class */
	UE_API void* GetClassDataInternal();

	/** Get the defaults that will be held on the instance */
	UE_API void* GetInstanceDataInternal();
};

#undef UE_API
