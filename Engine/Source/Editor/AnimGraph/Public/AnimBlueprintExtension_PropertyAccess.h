// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimBlueprintExtension.h"
#include "Animation/AnimSubsystem_PropertyAccess.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "IPropertyAccessCompiler.h"
#include "Internationalization/Text.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "AnimBlueprintExtension_PropertyAccess.generated.h"

#define UE_API ANIMGRAPH_API

class FString;
class IAnimBlueprintCompilationBracketContext;
class IAnimBlueprintCompilerCreationContext;
class IAnimBlueprintGeneratedClassCompiledData;
class UAnimBlueprintExtension_PropertyAccess;
class UClass;
class UObject;

enum class EPropertyAccessBatchType : uint8;
class FKismetCompilerContext;
class UEdGraph;
class UEdGraphPin;

// Delegate called when the library is compiled (whether successfully or not)
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPostLibraryCompiled, IAnimBlueprintCompilationBracketContext& /*InCompilationContext*/, IAnimBlueprintGeneratedClassCompiledData& /*OutCompiledData*/)

UCLASS(MinimalAPI)
class UAnimBlueprintExtension_PropertyAccess : public UAnimBlueprintExtension
{
	GENERATED_BODY()

	friend class UAnimBlueprintExtension_Base;

public:	
	// Try to determine the context in which this property access can safely be performed.
	static UE_API const FName ContextId_Automatic;

	// Can safely be executed on worker threads
	static UE_API const FName ContextId_UnBatched_ThreadSafe;

	// Executed batched on the game thread before the event graph is run
	static UE_API const FName ContextId_Batched_WorkerThreadPreEventGraph;

	// Executed batched on the game thread after the event graph is run
	static UE_API const FName ContextId_Batched_WorkerThreadPostEventGraph;

	// Executed batched on the game thread before the event graph is run
	static UE_API const FName ContextId_Batched_GameThreadPreEventGraph;

	// Executed batched on the game thread after the event graph is run
	static UE_API const FName ContextId_Batched_GameThreadPostEventGraph;
	
public:
	// Add a copy to the property access library we are compiling
	// @return an integer handle to the pending copy. This can be resolved to a true copy index by calling GetCompiledHandle
	UE_API FPropertyAccessHandle AddCopy(TArrayView<FString> InSourcePath, TArrayView<FString> InDestPath, const FName& InContextId, UObject* InObject = nullptr);

	// Add an access to the property access library we are compiling
	// @return an integer handle to the pending access. This can be resolved to a true access index by calling GetCompiledHandle
	UE_API FPropertyAccessHandle AddAccess(TArrayView<FString> InPath, UObject* InObject = nullptr);

	// Delegate called when the library is compiled (whether successfully or not)
	FSimpleMulticastDelegate& OnPreLibraryCompiled() { return OnPreLibraryCompiledDelegate; }

	// Delegate called when the library is compiled (whether successfully or not)
	FOnPostLibraryCompiled& OnPostLibraryCompiled() { return OnPostLibraryCompiledDelegate; }

	// Maps the initial copy handle to a true handle, post compilation
	UE_API FCompiledPropertyAccessHandle GetCompiledHandle(FPropertyAccessHandle InHandle) const;

	// Get the access type for the specified handle
	UE_API EPropertyAccessCopyType GetCompiledHandleAccessType(FPropertyAccessHandle InHandle) const;

	// Expands a property access path to a pure chain of BP nodes
	UE_API void ExpandPropertyAccess(FKismetCompilerContext& InCompilerContext, TArrayView<FString> InSourcePath, UEdGraph* InParentGraph, UEdGraphPin* InTargetPin) const;

	// Maps a compiled handle back to a human-readable context 
	static UE_API FText GetCompiledHandleContext(FCompiledPropertyAccessHandle InHandle);
	
	// Maps a compiled handle back to a human-readable context description
	static UE_API FText GetCompiledHandleContextDesc(FCompiledPropertyAccessHandle InHandle);

	// Whether a context name requires an auto-generated variable to be cached for an access
	static UE_API bool ContextRequiresCachedVariable(FName InName);
	
private:
	// UAnimBlueprintExtension interface
	UE_API virtual void HandleStartCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void HandleFinishCompilingClass(const UClass* InClass, IAnimBlueprintCompilationBracketContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

private:	
	// Property access library compiler
	TUniquePtr<IPropertyAccessLibraryCompiler> PropertyAccessLibraryCompiler;

	// Delegate called before the library is compiled
	FSimpleMulticastDelegate OnPreLibraryCompiledDelegate;

	// Delegate called when the library is compiled (whether successfully or not)
	FOnPostLibraryCompiled OnPostLibraryCompiledDelegate;

	UPROPERTY()
	FAnimSubsystem_PropertyAccess Subsystem;
};

#undef UE_API
