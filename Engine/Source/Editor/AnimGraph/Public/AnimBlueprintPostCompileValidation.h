// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Kismet2/CompilerResultsLog.h"

#include "AnimBlueprintPostCompileValidation.generated.h"

#define UE_API ANIMGRAPH_API

// Encapsulated Parameters, to upgrade these without changing function signature.
struct FAnimBPCompileValidationParams
{
	const class UAnimInstance* const DefaultAnimInstance;
	const class UAnimBlueprintGeneratedClass* const NewAnimBlueprintClass;
	FCompilerResultsLog& MessageLog;
	const TMap<FProperty*, class UAnimGraphNode_Base*>& AllocatedNodePropertiesToNodes;

	FAnimBPCompileValidationParams
	(
		const class UAnimInstance* const InDefaultAnimInstance,
		const class UAnimBlueprintGeneratedClass* const InNewAnimBlueprintClass,
		FCompilerResultsLog& InMessageLog,
		const TMap<FProperty*, class UAnimGraphNode_Base*>& InAllocatedNodePropertiesToNodes
	)
		: DefaultAnimInstance(InDefaultAnimInstance)
		, NewAnimBlueprintClass(InNewAnimBlueprintClass)
		, MessageLog(InMessageLog)
		, AllocatedNodePropertiesToNodes(InAllocatedNodePropertiesToNodes)
	{}
};

// This class is a base class for performing AnimBlueprint Post Compilation Validation.
UCLASS(MinimalAPI)
class UAnimBlueprintPostCompileValidation : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual void DoPostCompileValidation(FAnimBPCompileValidationParams& InParams) const;

protected:
	static UE_API void PCV_PreloadObject(const UObject* const ReferencedObject);

	static UE_API void PCV_GatherAnimSequences(TArray<const UAnimSequence*>& OutAnimSequences, const UAnimSequenceBase* const InAnimSequenceBase);
	static UE_API void PCV_GatherAnimSequences(TArray<const UAnimSequence*>& OutAnimSequences, const class UBlendSpace* const InBlendSpace);

	struct FPCV_GatherParams
	{
		bool bFilterBySyncGroup;
		FName SyncGroupName;
		bool bFilterByLoopingCondition;
		bool bLoopingCondition;

		UE_DEPRECATED(4.26, "Please use the constructor that takes an FName for the sync group")
		FPCV_GatherParams
		(
			bool InbFilterBySyncGroup = false,
			int32 InSyncGroupIndex = INDEX_NONE,
			bool InbFilterByLoopingCondition = false,
			bool InbLoopingCondition = false
		)
			: bFilterBySyncGroup(InbFilterBySyncGroup)
			, SyncGroupName(NAME_None)
			, bFilterByLoopingCondition(InbFilterByLoopingCondition)
			, bLoopingCondition(InbLoopingCondition)
		{}

		FPCV_GatherParams
		(
			bool InbFilterBySyncGroup = false,
			FName InSyncGroupName = NAME_None,
			bool InbFilterByLoopingCondition = false,
			bool InbLoopingCondition = false
		)
			: bFilterBySyncGroup(InbFilterBySyncGroup)
			, SyncGroupName(InSyncGroupName)
			, bFilterByLoopingCondition(InbFilterByLoopingCondition)
			, bLoopingCondition(InbLoopingCondition)
		{}
	};

	static UE_API void PCV_GatherAnimSequencesFromGraph(TArray<const UAnimSequence*>& OutAnimSequences, FAnimBPCompileValidationParams& PCV_Params, const FPCV_GatherParams& GatherParams);
	static UE_API void PCV_GatherBlendSpacesFromGraph(TArray<const class UBlendSpace*>& OutBlendSpaces, FAnimBPCompileValidationParams& PCV_Params);

	struct FPCV_ReferencedAnimSequence
	{
		const UAnimSequence* AnimSequence;
		const UObject* Referencer;

		FPCV_ReferencedAnimSequence
		(
			const UAnimSequence* InAnimSequence,
			const UObject* InReferencer
		)
			: AnimSequence(InAnimSequence)
			, Referencer(InReferencer)
		{}
	};

	struct FPCV_PropertyAndValue
	{
		const FProperty* Property;
		const void* Value;

		FPCV_PropertyAndValue
		(
			const FProperty* InProperty,
			const void* InValue
		)
			: Property(InProperty)
			, Value(InValue)
		{}
	};

	static UE_API void PCV_GatherAllReferencedAnimSequences(TArray<FPCV_ReferencedAnimSequence>& OutRefAnimSequences, FAnimBPCompileValidationParams& PCV_Params);
	static UE_API void PCV_GatherAnimSequencesFromStruct(TArray<FPCV_ReferencedAnimSequence>& OutRefAnimSequences, FAnimBPCompileValidationParams& PCV_Params, const UStruct* InStruct, const void* InData, TArray<FPCV_PropertyAndValue> InPropertyCallChain);
	static UE_API void PCV_GatherAnimSequencesFromProperty(TArray<FPCV_ReferencedAnimSequence>& OutRefAnimSequences, FAnimBPCompileValidationParams& PCV_Params, const FProperty* InProperty, const void* InData, TArray<FPCV_PropertyAndValue> InPropertyCallChain);

private:
	virtual bool NeedsLoadForClient() const override { return false; }
	virtual bool NeedsLoadForServer() const override { return false; }
};

#undef UE_API
