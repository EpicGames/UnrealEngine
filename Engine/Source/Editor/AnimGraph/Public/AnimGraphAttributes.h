// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "AnimGraphAttributes.generated.h"

#define UE_API ANIMGRAPH_API

struct FPropertyChangedEvent;
template <typename FuncType> class TFunctionRef;

UENUM()
enum class EAnimGraphAttributesDisplayMode
{
	// Always hide on pins
	HideOnPins,

	// Always show on pins
	ShowOnPins,

	// Automatically calculate visibility based on graph connectivity/usage
	Automatic
};

UENUM()
enum class EAnimGraphAttributeBlend
{
	// Attribute is subject to weighted blends as it flows through the graph
	Blendable,

	// Attribute is not blended. Corresponds to messaging between two nodes that are not necessarily directly connected.
	NonBlendable,
};

// Description of an attribute
USTRUCT()
struct FAnimGraphAttributeDesc
{
	GENERATED_BODY()

	FAnimGraphAttributeDesc() = default;

	FAnimGraphAttributeDesc(FName InName, EAnimGraphAttributeBlend InBlend, FSlateBrush InIcon = FSlateBrush(), FText InDisplayName = FText(), FText InToolTipText = FText(), FSlateColor InColor = FSlateColor::UseSubduedForeground(), EAnimGraphAttributesDisplayMode InDisplayMode = EAnimGraphAttributesDisplayMode::Automatic, int32 InSortOrder = INDEX_NONE)
		: Name(InName)
		, Icon(InIcon)
		, DisplayName(InDisplayName.IsEmpty() ? FText::FromName(InName) : InDisplayName)
		, ToolTipText(InToolTipText.IsEmpty() ? FText::FromName(InName) : InToolTipText)
		, Color(InColor)
		, DisplayMode(InDisplayMode)
		, Blend(InBlend)
		, SortOrder(InSortOrder)
	{
	}

	UPROPERTY(config)
	FName Name;

	UPROPERTY(config)
	FSlateBrush Icon;

	UPROPERTY(config)
	FText DisplayName;

	UPROPERTY(config)
	FText ToolTipText;

	UPROPERTY(config)
	FSlateColor Color;

	// How the attribute is displayed in the graph. This overrides any settings per node type.
	UPROPERTY(config)
	EAnimGraphAttributesDisplayMode DisplayMode = EAnimGraphAttributesDisplayMode::Automatic;

	// Blendability of the attribute
	UPROPERTY(config)
	EAnimGraphAttributeBlend Blend = EAnimGraphAttributeBlend::Blendable;

	// How to sort the attribute
	UPROPERTY(config)
	int32 SortOrder = INDEX_NONE;
};

UCLASS(MinimalAPI, Config = EditorPerProjectUserSettings)
class UAnimGraphAttributes : public UObject
{
	GENERATED_BODY()

public:
	// Register an anim graph attribute
	UE_API void Register(const FAnimGraphAttributeDesc& InDesc);

	// Get a previously registered attribute desc
	// @param	InName	The name of a previously registered attribute
	// @return nullptr if no desc is registered with the specified name
	UE_API const FAnimGraphAttributeDesc* FindAttributeDesc(FName InName) const;

	// Iterate over all attributes that are registered
	// @param	InFunction	FUnction receiving the various attribute descriptors
	UE_API void ForEachAttribute(TFunctionRef<void(const FAnimGraphAttributeDesc&)> InFunction) const;

private:
	// UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	// All attributes
	UPROPERTY(config)
	TArray<FAnimGraphAttributeDesc> Attributes;

	// Cached attribute index map for faster lookup
	TMap<FName, int32> AttributesMap;
};

#undef UE_API
