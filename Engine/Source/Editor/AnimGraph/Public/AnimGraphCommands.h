// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

UE_DECLARE_TCOMMANDS(class FAnimGraphCommands, ANIMGRAPH_API)

/** Anim Graph Commands */
class FAnimGraphCommands : public TCommands<FAnimGraphCommands>
{
public:
	FAnimGraphCommands()
		: TCommands<FAnimGraphCommands>(TEXT("AnimGraph"), NSLOCTEXT("Contexts", "AnimGraphCommands", "Anim Graph Commands"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	ANIMGRAPH_API virtual void RegisterCommands() override;

public:

	// Toggle pose watching for a given node
	TSharedPtr<FUICommandInfo> TogglePoseWatch;

	// Hide unbound property pins
	TSharedPtr<FUICommandInfo> HideUnboundPropertyPins;

	// SkeletalControl specific commands
	TSharedPtr< FUICommandInfo > SelectBone;
	// Blend list options
	TSharedPtr< FUICommandInfo > AddBlendListPin;
	TSharedPtr< FUICommandInfo > RemoveBlendListPin;

	// options for sequence/evaluator converter
	TSharedPtr< FUICommandInfo > ConvertToSeqEvaluator;
	TSharedPtr< FUICommandInfo > ConvertToSeqPlayer;

	// options for blendspace converter
	TSharedPtr< FUICommandInfo > ConvertToBSEvaluator;
	TSharedPtr< FUICommandInfo > ConvertToBSPlayer;
	TSharedPtr< FUICommandInfo > ConvertToBSGraph;

	// options for aimoffset converter
	TSharedPtr< FUICommandInfo > ConvertToAimOffsetLookAt;
	TSharedPtr< FUICommandInfo > ConvertToAimOffsetSimple;
	TSharedPtr< FUICommandInfo > ConvertToAimOffsetGraph;

	// options for sequence/evaluator converter
	TSharedPtr< FUICommandInfo > ConvertToPoseBlender;
	TSharedPtr< FUICommandInfo > ConvertToPoseByName;

	// option for opening the asset related to the graph node
	TSharedPtr< FUICommandInfo > OpenRelatedAsset;

	// Create an alias for this state adds a transition to it
	TSharedPtr< FUICommandInfo > CreateSelfTransition;

	// Set the current state as active
	TSharedPtr< FUICommandInfo > SetActiveState;
	
	// Reverse the selected transition(s)
	TSharedPtr< FUICommandInfo > ReverseTransition;
};
