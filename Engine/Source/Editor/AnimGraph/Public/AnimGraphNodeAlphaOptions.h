// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/InputScaleBias.h"
#include "AnimGraphNode_Base.h"
#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"

// Helper struct used to handle graph (editor) node alpha options
struct FAnimGraphNodeAlphaOptions
{
	// This assumes that a anim node contains the following members:
	//
	// 		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	// 		float Alpha;
	// 
	// 		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	// 		FInputScaleBias AlphaScaleBias;
	// 
	// 		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (DisplayName = "Blend Settings"))
	// 		FInputAlphaBoolBlend AlphaBoolBlend;
	// 
	// 		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	// 		FName AlphaCurveName;
	// 
	// 		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	// 		FInputScaleBiasClamp AlphaScaleBiasClamp;
	// 
	// 		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	// 		EAnimAlphaInputType AlphaInputType;
	// 
	// 		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault, DisplayName = "bEnabled", DisplayAfter = "AlphaScaleBias"))
	// 		bool bAlphaBoolEnabled;
	//
	// This is implemented in this way (rather than encapsulating the options in their own struct) because
	// we dont support exposing pins from sub-structs on anim nodes (or anything else for that matter).

	// Called from an UAnimGraphNode_Base-derived classes CustomizePinData to update alpha pins
	template<typename AnimNodeType>
	static void HandleCustomizePinData(const AnimNodeType& InAnimNode, UEdGraphPin* InPin)
	{
		if (InPin->PinName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, Alpha))
		{
			InPin->bHidden = (InAnimNode.AlphaInputType != EAnimAlphaInputType::Float);

			if (!InPin->bHidden)
			{
				InPin->PinFriendlyName = InAnimNode.AlphaScaleBias.GetFriendlyName(InAnimNode.AlphaScaleBiasClamp.GetFriendlyName(InPin->PinFriendlyName));
			}
		}

		if (InPin->PinName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, bAlphaBoolEnabled))
		{
			InPin->bHidden = (InAnimNode.AlphaInputType != EAnimAlphaInputType::Bool);
		}

		if (InPin->PinName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, AlphaCurveName))
		{
			InPin->bHidden = (InAnimNode.AlphaInputType != EAnimAlphaInputType::Curve);

			if (!InPin->bHidden)
			{
				InPin->PinFriendlyName = InAnimNode.AlphaScaleBiasClamp.GetFriendlyName(InPin->PinFriendlyName);
			}
		}
	}

	// Called from an UAnimGraphNode_Base-derived classes PostEditChangeProperty to update alpha properties and pins
	template<typename AnimNodeType>
	static void HandlePostEditChangeProperty(const AnimNodeType& InAnimNode, UAnimGraphNode_Base* InNode, FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

		// Reconstruct node to show updates to PinFriendlyNames.
		if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, AlphaScaleBias))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
			|| (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
		{
			InNode->ReconstructNode();
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, AlphaInputType))
		{
			FScopedTransaction Transaction(NSLOCTEXT("AnimGraphNodeAlphaOptions", "ChangeAlphaInputType", "Change Alpha Input Type"));
			InNode->Modify();

			// Break links to pins going away
			for (int32 PinIndex = 0; PinIndex < InNode->Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = InNode->Pins[PinIndex];
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, Alpha))
				{
					if (InAnimNode.AlphaInputType != EAnimAlphaInputType::Float)
					{
						Pin->BreakAllPinLinks();
						InNode->RemoveBindings(Pin->PinName);
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, bAlphaBoolEnabled))
				{
					if (InAnimNode.AlphaInputType != EAnimAlphaInputType::Bool)
					{
						Pin->BreakAllPinLinks();
						InNode->RemoveBindings(Pin->PinName);
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(AnimNodeType, AlphaCurveName))
				{
					if (InAnimNode.AlphaInputType != EAnimAlphaInputType::Curve)
					{
						Pin->BreakAllPinLinks();
						InNode->RemoveBindings(Pin->PinName);
					}
				}
			}

			InNode->ReconstructNode();

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InNode->GetBlueprint());
		}
	}

	// Called from an UAnimGraphNode_Base-derived classes CustomizeDetails to update alpha properties and pins
	template<typename AnimNodeType>
	static void HandleCustomizeDetails(const AnimNodeType& InAnimNode, TSharedRef<IPropertyHandle> InNodeHandle, IDetailLayoutBuilder& InDetailBuilder)
	{
		if (InAnimNode.AlphaInputType != EAnimAlphaInputType::Bool)
		{
			InDetailBuilder.HideProperty(InNodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(AnimNodeType, bAlphaBoolEnabled)));
			InDetailBuilder.HideProperty(InNodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(AnimNodeType, AlphaBoolBlend)));
		}

		if (InAnimNode.AlphaInputType != EAnimAlphaInputType::Float)
		{
			InDetailBuilder.HideProperty(InNodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(AnimNodeType, Alpha)));
			InDetailBuilder.HideProperty(InNodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(AnimNodeType, AlphaScaleBias)));
		}

		if (InAnimNode.AlphaInputType != EAnimAlphaInputType::Curve)
		{
			InDetailBuilder.HideProperty(InNodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(AnimNodeType, AlphaCurveName)));
		}

		if ((InAnimNode.AlphaInputType != EAnimAlphaInputType::Float)
			&& (InAnimNode.AlphaInputType != EAnimAlphaInputType::Curve))
		{
			InDetailBuilder.HideProperty(InNodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(AnimNodeType, AlphaScaleBiasClamp)));
		}
	}
};
