// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "AnimGraphNodeCustomizationInterface.generated.h"

UINTERFACE(MinimalAPI, Blueprintable)
class UAnimGraphNodeCustomizationInterface : public UInterface
{
	GENERATED_BODY()
};

class IAnimGraphNodeCustomizationInterface
{
	GENERATED_BODY()

public:
	/**
	 * Get the custom color for this node
	 */
	UFUNCTION(BlueprintNativeEvent, CallInEditor, Category = "Customization")
	FLinearColor GetTitleColor() const;
	virtual FLinearColor GetTitleColor_Implementation() const
	{
		return FLinearColor(0.2f, 0.2f, 0.8f);
	}
};