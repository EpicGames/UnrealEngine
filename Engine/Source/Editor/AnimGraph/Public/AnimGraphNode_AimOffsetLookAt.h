// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BlueprintNodeSignature.h"
#include "AnimGraphNode_BlendSpaceBase.h"
#include "AnimNodes/AnimNode_AimOffsetLookAt.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_AimOffsetLookAt.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI, hidecategories = Coordinates)
class UAnimGraphNode_AimOffsetLookAt : public UAnimGraphNode_BlendSpaceBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_AimOffsetLookAt Node;

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual UAnimationAsset* GetAnimationAsset() const override { return Node.GetBlendSpace(); }
	virtual void GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap) override;
	ANIMGRAPH_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	// End of UAnimGraphNode_Base interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	// End of UK2Node interface

	// UAnimGraphNode_AssetPlayerBase interface
	virtual void SetAnimationAsset(UAnimationAsset* Asset) override;
	// End of UAnimGraphNode_AssetPlayerBase interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
