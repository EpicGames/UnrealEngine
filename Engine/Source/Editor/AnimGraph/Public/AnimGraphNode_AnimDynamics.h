// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Input/Reply.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Animation/AnimPhysicsSolver.h"
#include "BoneControllers/AnimNode_AnimDynamics.h"
#include "AnimGraphNode_SkeletalControlBase.h"

#include "AnimGraphNode_AnimDynamics.generated.h"

class FCompilerResultsLog;
class FPrimitiveDrawInterface;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class USkeletalMeshComponent;

namespace AnimDynamicsNodeConstants
{
	const FLinearColor ShapeDrawColor = FLinearColor::White;
	const FLinearColor ActiveBodyDrawColor = FLinearColor::Yellow;
	const FLinearColor LimitLineDrawColor = FLinearColor::Yellow;
	const float LimitPlaneDrawSize = 50.0f;
	const float ShapeLineWidth = 0.2f;
	const float SelectedShapeLineWidth = 0.4f;
	const float BodyLineWidth = 0.2f;
	const float TransformLineWidth = 0.05f;
	const float TransformBasisScale = 10.0f;
}

UCLASS(MinimalAPI)
class UAnimGraphNode_AnimDynamics : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_AnimDynamics Node;

	/** Preview the live physics object on the mesh */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bPreviewLive;

	/** Show linear (prismatic) limits in the viewport */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bShowLinearLimits;

	/** Show angular limit ranges in the viewport */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bShowAngularLimits;

	/** Show planar limit info (actual plane, plane normal) in the viewport */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bShowPlanarLimit;

	/** Show spherical limits in the viewport (preview live only) */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bShowSphericalLimit;

	/** If planar limits are enabled and the collision mode isn't CoM, draw sphere collision sizes */
	UPROPERTY(EditAnywhere, Category = Preview)
	bool bShowCollisionSpheres;

	virtual void PostLoad() override;

	static FReply ResetButtonClicked(IDetailLayoutBuilder* DetailLayoutBuilder);
	void ResetSim();

	// UObject
	virtual void Serialize(FArchive& Ar) override;

	// UEdGraphNode_Base
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetNodeCategory() const override;

	// UAnimGraphNode_Base
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual FEditorModeID GetEditorMode() const override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
	// UAnimGraphNode_SkeletalControlBase
	virtual void GetOnScreenDebugInfo(TArray<FText>& DebugInfo, FAnimNode_Base* RuntimeAnimNode, USkeletalMeshComponent* PreviewSkelMeshComp) const override;

	FAnimNode_AnimDynamics* GetPreviewDynamicsNode() const;
	bool IsPreviewLiveActive() const { return bPreviewLive;  }

	// UI callbacks for customising the physics bodies array in the details panel.
	void OnPhysicsBodyDefCustomizeDetails(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout);
	FText BodyDefinitionUINameString(const uint32 BodyIndex) const;

	void UpdatePreviewDynamicsNode(const int32 PhysicsBodyIndex);

protected:

	virtual FText GetControllerDescription() const override;

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	void DrawLinearLimits(FPrimitiveDrawInterface* PDI, FTransform ShapeTransform, const FAnimNode_AnimDynamics& NodeToVisualise) const;
	void DrawAngularLimits(FPrimitiveDrawInterface* PDI, FTransform JointTransform, const FAnimNode_AnimDynamics& NodeToVisualize) const;

	// UAnimGraphNode_SkeletalControlBase protected interface
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase protected interface

	USkeleton* GetSkeleton() const;

private:

	FNodeTitleTextTable CachedNodeTitles;
};
