// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_ApplyAdditive.h"
#include "AnimGraphNode_ApplyAdditive.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_ApplyAdditive : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_ApplyAdditive Node;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual FString GetNodeCategory() const override;
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	// End of UAnimGraphNode_Base interface
};
