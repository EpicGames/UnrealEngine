// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_ApplyLimits.h"
#include "AnimGraphNode_ApplyLimits.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_ApplyLimits : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_ApplyLimits Node;

public:
	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

protected:
	
	void DrawAngularLimits(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp, const FAnimNode_ApplyLimits& NodeToVisualize) const;

	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent * PreviewSkelMeshComp) const override;
	// End of UAnimGraphNode_SkeletalControlBase interface
};
