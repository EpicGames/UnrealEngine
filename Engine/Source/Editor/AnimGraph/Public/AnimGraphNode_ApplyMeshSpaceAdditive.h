// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_ApplyMeshSpaceAdditive.h"
#include "AnimGraphNode_ApplyMeshSpaceAdditive.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_ApplyMeshSpaceAdditive : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_ApplyMeshSpaceAdditive Node;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	// UEdGraphNode interface
	FText GetTooltipText() const override;
	FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	FLinearColor GetNodeTitleColor() const override;
	void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	FString GetNodeCategory() const override;
	void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	// End of UAnimGraphNode_Base interface
};
