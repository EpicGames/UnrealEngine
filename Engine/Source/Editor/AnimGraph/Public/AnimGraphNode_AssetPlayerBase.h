// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimationAsset.h"
#include "AnimGraphNode_Base.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_AssetPlayerBase.generated.h"

#define UE_API ANIMGRAPH_API

/** Get the default anim node class for playing a particular asset */
ANIMGRAPH_API UClass* GetNodeClassForAsset(const UClass* AssetClass);

/** See if a particular anim NodeClass can play a particular anim AssetClass */
ANIMGRAPH_API bool SupportNodeClassForAsset(const UClass* AssetClass, UClass* NodeClass);

/** Helper / intermediate for asset player graphical nodes */
UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_AssetPlayerBase : public UAnimGraphNode_Base
{
	GENERATED_BODY()
public:
	// Deprecated - sync group data is held on the contained FAnimNode_Base
	UPROPERTY()
	FAnimationGroupReference SyncGroup_DEPRECATED;

	/** UObject interface */
	UE_API void Serialize(FArchive& Ar) override;
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** UEdGraphNode interface */
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API virtual FText GetTooltipText() const override;

	/** UK2Node interface */
	UE_API virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;

	/** UAnimGraphNode_Base interface */
	UE_API virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual void SetAnimationAsset(UAnimationAsset* Asset) { check(false); /*Base function called*/ }

	// Override this to copy any relevant settings from the animation asset used when creating a node with a drag/drop operation
	virtual void CopySettingsFromAnimationAsset(UAnimationAsset* Asset) {}

	// Helper function to gather menu actions from specific asset types supported by this node
	static UE_API void GetMenuActionsHelper(
		FBlueprintActionDatabaseRegistrar& InActionRegistrar,
		TSubclassOf<UAnimGraphNode_Base> InNodeClass,
		const TArray<TSubclassOf<UObject>>& InAssetTypes,
		const TArray<TSubclassOf<UObject>>& InExcludedAssetTypes,
		const TFunctionRef<FText(const FAssetData&, UClass*)>& InMenuNameFunction,
		const TFunctionRef<FText(const FAssetData&, UClass*)>& InMenuTooltipFunction,
		const TFunction<void(UEdGraphNode*, bool, const FAssetData)>& InSetupNewNodeFromAssetFunction,
		const TFunction<void(UEdGraphNode*, bool, TSubclassOf<UObject>)>& InSetupNewNodeFromClassFunction = {},
		const TFunction<FText(const FAssetData&)>& InMenuCategoryFunction = nullptr);

	// Helper function to validate player nodes
	UE_API void ValidateAnimNodeDuringCompilationHelper(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog, UAnimationAsset* InAsset, TSubclassOf<UAnimationAsset> InAssetType, UEdGraphPin* InExposedPin, FName InPropertyName);

	// Helper function to preload required assets (including those referenced on pins)
	UE_API void PreloadRequiredAssetsHelper(UAnimationAsset* InAsset, UEdGraphPin* InExposedPin);
	
protected:
	// Helper functions to build a title for an asset player node
	UE_API FText GetNodeTitleHelper(ENodeTitleType::Type InTitleType, UEdGraphPin* InAssetPin, const FText& InAssetDesc, const TFunction<FText(UAnimationAsset*)> InPostFixFunctionRef = nullptr) const;
	UE_API FText GetNodeTitleForAsset(ENodeTitleType::Type InTitleType, UAnimationAsset* InAsset, const FText& InAssetDesc, const TFunction<FText(UAnimationAsset*)> InPostFixFunctionRef = nullptr) const;
	UE_API void AddSyncGroupToNodeTitle(ENodeTitleType::Type InTitleType, FText & InOutTitle) const;
	
	// Default setup function that can be used with GetMenuActionsHalper
	static UE_API void SetupNewNode(UEdGraphNode* InNewNode, bool bInIsTemplateNode, const FAssetData InAssetData);

	/** Used for filtering in the Blueprint context menu when the sequence asset this node uses is unloaded */
	FString UnloadedSkeletonName;
};

#undef UE_API
