// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimNodeBase.h"
#include "Editor.h"
#include "IPropertyAccessEditor.h"
#include "K2Node.h"
#include "Engine/MemberReference.h"

#include "AnimGraphNode_Base.generated.h"

#define UE_API ANIMGRAPH_API

class FAnimGraphNodeDetails;
class FBlueprintActionDatabaseRegistrar;
class FCanvas;
class FCompilerResultsLog;
class FPrimitiveDrawInterface;
class IDetailLayoutBuilder;
class UAnimGraphNode_Base;
class UEdGraphSchema;
class USkeletalMeshComponent;
class IAnimBlueprintGeneratedClassCompiledData;
class IAnimBlueprintCompilationContext;
class IAnimBlueprintCopyTermDefaultsContext;
class IAnimBlueprintNodeCopyTermDefaultsContext;
class IAnimBlueprintNodeOverrideAssetsContext;
class UAnimBlueprintExtension;
class UAnimGraphNodeBinding;

struct FPoseLinkMappingRecord
{
public:
	static FPoseLinkMappingRecord MakeFromArrayEntry(UAnimGraphNode_Base* LinkingNode, UAnimGraphNode_Base* LinkedNode, FArrayProperty* ArrayProperty, int32 ArrayIndex)
	{
		checkSlow(CastFieldChecked<FStructProperty>(ArrayProperty->Inner)->Struct->IsChildOf(FPoseLinkBase::StaticStruct()));

		FPoseLinkMappingRecord Result;
		Result.LinkingNode = LinkingNode;
		Result.LinkedNode = LinkedNode;
		Result.ChildProperty = ArrayProperty;
		Result.ChildPropertyIndex = ArrayIndex;

		return Result;
	}

	static FPoseLinkMappingRecord MakeFromMember(UAnimGraphNode_Base* LinkingNode, UAnimGraphNode_Base* LinkedNode, FStructProperty* MemberProperty)
	{
		checkSlow(MemberProperty->Struct->IsChildOf(FPoseLinkBase::StaticStruct()));

		FPoseLinkMappingRecord Result;
		Result.LinkingNode = LinkingNode;
		Result.LinkedNode = LinkedNode;
		Result.ChildProperty = MemberProperty;
		Result.ChildPropertyIndex = INDEX_NONE;

		return Result;
	}

	static FPoseLinkMappingRecord MakeInvalid()
	{
		FPoseLinkMappingRecord Result;
		return Result;
	}

	bool IsValid() const
	{
		return LinkedNode != nullptr;
	}

	UAnimGraphNode_Base* GetLinkedNode() const
	{
		return LinkedNode;
	}

	UAnimGraphNode_Base* GetLinkingNode() const
	{
		return LinkingNode;
	}

	ANIMGRAPH_API void PatchLinkIndex(uint8* DestinationPtr, int32 LinkID, int32 SourceLinkID) const;
protected:
	FPoseLinkMappingRecord()
		: LinkedNode(nullptr)
		, LinkingNode(nullptr)
		, ChildProperty(nullptr)
		, ChildPropertyIndex(INDEX_NONE)
	{
	}

protected:
	// Linked node for this pose link, can be nullptr
	UAnimGraphNode_Base* LinkedNode;

	// Linking node for this pose link, can be nullptr
	UAnimGraphNode_Base* LinkingNode;

	// Will either be an array property containing FPoseLinkBase derived structs, indexed by ChildPropertyIndex, or a FPoseLinkBase derived struct property 
	FProperty* ChildProperty;

	// Index when ChildProperty is an array
	int32 ChildPropertyIndex;
};

UENUM()
enum class EBlueprintUsage : uint8
{
	NoProperties,
	DoesNotUseBlueprint,
	UsesBlueprint
};

/** Enum that indicates level of support of this node for a particular asset class */
enum class EAnimAssetHandlerType : uint8
{
	PrimaryHandler,
	Supported,
	NotSupported
};

/** The type of a property binding */
UENUM()
enum class EAnimGraphNodePropertyBindingType
{
	None,
	Property,
	Function,
};

USTRUCT()
struct FAnimGraphNodePropertyBinding
{
	GENERATED_BODY()

	FAnimGraphNodePropertyBinding() = default;

	/** Pin type */
	UPROPERTY()
	FEdGraphPinType PinType;

	/** Source type if the binding is a promotion */
	UPROPERTY()
	FEdGraphPinType PromotedPinType;

	/** Property binding name */
	UPROPERTY()
	FName PropertyName = NAME_None;

	UPROPERTY()
	int32 ArrayIndex = INDEX_NONE;
	
	/** The property path as text */
	UPROPERTY()
	FText PathAsText;

	/** The property path a pin is bound to */
	UPROPERTY()
	TArray<FString> PropertyPath;

	/** The context of the binding */
	UPROPERTY()
	FName ContextId = NAME_None;

	UPROPERTY()
	FText CompiledContext;

	UPROPERTY()
	FText CompiledContextDesc;
	
	/** Whether the binding is a function or not */
	UPROPERTY()
	EAnimGraphNodePropertyBindingType Type = EAnimGraphNodePropertyBindingType::Property;

	/** Whether the pin is bound or not */
	UPROPERTY()
	bool bIsBound = false;

	/** Whether the pin binding is a promotion (e.g. bool->int) */
	UPROPERTY()
	bool bIsPromotion = false;

	UPROPERTY()
	bool bOnlyUpdateWhenActive = false;
};

/**
  * This is the base class for any animation graph nodes that generate or consume an animation pose in
  * the animation blend graph.
  *
  * Any concrete implementations will be paired with a runtime graph node derived from FAnimNode_Base
  */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UAnimGraphNode_Base : public UK2Node
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=PinOptions, EditFixedSize)
	TArray<FOptionalPinFromProperty> ShowPinForProperties;

	/** Map from property name->binding info */
 	UPROPERTY()
 	TMap<FName, FAnimGraphNodePropertyBinding> PropertyBindings_DEPRECATED;

	/** Properties marked as always dynamic, so they can be set externally */
	UPROPERTY()
	TSet<FName> AlwaysDynamicProperties;

	UPROPERTY(Transient)
	EBlueprintUsage BlueprintUsage;

	// Function called before the node is updated for the first time
	UPROPERTY(EditAnywhere, Category = Functions, meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnInitialUpdate"), DisplayName="On Initial Update")
	FMemberReference InitialUpdateFunction;

	// Function called when the node becomes relevant, meaning it goes from having no weight to any weight.
	UPROPERTY(EditAnywhere, Category = Functions, meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnBecomeRelevant"), DisplayName="On Become Relevant")
	FMemberReference BecomeRelevantFunction;

	// Function called when the node is updated
	UPROPERTY(EditAnywhere, Category = Functions, meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnUpdate"), DisplayName="On Update")
	FMemberReference UpdateFunction;

private:
	// Bindings for pins that this node exposes
	UPROPERTY(EditAnywhere, Instanced, Category=Bindings)
	TObjectPtr<UAnimGraphNodeBinding> Binding;

	// Optional reference tag name. If this is set then this node can be referenced from elsewhere in this animation blueprint using an anim node reference
	UPROPERTY(EditAnywhere, Category = Tag)
	FName Tag;

public:
	// UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FString GetDocumentationLink() const override;
	UE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	virtual bool ShowPaletteIconOnNode() const override{ return false; }
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	UE_API virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	UE_API virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual void DestroyNode() override;
	// End of UEdGraphNode interface

	// UK2Node interface
	UE_API virtual void ReconstructNode() override;
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual bool CanPlaceBreakpoints() const override { return false; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;
	UE_API virtual void GetNodeAttributes(TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes) const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;

	// By default return any animation assets we have
	virtual UObject* GetJumpTargetForDoubleClick() const override { return GetAnimationAsset(); }
	UE_API virtual bool CanJumpToDefinition() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) override;
	UE_API virtual void HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName) override;
	UE_API virtual void ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement) override;
	UE_API virtual bool ReferencesVariable(const FName& InVarName, const UStruct* InScope) const override;
	UE_API virtual bool ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const override;
	// End of UK2Node interface

	// UAnimGraphNode_Base interface

	// Whether or not you can add a pose watch on this node
	virtual bool IsPoseWatchable() const { return true; }

	// Gets the menu category this node belongs in
	UE_API virtual FString GetNodeCategory() const;

	// Is this node a sink that has no pose outputs?
	virtual bool IsSinkNode() const { return false; }

	// Create any output pins necessary for this node
	UE_API virtual void CreateOutputPins();

	// customize pin data based on the input
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const {}

	// Gives each visual node a chance to do final validation before it's node is harvested for use at runtime
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog);

	// Gives each visual node a chance to validate that they are still valid in the context of the compiled class, giving a last shot at error or warning generation after primary compilation is finished
	virtual void ValidateAnimNodePostCompile(FCompilerResultsLog& MessageLog, UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex) {}

	// If using CopyPoseFromMesh, the AnimBlueprint Compiler will cache this off for optimizations. 
	virtual bool UsingCopyPoseFromMesh() const { return false; }

	// Gives each visual node a chance to update the node template before it is inserted in the compiled class
	virtual void BakeDataDuringCompilation(FCompilerResultsLog& MessageLog) {}

	// Give the node a chance to change the display name of a pin
	UE_API virtual void PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const;

	/** Get the animation blueprint to which this node belongs */
	UAnimBlueprint* GetAnimBlueprint() const { return CastChecked<UAnimBlueprint>(GetBlueprint()); }

	// Populate the supplied arrays with the currently reffered to animation assets 
	virtual void GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimAssets) const {}

	// Replace references to animations that exist in the supplied maps 	
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap) {};

	// Helper function for GetAllAnimationSequencesReferred
	UE_API void HandleAnimReferenceCollection(UAnimationAsset* AnimAsset, TArray<UAnimationAsset*>& AnimationAssets) const;

	// Helper function for ReplaceReferredAnimations	
	template<class AssetType>
	void HandleAnimReferenceReplacement(AssetType*& OriginalAsset, const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap);
	template<class AssetType>
	void HandleAnimReferenceReplacement(TObjectPtr<AssetType>& OriginalAsset, const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap);

	/**
	 * Selection notification callback.
	 * If a node needs to handle viewport input etc. then it should push an editor mode here.
	 * @param	bInIsSelected	Whether we selected or deselected the node
	 * @param	InModeTools		The mode tools. Use this to push the editor mode if required.
	 * @param	InRuntimeNode	The runtime node to go with this skeletal control. This may be NULL in some cases when bInIsSelected is false.
	 */
	UE_API virtual void OnNodeSelected(bool bInIsSelected, class FEditorModeTools& InModeTools, struct FAnimNode_Base* InRuntimeNode);

	/** Pose Watch change notification callback. Should be called every time a pose watch on this node is created or destroyed. */
	UE_API virtual void OnPoseWatchChanged(const bool IsPoseWatchEnabled, TObjectPtr<class UPoseWatch> InPoseWatch, FEditorModeTools& InModeTools, FAnimNode_Base* InRuntimeNode);

	/**
	 * Override this function to push an editor mode when this node is selected
	 * @return the editor mode to use when this node is selected
	 */
	UE_API virtual FEditorModeID GetEditorMode() const;

	// Draw function for supporting visualization
	virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent * PreviewSkelMeshComp) const {}
	/**
	 *	Draw function called on nodes that are selected and / or have a pose watch enabled.
	 *	Default implementation calls the basic draw function for selected nodes and does nothing for pose watched nodes. Nodes
	 *	that should render something when a pose watch is enabled but they are not selected should override this function.
	 */
	UE_API virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp, const bool bIsSelected, const bool bIsPoseWatchEnabled) const;

	// Canvas draw function to draw to viewport
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas, USkeletalMeshComponent * PreviewSkelMeshComp) const {}

	// Function to collect strings from nodes to display in the viewport.
	// Use this rather than DrawCanvas when adding general text to the viewport.
	virtual void GetOnScreenDebugInfo(TArray<FText>& DebugInfo, FAnimNode_Base* RuntimeAnimNode, USkeletalMeshComponent* PreviewSkelMeshComp) const {}

	/** Called after editing a default value to update internal node from pin defaults. This is needed for forwarding code to propagate values to preview. */
	virtual void CopyPinDefaultsToNodeData(UEdGraphPin* InPin) {}

	/** Called to propagate data from the internal node to the preview in Persona. */
	virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) {}

	// BEGIN Interface to support transition getter
	// if you return true for DoesSupportExposeTimeForTransitionGetter
	// you should implement all below functions
	virtual bool DoesSupportTimeForTransitionGetter() const { return false; }
	virtual UAnimationAsset* GetAnimationAsset() const { return nullptr; }
	virtual TSubclassOf<UAnimationAsset> GetAnimationAssetClass() const { return nullptr; }
	virtual const TCHAR* GetTimePropertyName() const { return nullptr; }
	virtual UScriptStruct* GetTimePropertyStruct() const { return nullptr; }
	// END Interface to support transition getter

	// can customize details tab 
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder){ }

	/** Try to find the preview node instance for this anim graph node */
	UE_API FAnimNode_Base* FindDebugAnimNode(USkeletalMeshComponent * PreviewSkelMeshComp) const;

	template<typename NodeType>
	NodeType* GetActiveInstanceNode(UObject* AnimInstanceObject) const
	{
		if(!AnimInstanceObject)
		{
			return nullptr;
		}

		if(UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstanceObject->GetClass()))
		{
			return AnimClass->GetPropertyInstance<NodeType>(AnimInstanceObject, NodeGuid);
		}

		return nullptr;
	}

	/** 
	 *	Returns whether this node supports the supplied asset class
	 *	@param	bPrimaryAssetHandler	Is this the 'primary' handler for this asset (the node that should be created when asset is dropped)
	 */
	UE_API virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const;

	// Event that observers can bind to so that they are notified about changes
	// made to this node through the property system
	DECLARE_EVENT_OneParam(UAnimGraphNode_Base, FOnNodePropertyChangedEvent, FPropertyChangedEvent&);
	FOnNodePropertyChangedEvent& OnNodePropertyChanged() { return PropertyChangeEvent;	}

	// Event that observers can bind to so that they are notified about changes to pin visibility
	DECLARE_EVENT_TwoParams(UAnimGraphNode_Base, FPinVisibilityChangedEvent, bool /*bInVisible*/, int32 /*InOptionalPinIndex*/);
	FPinVisibilityChangedEvent& OnPinVisibilityChanged() { return PinVisibilityChangedEvent; }
	
	/**
	 * Helper function to check whether a pin is valid and linked to something else in the graph
	 * @param	InPinName		The name of the pin @see UEdGraphNode::FindPin
	 * @param	InPinDirection	The direction of the pin we are looking for. If this is EGPD_MAX, all directions are considered
	 * @return true if the pin is present and connected
	 */
	UE_API bool IsPinExposedAndLinked(const FString& InPinName, const EEdGraphPinDirection Direction = EGPD_MAX) const;

	/**
	 * Helper function to check whether a pin is valid and bound via property access
	 * @param	InPinName		The name of the pin @see UEdGraphNode::FindPin
	 * @param	InPinDirection	The direction of the pin we are looking for. If this is EGPD_MAX, all directions are considered
	 * @return true if the pin is present and bound
	 */
	UE_API bool IsPinExposedAndBound(const FString& InPinName, const EEdGraphPinDirection InDirection = EGPD_MAX) const;

	/**
	 * Helper function to check whether a pin is not linked, not bound via property access and still has its default value
	 * @param	InPinName		The name of the pin @see UEdGraphNode::FindPin
	 * @param	InPinDirection	The direction of the pin we are looking for. If this is EGPD_MAX, all directions are considered
	 * @return true if the pin is unlinked, unbound and still has its default value
	 */
	UE_API bool IsPinUnlinkedUnboundAndUnset(const FString& InPinName, const EEdGraphPinDirection InDirection) const;

	// Event that is broadcast to inform observers that the node title has changed
	// The default SAnimationGraphNode uses this to invalidate cached node title text
	DECLARE_EVENT(UAnimGraphNode_Base, FOnNodeTitleChangedEvent);
	FOnNodeTitleChangedEvent& OnNodeTitleChangedEvent() { return NodeTitleChangedEvent; }

	using FNodeAttributeArray = TArray<FName, TInlineAllocator<4>>;

	// Get the named attribute types that this node takes (absorbs) as inputs. Other attributes are assumed to 'pass through' this node.
	virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const {}

	// Get the named attribute types that this node provides as outputs. Other attributes are assumed to 'pass through' this node.
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const {}

	// @return wether to show graph attribute icons on pins for this node.
	virtual bool ShouldShowAttributesOnPins() const { return true; }

	// Some anim graph nodes can act as proxies to outer nodes (e.g. sink nodes in nested graphs).
	// Returning that outer node here allows attribute visualization to be forwarded to the inner node from the outer.
	// @return the proxy for attribute display
	virtual const UAnimGraphNode_Base* GetProxyNodeForAttributes() const { return this; }

	// Sets the visibility of the specified pin, reconstructs the node if it changes
	UE_API void SetPinVisibility(bool bInVisible, int32 InOptionalPinIndex);

	// Arguments used to construct a property binding widget
	struct FAnimPropertyBindingWidgetArgs
	{
		FAnimPropertyBindingWidgetArgs(const TArray<UAnimGraphNode_Base*>& InNodes, FProperty* InPinProperty, FName InPinName, FName InBindingName, int32 InOptionalPinIndex)
			: Nodes(InNodes)
			, PinProperty(InPinProperty)
			, PinName(InPinName)
			, BindingName(InBindingName)
			, OptionalPinIndex(InOptionalPinIndex)
			, bOnGraphNode(true)
			, bPropertyIsOnFNode(true)
		{
		}

		// The nodes to display the binding for
		TArray<UAnimGraphNode_Base*> Nodes;
		
		// The pin property for this binding
		FProperty* PinProperty = nullptr;
		
		// The name of the pin
		FName PinName = NAME_None;

		// The name of the property we are binding to
		FName BindingName = NAME_None;
		
		// The optional pin index that refers to this pin
		int32 OptionalPinIndex = INDEX_NONE;
		
		// Whether this is for display on a graph node
		bool bOnGraphNode = true;

		// Whether the property is on the FAnimNode_Base of this anim node
		bool bPropertyIsOnFNode = true;
		
		// Delegate used to to access property bindings to modify for the specified node
		DECLARE_DELEGATE_TwoParams(FOnGetOptionalPins, UAnimGraphNode_Base* /*InNode*/, TArrayView<FOptionalPinFromProperty>& /*OutOptionalPins*/);
		FOnGetOptionalPins OnGetOptionalPins;
		
		// Delegate used to set pin visibility
		DECLARE_DELEGATE_ThreeParams(FOnSetPinVisibility, UAnimGraphNode_Base* /*InNode*/, bool /*bInVisible*/, int32 /*InOptionalPinIndex*/);
		FOnSetPinVisibility OnSetPinVisibility;

		// Menu extender used to add custom entries in the binding menu
		TSharedPtr<FExtender> MenuExtender;
	};
	
	// Make a property binding widget to edit the bindings of the passed-in nodes
	static UE_API TSharedRef<SWidget> MakePropertyBindingWidget(const FAnimPropertyBindingWidgetArgs& InArgs);

	// Get the property corresponding to a pin. For array element pins returns the outer array property. Returns null if a property cannot be found.
	UE_API FProperty* GetPinProperty(const UEdGraphPin* InPin) const;
	UE_API virtual FProperty* GetPinProperty(FName InPinName) const;
	
	// Check whether the named pin is bindable
	UE_API virtual bool IsPinBindable(const UEdGraphPin* InPin) const;

	// Get the tag for this node, if any
	FName GetTag() const { return Tag; }

	// Set the tag for this node
	UE_API void SetTag(FName InTag);

	// Get the currently-debugged runtime anim node (in the anim BP debugger that this node is currently being edited in)
	// @return nullptr if the node cannot be found
	FAnimNode_Base* GetDebuggedAnimNode() const { return GetDebuggedAnimNode<FAnimNode_Base>(); }

	// Get the currently-debugged runtime anim node of a specified type (in the anim BP debugger that this node is currently being edited in)
	// @return nullptr if the node cannot be found
	template< typename TNodeType > TNodeType* GetDebuggedAnimNode() const;

	// Refreshes the debugged component post-edit.
	// This is required to see changes as the component may be either an editor-only component that is not ticking,
	// or in a paused PIE world
	UE_API void PostEditRefreshDebuggedComponent();

	// Gets editor information for all the bound anim node functions (category metadata string, member variable's name)
	// Used by SAnimGraphNode to display all bound functions of an anim node.
	UE_API virtual void GetBoundFunctionsInfo(TArray<TPair<FName, FName>> & InOutBindingsInfo);
	
	// Check if a specified function reference appears to be valid by inspecting only the validity of the name and guid
	static UE_API bool IsPotentiallyBoundFunction(const FMemberReference& FunctionReference);

	// Check whether the specified property is bound via PropertyBindings
	UE_API virtual bool HasBinding(FName InPropertyName) const;

	// Update the property binding name
	UE_API bool RedirectBinding(FName OldPropertyBindingName, FName NewPropertyBindingName);
	
	// Get the bindings for this node
	const UAnimGraphNodeBinding* GetBinding() const { return Binding; }

	// Get the mutable bindings for this node
	UAnimGraphNodeBinding* GetMutableBinding() { return Binding; }

	// Remove any bindings for the specified name
	UE_API void RemoveBindings(FName InBindingName);

	// Gets the animation FNode type represented by this ed graph node
	UE_API UScriptStruct* GetFNodeType() const;

	// Gets the animation FNode property represented by this ed graph node
	UE_API FStructProperty* GetFNodeProperty() const;

	// Get the runtime anim node that we template
	UE_API FAnimNode_Base* GetFNode();

protected:
	friend class FAnimBlueprintCompilerContext;
	friend class FAnimGraphNodeDetails;
	friend class UAnimBlueprintExtension;
	friend class UAnimBlueprintExtension_Base;
	friend class SAnimationGraphNode;
	friend class UAnimationGraphSchema;

	// Set the tag for this node but without regenerating any BP data for tagging
	void SetTagInternal(FName InTag) { Tag = InTag; }
	
	// Get the extension types that this node type holds on the anim blueprint. Some extension types are always requested by the system
	virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const {}
	
	// This will be called when a pose link is found, and can be called with PoseProperty being either of:
	//  - an array property (ArrayIndex >= 0)
	//  - a single pose property (ArrayIndex == INDEX_NONE)
	UE_API virtual void CreatePinsForPoseLink(FProperty* PoseProperty, int32 ArrayIndex);

	//
	UE_API virtual FPoseLinkMappingRecord GetLinkIDLocation(const UScriptStruct* NodeType, UEdGraphPin* InputLinkPin);

	/** Get the property (and possibly array index) associated with the supplied pin */
	UE_API virtual void GetPinAssociatedProperty(const UScriptStruct* NodeType, const UEdGraphPin* InputPin, FProperty*& OutProperty, int32& OutIndex) const;

	// Process this node's data during compilation
	UE_API void ProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	// Process this node's data during compilation (override point)
	virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}

	// Copy this node's data during the last phase of compilation where term defaults are copied to the new CDO
	UE_API void CopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	// Override point for CopyTermDefaultsToDefaultObject
	// Copy this node's data during the last phase of compilation where term defaults are copied to the new CDO
	virtual void OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) {}

	// Called to override the assets held on a runtime anim node. Implements per-node logic for child anim blueprints.
	UE_API void OverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const;

	// Override point for OverrideAssets
	virtual void OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const {}

	// Whether this node should create BP evaluation handlers as part of compilation
	virtual bool ShouldCreateStructEvalHandlers() const { return true; }
	
	// Allocates or reallocates pins
	UE_API void InternalPinCreation(TArray<UEdGraphPin*>* OldPins);

	// Override point to create custom pins
	// @param	OldPins		In the case of reconstruction, the OldPins array contains all the pins that the node had prior, otherwise if the
	// 				node is being created anew the array ptr will be null
	virtual void CreateCustomPins(TArray<UEdGraphPin*>* OldPins) {}

	// Get the pin binding info for the supplied pin
	// @param	InPinName	The name of the pin
	// @param	OutBindingName	The name of the binding that this pin represents (for array pins the binding name includes the array index as e.g. Binding_1)
	// @param	OutPinProperty	The property that the binding represents
	// @param	OutOptionalPinIndex	The optional pin index (index into the ShowPinForProperties array) 
	// @return false if the pin cannot be bound
	UE_API virtual bool GetPinBindingInfo(FName InPinName, FName& OutBindingName, FProperty*& OutPinProperty, int32& OutOptionalPinIndex) const;

	FOnNodePropertyChangedEvent PropertyChangeEvent;

	FOnNodeTitleChangedEvent NodeTitleChangedEvent;

	FPinVisibilityChangedEvent PinVisibilityChangedEvent;

	// Helper function used to refresh the type of a binding
	UE_API void RecalculateBindingType(FAnimGraphNodePropertyBinding& InBinding);
	
	/** @return the current object being debugged from the blueprint for this node. Can be nullptr. */
	UE_API UObject* GetObjectBeingDebugged() const;

	/** Helper function used to validate anim node function references */
	UE_API void ValidateFunctionRef(FName InPropertyName, const FMemberReference& InRef, const FText& InFunctionName, FCompilerResultsLog& MessageLog);
	
	// Create the bindings subobject if required
	UE_API void EnsureBindingsArePresent();

protected:
	// Old shown pins. Needs to be a member variable to track pin visibility changes between Pre and PostEditChange 
	TArray<FName> OldShownPins;
};

template<class AssetType>
void UAnimGraphNode_Base::HandleAnimReferenceReplacement(AssetType*& OriginalAsset, const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	AssetType* CacheOriginalAsset = OriginalAsset;
	OriginalAsset = nullptr;

	if (UAnimationAsset* const* ReplacementAsset = AnimAssetReplacementMap.Find(CacheOriginalAsset))
	{
		OriginalAsset = Cast<AssetType>(*ReplacementAsset);
	}
}

template<class AssetType>
void UAnimGraphNode_Base::HandleAnimReferenceReplacement(TObjectPtr<AssetType>& OriginalAsset, const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
{
	HandleAnimReferenceReplacement(static_cast<AssetType*&>(OriginalAsset), AnimAssetReplacementMap);
}

template<class TNodeType> TNodeType* UAnimGraphNode_Base::GetDebuggedAnimNode() const
{
	if (UObject* ActiveObject = GetObjectBeingDebugged())
	{
		if (UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>((UObject*)ActiveObject->GetClass()))
		{
			return Class->GetPropertyInstance<TNodeType>(ActiveObject, this);
		}
	}

	return nullptr;
}

#undef UE_API
