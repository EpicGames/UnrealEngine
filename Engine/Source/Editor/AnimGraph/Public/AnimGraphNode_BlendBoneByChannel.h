// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_BlendBoneByChannel.h"
#include "AnimGraphNode_BlendBoneByChannel.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendBoneByChannel : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_BlendBoneByChannel BlendNode;

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	virtual FString GetNodeCategory() const override;
	//~ End UAnimGraphNode_Base Interface
};
