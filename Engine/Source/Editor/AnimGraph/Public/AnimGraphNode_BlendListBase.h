// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_BlendListBase.generated.h"

#define UE_API ANIMGRAPH_API

UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_BlendListBase : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void Serialize(FArchive& Ar) override;

	// UEdGraphNode interface
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	UE_API virtual FString GetNodeCategory() const override;
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UAnimGraphNode_Base interface

protected:
	// removes removed pins and adjusts array indices of remained pins
	UE_API void RemovePinsFromOldPins(TArray<UEdGraphPin*>& OldPins, int32 RemovedArrayIndex);

	int32 RemovedPinArrayIndex;
};

#undef UE_API
