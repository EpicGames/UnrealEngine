// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_BlendListBase.h"
#include "AnimNodes/AnimNode_BlendListByBool.h"
#include "AnimGraphNode_BlendListByBool.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendListByBool : public UAnimGraphNode_BlendListBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_BlendListByBool Node;

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	// End of UAnimGraphNode_Base interface
};
