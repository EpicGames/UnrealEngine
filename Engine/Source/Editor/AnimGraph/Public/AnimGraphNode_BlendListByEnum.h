// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "NodeDependingOnEnumInterface.h"
#include "AnimGraphNode_BlendListBase.h"
#include "AnimNodes/AnimNode_BlendListByEnum.h"
#include "AnimGraphNode_BlendListByEnum.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendListByEnum : public UAnimGraphNode_BlendListBase, public INodeDependingOnEnumInterface
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_BlendListByEnum Node;
	
protected:
	/** Name of the enum being switched on */
	UPROPERTY()
	TObjectPtr<UEnum> BoundEnum;

	UPROPERTY()
	TArray<FName> VisibleEnumEntries;
public:
	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void Serialize(FArchive& Ar) override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface

	// UAnimGraphNode_Base interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual void BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog) override;
	virtual void PreloadRequiredAssets() override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of UAnimGraphNode_Base interface

	//@TODO: Generalize this behavior (returning a list of actions/delegates maybe?)
	virtual void RemovePinFromBlendList(UEdGraphPin* Pin);

	// INodeDependingOnEnumInterface
	virtual class UEnum* GetEnum() const override { return BoundEnum; }
	virtual void ReloadEnum(class UEnum* InEnum) override;
	virtual bool ShouldBeReconstructedAfterEnumChanged() const override {return true;}
	// End of INodeDependingOnEnumInterface
protected:
	// Exposes a pin corresponding to the specified element name
	void ExposeEnumElementAsPin(FName EnumElementName);

	// Gets information about the specified pin.  If both bIsPosePin and bIsTimePin are false, the index is meaningless
	static void GetPinInformation(const FString& InPinName, int32& Out_PinIndex, bool& Out_bIsPosePin, bool& Out_bIsTimePin);

private:
	void ClearEnum();

	static void RefreshAvailableEnums();

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;

	/** Shared array that contains all possible enums to select from */
	static TArray<TSharedPtr<FString>> AvailableEnums;
};
