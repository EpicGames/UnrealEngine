// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_BlendListBase.h"
#include "AnimNodes/AnimNode_BlendListByInt.h"
#include "AnimGraphNode_BlendListByInt.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendListByInt : public UAnimGraphNode_BlendListBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_BlendListByInt Node;

	// Adds a new pose pin
	//@TODO: Generalize this behavior (returning a list of actions/delegates maybe?)
	ANIMGRAPH_API virtual void AddPinToBlendList();
	ANIMGRAPH_API virtual void RemovePinFromBlendList(UEdGraphPin* Pin);

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PostPlacedNewNode() override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface
};
