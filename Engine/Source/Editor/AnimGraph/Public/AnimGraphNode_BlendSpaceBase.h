// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "Animation/BlendSpace.h"
#include "AnimGraphNode_BlendSpaceBase.generated.h"

UCLASS(Abstract, MinimalAPI)
class UAnimGraphNode_BlendSpaceBase : public UAnimGraphNode_AssetPlayerBase
{
	GENERATED_UCLASS_BODY()

	// UEdGraphNode interface
	ANIMGRAPH_API virtual FLinearColor GetNodeTitleColor() const override;
	ANIMGRAPH_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	ANIMGRAPH_API virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	ANIMGRAPH_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	ANIMGRAPH_API virtual void PreloadRequiredAssets() override;
	ANIMGRAPH_API virtual void PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const override;
	ANIMGRAPH_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const;
	virtual TSubclassOf<UAnimationAsset> GetAnimationAssetClass() const { return UBlendSpace::StaticClass(); }
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	// End of UAnimGraphNode_Base interface

protected:
	UBlendSpace* GetBlendSpace() const { return Cast<UBlendSpace>(GetAnimationAsset()); }

	/** Util to determine is an asset class is an aim offset */
	static bool IsAimOffsetBlendSpace(const UClass* BlendSpaceClass);
};
