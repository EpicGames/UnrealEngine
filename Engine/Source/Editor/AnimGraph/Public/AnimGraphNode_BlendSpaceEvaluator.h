// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_BlendSpaceBase.h"
#include "AnimNodes/AnimNode_BlendSpaceEvaluator.h"
#include "AnimGraphNode_BlendSpaceEvaluator.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendSpaceEvaluator : public UAnimGraphNode_BlendSpaceBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_BlendSpaceEvaluator Node;

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual void BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog) override;
	// Interface to support transition getter
	virtual bool DoesSupportTimeForTransitionGetter() const override;
	virtual UAnimationAsset* GetAnimationAsset() const override;
	virtual const TCHAR* GetTimePropertyName() const override;
	virtual UScriptStruct* GetTimePropertyStruct() const override;
	virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	virtual void OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const override;
	// End of UAnimGraphNode_Base interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface

	// UAnimGraphNode_AssetPlayerBase interface
	virtual void SetAnimationAsset(UAnimationAsset* Asset) override;
	virtual void CopySettingsFromAnimationAsset(UAnimationAsset* Asset) override;
	// End of UAnimGraphNode_AssetPlayerBase interface

private:
	FText GetNodeTitleForBlendSpace(ENodeTitleType::Type TitleType, UBlendSpace* InBlendSpace) const;

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;

};
