// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "AnimNodes/AnimNode_BlendSpaceGraph.h"
#include "AnimGraphNode_BlendSpaceGraph.generated.h"

#define UE_API ANIMGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FCompilerResultsLog;

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendSpaceGraph : public UAnimGraphNode_BlendSpaceGraphBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_BlendSpaceGraph Node;

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	
	// UK2Node interface
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;

	// UAnimGraphNode_Base interface
	UE_API virtual void BakeDataDuringCompilation(FCompilerResultsLog& MessageLog) override;
};

#undef UE_API
