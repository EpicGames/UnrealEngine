// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Containers/ArrayView.h"
#include "AnimGraphNode_BlendSpaceGraphBase.generated.h"

#define UE_API ANIMGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class IAnimBlueprintCopyTermDefaultsContext;
class IAnimBlueprintNodeCopyTermDefaultsContext;
class IAnimBlueprintGeneratedClassCompiledData;
class UBlendSpaceGraph;
class UAnimationBlendSpaceSampleGraph;

UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_BlendSpaceGraphBase : public UAnimGraphNode_Base
{
	GENERATED_BODY()

public:
	UE_API UAnimGraphNode_BlendSpaceGraphBase();

	// Access the graphs for each sample
	TArrayView<UEdGraph* const> GetGraphs() const { return Graphs; }

	// Access the 'dummy' blendspace graph
	UBlendSpaceGraph* GetBlendSpaceGraph() const { return BlendSpaceGraph; }

	// Adds a new graph to the internal array
	UE_API UAnimationBlendSpaceSampleGraph* AddGraph(FName InSampleName, UAnimSequence* InSequence);

	/** Returns the sample index associated with the graph, or -1 if not found */
	UE_API int32 GetSampleIndex(const UEdGraph* Graph) const;

	// Removes the graph at the specified index
	UE_API void RemoveGraph(int32 InSampleIndex);

	// Replaces the graph at the specified index
	UE_API void ReplaceGraph(int32 InSampleIndex, UAnimSequence* InSequence);

	// Setup this node from the specified asset
	UE_API void SetupFromAsset(const FAssetData& InAssetData, bool bInIsTemplateNode);

	// UEdGraphNode interface
	UE_API virtual void PostPlacedNewNode() override;

	// @return the sync group name assigned to this node
	UE_API FName GetSyncGroupName() const;

	// Set the sync group name assigned to this node
	UE_API void SetSyncGroupName(FName InName);

protected:
	// Get the name of the blendspace graph
	UE_API FString GetBlendSpaceGraphName() const;

	// Get the name of the blendspace
	UE_API FString GetBlendSpaceName() const;

	// Setup this node from the specified class
	UE_API void SetupFromClass(TSubclassOf<UBlendSpace> InBlendSpaceClass, bool bInIsTemplateNode);

	// Internal blendspace
	UPROPERTY()
	TObjectPtr<UBlendSpace> BlendSpace;

	// Blendspace class, for template nodes
	UPROPERTY()
	TSubclassOf<UBlendSpace> BlendSpaceClass;

	// Dummy blendspace graph (used for navigation only)
	UPROPERTY()
	TObjectPtr<UBlendSpaceGraph> BlendSpaceGraph;

	// Linked animation graphs for sample points
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> Graphs;

	// Skeleton name used for filtering unloaded assets 
	FString SkeletonName;

protected:
	// UEdGraphNode interface
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual TArray<UEdGraph*> GetSubGraphs() const override;
	UE_API virtual void DestroyNode() override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;
	UE_API virtual TSharedPtr<INameValidatorInterface> MakeNameValidator() const override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	UE_API virtual void PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const override;

	// UAnimGraphNode_Base interface
	UE_API virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	UE_API virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	UE_API virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;

	// UK2Node interface
	UE_API virtual void PreloadRequiredAssets() override;
	UE_API virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;

	// Helper function for compilation
	UE_API UAnimGraphNode_Base* ExpandGraphAndProcessNodes(UEdGraph* SourceGraph, UAnimGraphNode_Base* SourceRootNode, IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData);

	// Helper function for AddGraph/ReplaceGraph - builds the new graph but doesn't add it to Graphs array.
	UE_API UAnimationBlendSpaceSampleGraph* AddGraphInternal(FName InSampleName, UAnimSequence* InSequence);
};

#undef UE_API
