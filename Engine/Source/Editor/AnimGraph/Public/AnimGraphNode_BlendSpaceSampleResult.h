// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_BlendSpaceSampleResult.h"
#include "AnimGraphNode_BlendSpaceSampleResult.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendSpaceSampleResult : public UAnimGraphNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_BlendSpaceSampleResult Node;

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsNodeRootSet() const override { return true; }
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	virtual bool IsSinkNode() const override { return true; }
	virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual UAnimGraphNode_Base* GetProxyNodeForAttributes() const override;
	//~ End UAnimGraphNode_Base Interface
};
