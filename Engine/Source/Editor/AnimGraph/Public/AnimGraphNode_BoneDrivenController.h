// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_BoneDrivenController.h"
#include "AnimGraphNode_BoneDrivenController.generated.h"

#define UE_API ANIMGRAPH_API

class FCompilerResultsLog;
class FPrimitiveDrawInterface;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class USkeletalMeshComponent;

/**
 * This is the 'source version' of a bone driven controller, which maps part of the state from one bone to another (e.g., 2 * source.x -> target.z)
 */
UCLASS(MinimalAPI)
class UAnimGraphNode_BoneDrivenController : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_BoneDrivenController Node;

public:
	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of UAnimGraphNode_Base interface

	// UAnimGraphNode_SkeletalControlBase interface
	UE_API virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const override;
	// End of UAnimGraphNode_SkeletalControlBase interface

protected:

	// UAnimGraphNode_SkeletalControlBase protected interface
	UE_API virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase protected interface

	// Should non-curve mapping values be shown (multiplier, range)?
	static UE_API EVisibility AreNonCurveMappingValuesVisible(IDetailLayoutBuilder* DetailLayoutBuilder);
	static UE_API EVisibility AreRemappingValuesVisible(IDetailLayoutBuilder* DetailLayoutBuilder);

	// Should destination bone or morph target properties be visible
	static UE_API EVisibility AreTargetBonePropertiesVisible(IDetailLayoutBuilder* DetailLayoutBuilder);
	static UE_API EVisibility AreTargetCurvePropertiesVisible(IDetailLayoutBuilder* DetailLayoutBuilder);

	static UE_API void AddTripletPropertyRow(const FText& Name, const FText& Tooltip, IDetailCategoryBuilder& Category, TSharedRef<IPropertyHandle> PropertyHandle, const FName XPropertyName, const FName YPropertyName, const FName ZPropertyName, TAttribute<EVisibility> VisibilityAttribute);
	static UE_API void AddRangePropertyRow(const FText& Name, const FText& Tooltip, IDetailCategoryBuilder& Category, TSharedRef<IPropertyHandle> PropertyHandle, const FName MinPropertyName, const FName MaxPropertyName, TAttribute<EVisibility> VisibilityAttribute);
	static UE_API FText ComponentTypeToText(EComponentType::Type Component);
};

#undef UE_API
