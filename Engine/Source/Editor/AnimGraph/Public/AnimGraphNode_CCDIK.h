// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_CCDIK.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_CCDIK.generated.h"

// Editor node for CCDIK IK skeletal controller
UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_CCDIK : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_CCDIK Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* AnimNode) override;
	virtual FEditorModeID GetEditorMode() const override;
	// End of UAnimGraphNode_Base interface

	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};