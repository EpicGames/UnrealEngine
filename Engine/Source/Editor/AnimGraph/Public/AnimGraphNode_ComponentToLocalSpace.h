// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNodeSpaceConversions.h"
#include "AnimGraphNode_ComponentToLocalSpace.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_ComponentToLocalSpace : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_ConvertComponentToLocalSpace Node;

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const override;
	// End of UAnimGraphNode_Base interface
};
