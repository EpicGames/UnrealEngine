// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_Constraint.h"
#include "EdGraph/EdGraphNodeUtils.h" // for FNodeTitleTextTable
#include "AnimGraphNode_Constraint.generated.h"

#define UE_API ANIMGRAPH_API

UCLASS(MinimalAPI)
class UAnimGraphNode_Constraint : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_Constraint Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

	// UObject interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

protected:
	// UAnimGraphNode_Base interface
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	UE_API virtual void PostProcessPinName(const UEdGraphPin* Pin, FString& DisplayName) const override;
	// End of UAnimGraphNode_Base interface

	// UAnimGraphNode_SkeletalControlBase interface
	UE_API virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const override;
	UE_API virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};

#undef UE_API
