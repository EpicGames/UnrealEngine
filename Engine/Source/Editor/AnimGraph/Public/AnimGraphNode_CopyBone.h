// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_CopyBone.h"
#include "AnimGraphNode_CopyBone.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_CopyBone : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_CopyBone Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
