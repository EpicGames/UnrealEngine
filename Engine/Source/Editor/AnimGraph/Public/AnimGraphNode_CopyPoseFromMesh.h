// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_CopyPoseFromMesh.h"
#include "AnimGraphNode_CopyPoseFromMesh.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_CopyPoseFromMesh : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_CopyPoseFromMesh Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual bool UsingCopyPoseFromMesh() const override { return true; }
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UAnimGraphNode_Base interface
};
