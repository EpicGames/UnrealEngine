// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_CurveSource.h"
#include "AnimGraphNode_CurveSource.generated.h"

/** Supply curves from some external source (e.g. audio) */
UCLASS(MinimalAPI)
class UAnimGraphNode_CurveSource : public UAnimGraphNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_CurveSource Node;

public:
	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual FString GetNodeCategory() const override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UAnimGraphNode_Base interface
};
