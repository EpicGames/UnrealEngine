// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_CustomProperty.h"
#include "IClassVariableCreator.h"

#include "AnimGraphNode_CustomProperty.generated.h"

#define UE_API ANIMGRAPH_API

class FCompilerResultsLog;
class IDetailLayoutBuilder;
class IPropertyHandle;

UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_CustomProperty : public UAnimGraphNode_Base, public IClassVariableCreator
{
	GENERATED_BODY()

public:

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	
	// IClassVariableCreator interface
	UE_API virtual void CreateClassVariablesFromBlueprint(IAnimBlueprintVariableCreationContext& InCreationContext) override;

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput /*= NULL*/) const override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	//~ End UEdGraphNode Interface.

	// UAnimGraphNode_Base interface
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	UE_API virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void CreateCustomPins(TArray<UEdGraphPin*>* OldPins) override;
	UE_API virtual FProperty* GetPinProperty(FName InPinName) const override;
	UE_API virtual bool IsPinBindable(const UEdGraphPin* InPin) const override;
	UE_API virtual bool GetPinBindingInfo(FName InPinName, FName& OutBindingName, FProperty*& OutPinProperty, int32& OutOptionalPinIndex) const override;
	UE_API virtual bool HasBinding(FName InBindingName) const override;
	
	// Gets the property on InOwnerInstanceClass that corresponds to InInputPin
	UE_API void GetInstancePinProperty(const IAnimBlueprintCompilationContext& InCompilationContext, UEdGraphPin* InInputPin, FProperty*& OutProperty);
	// Gets the unique name for the property linked to a given pin
	UE_API FString GetPinTargetVariableName(const UEdGraphPin* InPin) const;
	// Gets the unique name for the property linked to a given pin name
	UE_API FString GetPinTargetVariableName(FName InPinName) const;
	// Gets Target Class this properties to link
	UE_API UClass* GetTargetClass() const;
	virtual UStruct* GetTargetStruct() const { return GetTargetClass(); }
	// Add Source and Target Properties - Check FAnimNode_CustomProperty
	UE_API void AddSourceTargetProperties(const FName& InSourcePropertyName, const FName& InTargetPropertyName);
	// Helper used to get the skeleton class we are targeting
	UE_API virtual UClass* GetTargetSkeletonClass() const;
	virtual UStruct* GetTargetSkeletonStruct() const { return GetTargetSkeletonClass(); }

	// ----- UI CALLBACKS ----- //
	// User changed the instance class etc.
	UE_API void OnStructuralPropertyChanged(IDetailLayoutBuilder* DetailBuilder);
	// User changed the instance class
	UE_API void OnInstanceClassChanged(IDetailLayoutBuilder* DetailBuilder);
protected:
	friend struct FCustomPropertyOptionalPinManager;
	friend class SAnimationGraphNode;
	friend class UAnimationGraphSchema;
	
	/** List of property names we know to exist on the target class, so we can detect when
	 *  Properties are added or removed on reconstruction
	 * Deprecated, use CustomPinProperties instead.
	 */
	UPROPERTY()
	TArray<FName> KnownExposableProperties_DEPRECATED;

	/** Names of properties the user has chosen to expose. Deprecated, use CustomPinProperties instead. */
	UPROPERTY()
	TArray<FName> ExposedPropertyNames_DEPRECATED;

	/** Exposed pin data for custom properties */
	UPROPERTY()
	TArray<FOptionalPinFromProperty> CustomPinProperties;
	
	// Gets a property's type as FText (for UI)
	UE_API FText GetPropertyTypeText(FProperty* Property);

	// internal node accessor
	virtual FAnimNode_CustomProperty* GetCustomPropertyNode() PURE_VIRTUAL(UAnimGraphNode_CustomProperty::GetCustomPropertyNode, return nullptr;);
	virtual const FAnimNode_CustomProperty* GetCustomPropertyNode() const PURE_VIRTUAL(UAnimGraphNode_CustomProperty::GetCustomPropertyNode, return nullptr;);

	// Check whether the specified property is structural (i.e. should we rebuild the UI if it changes)
	virtual bool IsStructuralProperty(FProperty* InProperty) const { return false; }

	// Whether this node needs a valid target class up-front
	virtual bool NeedsToSpecifyValidTargetClass() const { return true; }
	
	// Sets the visibility of the specified pin, reconstructs the node if it changes
	UE_API void SetCustomPinVisibility(bool bInVisible, int32 InOptionalPinIndex);

	// Helper function for GetPinTargetVariableName
	UE_API FString GetPinTargetVariableNameBase(FName InPinName) const;
};

#undef UE_API
