// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_StateResult.h"
#include "AnimGraphNode_CustomTransitionResult.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_CustomTransitionResult : public UAnimGraphNode_StateResult
{
	GENERATED_UCLASS_BODY()

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface
};
