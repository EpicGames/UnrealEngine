// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_DeadBlending.h"
#include "AnimGraphNode_DeadBlending.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_DeadBlending : public UAnimGraphNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_DeadBlending Node;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual void OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
};
