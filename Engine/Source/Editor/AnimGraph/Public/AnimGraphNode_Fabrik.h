// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_Fabrik.h"
#include "AnimGraphNode_Fabrik.generated.h"

class FPrimitiveDrawInterface;
class USkeletalMeshComponent;

// Editor node for FABRIK IK skeletal controller
UCLASS(MinimalAPI)
class UAnimGraphNode_Fabrik : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_Fabrik Node;

public:
	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* AnimNode) override;
	virtual FEditorModeID GetEditorMode() const override;
	virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent * PreviewSkelMeshComp) const override;
	// End of UAnimGraphNode_Base interface

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};
