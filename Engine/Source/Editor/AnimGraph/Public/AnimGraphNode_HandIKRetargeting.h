// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_HandIKRetargeting.h"
#include "AnimGraphNode_HandIKRetargeting.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_HandIKRetargeting : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_HandIKRetargeting Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};
