// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_RefPoseBase.h"
#include "AnimGraphNode_IdentityPose.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_IdentityPose : public UAnimGraphNode_RefPoseBase
{
	GENERATED_UCLASS_BODY()

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface
};
