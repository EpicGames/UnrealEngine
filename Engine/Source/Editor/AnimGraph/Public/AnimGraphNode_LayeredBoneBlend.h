// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_BlendListBase.h"
#include "AnimNodes/AnimNode_LayeredBoneBlend.h"
#include "AnimGraphNode_LayeredBoneBlend.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_LayeredBoneBlend : public UAnimGraphNode_BlendListBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_LayeredBoneBlend Node;

	// Adds a new pose pin
	//@TODO: Generalize this behavior (returning a list of actions/delegates maybe?)
	ANIMGRAPH_API virtual void AddPinToBlendByFilter();
	ANIMGRAPH_API virtual void RemovePinFromBlendByFilter(UEdGraphPin* Pin);

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface

	// UAnimGraphNode_Base interface
	virtual FString GetNodeCategory() const override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void PreloadRequiredAssets() override;
	// End of UAnimGraphNode_Base interface


	// Gives each visual node a chance to validate that they are still valid in the context of the compiled class, giving a last shot at error or warning generation after primary compilation is finished
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	// End of UAnimGraphNode_Base interface
};
