// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "K2Node_ExternalGraphInterface.h"
#include "Animation/AnimNode_LinkedAnimGraph.h"
#include "Engine/MemberReference.h"

#include "AnimGraphNode_LinkedAnimGraph.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_LinkedAnimGraph : public UAnimGraphNode_LinkedAnimGraphBase, public IK2Node_ExternalGraphInterface
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_LinkedAnimGraph Node;
	
	// Begin UObject
	virtual void Serialize(FArchive& Ar) override;
	
	// Begin UEdGraphNode
	virtual void PostPasteNode() override;

	// Begin UK2Node
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	
	// Begin UAnimGraphNode_CustomProperty
	virtual FAnimNode_CustomProperty* GetCustomPropertyNode() override;
	virtual const FAnimNode_CustomProperty* GetCustomPropertyNode() const override;

	// Begin UAnimGraphNode_LinkedAnimGraphBase
	virtual FAnimNode_LinkedAnimGraph* GetLinkedAnimGraphNode() override;
	virtual const FAnimNode_LinkedAnimGraph* GetLinkedAnimGraphNode() const override;

	// IK2Node_ExternalGraphInterface interface
	virtual TArray<UEdGraph*> GetExternalGraphs() const override;

private:
	// Setup the node from a specified anim BP
	void SetupFromAsset(const FAssetData& InAssetData, bool bInIsTemplateNode);
};

UE_DEPRECATED(4.24, "UAnimGraphNode_SubInstance has been renamed to UAnimGraphNode_LinkedAnimGraph")
typedef UAnimGraphNode_LinkedAnimGraph UAnimGraphNode_SubInstance;