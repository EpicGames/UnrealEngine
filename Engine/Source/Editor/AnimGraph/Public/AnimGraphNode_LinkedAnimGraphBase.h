// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "AnimGraphNode_CustomProperty.h"
#include "EdGraphSchema_K2_Actions.h"
#include "IClassVariableCreator.h"
#include "K2Node_EventNodeInterface.h"

#include "AnimGraphNode_LinkedAnimGraphBase.generated.h"

class FCompilerResultsLog;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SToolTip;
struct FAnimNode_LinkedAnimGraph;

UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_LinkedAnimGraphBase : public UAnimGraphNode_CustomProperty, public IK2Node_EventNodeInterface
{
	GENERATED_BODY()

public:
	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	
	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual void CreateOutputPins() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End UEdGraphNode Interface.

	// UAnimGraphNode_Base interface
	virtual FPoseLinkMappingRecord GetLinkIDLocation(const UScriptStruct* NodeType, UEdGraphPin* SourcePin) override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual bool ShouldShowAttributesOnPins() const override { return false; }
	virtual void OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;

	// UAnimGraphNode_CustomProperty interface
	virtual bool IsStructuralProperty(FProperty* InProperty) const override;
	virtual bool NeedsToSpecifyValidTargetClass() const override { return false; }

	// Node accessor
	virtual FAnimNode_LinkedAnimGraph* GetLinkedAnimGraphNode() PURE_VIRTUAL(UAnimGraphNode_LinkedAnimGraphBase::GetLinkedAnimGraphNode, return nullptr;);
	virtual const FAnimNode_LinkedAnimGraph* GetLinkedAnimGraphNode() const PURE_VIRTUAL(UAnimGraphNode_LinkedAnimGraphBase::GetLinkedAnimGraphNode, return nullptr;);

	// IK2Node_EventNodeInterface interface
	virtual TSharedPtr<FEdGraphSchemaAction> GetEventNodeAction(const FText& ActionCategory) override;

protected:
	friend class UAnimBlueprintExtension_LinkedAnimGraph;

	// Called pre-compilation to allocate pose links
	void AllocatePoseLinks();

	// Finds out whether there is a loop in the graph formed by linked instances from this node
	bool HasInstanceLoop();
	
	/** Generates widgets for exposing/hiding Pins for this node using the provided detail builder */
	void GenerateExposedPinsDetails(IDetailLayoutBuilder &DetailBuilder);

	// Finds out whether there is a loop in the graph formed by linked instances from CurrNode, used by HasInstanceLoop. VisitedNodes and NodeStack are required
	// to track the graph links
	// VisitedNodes - Node we have searched the links of, so we don't do it twice
	// NodeStack - The currently considered chain of nodes. If a loop is detected this will contain the chain that causes the loop
	static bool HasInstanceLoop_Recursive(UAnimGraphNode_LinkedAnimGraphBase* CurrNode, TArray<FGuid>& VisitedNodes, TArray<FGuid>& NodeStack);

	// ----- UI CALLBACKS ----- //

	// Gets path to the currently selected instance class' blueprint
	virtual FString GetCurrentInstanceBlueprintPath() const;
	// Filter callback for blueprints (only accept matching skeletons/interfaces)
	virtual bool OnShouldFilterInstanceBlueprint(const FAssetData& AssetData) const;
	// Instance blueprint was changed by user
	void OnSetInstanceBlueprint(const FAssetData& AssetData, IDetailLayoutBuilder* InDetailBuilder);
	// ----- END UI CALLBACKS ----- //

	virtual FLinearColor GetDefaultNodeTitleColor() const;

	// Handler for when the function reference gets re-resolved on node reconstruction
	virtual void HandleFunctionReferenceChanged(FName InNewName) {}
	
	// Helper func for generating pins
	void IterateFunctionParameters(UFunction* InFunction, TFunctionRef<void(const FName&, const FEdGraphPinType&)> InFunc) const;
	
	// Reference to the stub function that this node uses
	UPROPERTY()
	FMemberReference FunctionReference;
	
	// Skeleton name used for filtering unloaded classes 
	FString SkeletonName;

	// Template flag used for filtering unloaded classes
	bool bIsTemplateAnimBlueprint = false;

	// Interface flag used for filtering unloaded classes
	bool bIsInterfaceBlueprint = false;
};

UE_DEPRECATED(4.24, "UAnimGraphNode_SubInstanceBase has been renamed to UAnimGraphNode_LinkedAnimGraphBase")
typedef UAnimGraphNode_LinkedAnimGraphBase UAnimGraphNode_SubInstanceBase;
