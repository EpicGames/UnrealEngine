// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "Animation/AnimNode_LinkedAnimLayer.h"

#include "AnimGraphNode_LinkedAnimLayer.generated.h"

class FCompilerResultsLog;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SToolTip;

UCLASS(MinimalAPI)
class UAnimGraphNode_LinkedAnimLayer : public UAnimGraphNode_LinkedAnimGraphBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_LinkedAnimLayer Node;

	/** Guid of the named layer graph we refer to */
	UPROPERTY()
	FGuid InterfaceGuid;

	// Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;

	//~ Begin UEdGraphNode Interface.
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual void JumpToDefinition() const override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput /*= NULL*/) const override;
	virtual void ReconstructNode() override;
	//~ End UEdGraphNode Interface.

	// UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	
	// UAnimGraphNode_Base interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	// Optionally updates layer GUID if it is invalid
	void UpdateGuidForLayer();

	// Sets the name of the layer we refer to
	void SetLayerName(FName InName);

	// Gets the name of the layer we refer to
	FName GetLayerName() const;
protected:

	void GetLinkTarget(UObject* &OutTargetGraph, UBlueprint* &OutTargetBlueprint) const;

	// ----- UI CALLBACKS ----- //
	// Handlers for layer combo
	void GetLayerNames(TArray<TSharedPtr<FString>>& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems);
	FString GetLayerNameString() const;
	void OnLayerChanged(IDetailLayoutBuilder* DetailBuilder);
	bool HasAvailableLayers() const;
	bool HasValidNonSelfLayer() const;
	void HandleSetObjectBeingDebugged(UObject* InDebugObj);
	void HandleInstanceChanged();
	// ----- END UI CALLBACKS ----- //

	// UAnimGraphNode_Base interface
	virtual FProperty* GetPinProperty(FName InPinName) const override;
	virtual void CreateCustomPins(TArray<UEdGraphPin*>* OldPins) override;

	// Begin UAnimGraphNode_CustomProperty
	virtual FAnimNode_CustomProperty* GetCustomPropertyNode() override { return &Node;  }
	virtual const FAnimNode_CustomProperty* GetCustomPropertyNode() const override { return &Node; }
	virtual UClass* GetTargetSkeletonClass() const override;

	// Begin UAnimGraphNode_LinkedAnimGraphBase
	virtual FAnimNode_LinkedAnimGraph* GetLinkedAnimGraphNode() override;
	virtual const FAnimNode_LinkedAnimGraph* GetLinkedAnimGraphNode() const override;
	virtual bool OnShouldFilterInstanceBlueprint(const FAssetData& AssetData) const override;
	virtual FString GetCurrentInstanceBlueprintPath() const override;
	virtual bool IsStructuralProperty(FProperty* InProperty) const override;
	virtual FLinearColor GetDefaultNodeTitleColor() const override;
	virtual void HandleFunctionReferenceChanged(FName InNewName) override;
	
	friend class FAnimationLayerDragDropAction;
	
	// Helper function to get the interface currently in use by the selected layer
	TSubclassOf<UInterface> GetInterfaceForLayer() const;

	// Helper function to get the interface graph GUID currently in use by the selected layer
	FGuid GetGuidForLayer() const;

	// Get the preview node, if any, when instanced in an animation blueprint and debugged
	FAnimNode_LinkedAnimLayer* GetPreviewNode() const;

	// Helper function to setup a newly spawned node
	void SetupFromLayerId(FName InLayerId);
	
	// Used during compilation to check if the blueprint structure causes any circular references or nested linked layer nodes
	void ValidateCircularRefAndNesting(const UEdGraph* CurrentGraph, const TArray<UEdGraph*>& AllGraphs, TArray<const UEdGraph*> GraphStack, bool bWithinLinkedLayerGraph, FCompilerResultsLog& MessageLog);

	// Handle used to hook into object being debugged changing
	FDelegateHandle SetObjectBeingDebuggedHandle;
};

UE_DEPRECATED(4.24, "UAnimGraphNode_Layer has been renamed to UAnimGraphNode_LinkedAnimLayer")
typedef UAnimGraphNode_LinkedAnimLayer UAnimGraphNode_Layer;