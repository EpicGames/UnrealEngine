// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "Engine/MemberReference.h"
#include "IClassVariableCreator.h"

#include "AnimGraphNode_LinkedInputPose.generated.h"

#define UE_API ANIMGRAPH_API

class SEditableTextBox;

/** Required info for reconstructing a manually specified pin */
USTRUCT()
struct FAnimBlueprintFunctionPinInfo
{
	GENERATED_BODY()

	FAnimBlueprintFunctionPinInfo()
		: Name(NAME_None)
	{
		Type.ResetToDefaults();
	}

	FAnimBlueprintFunctionPinInfo(const FName& InName, const FEdGraphPinType& InType)
		: Name(InName)
		, Type(InType)
	{
	}

	/** The name of this parameter */
	UPROPERTY(EditAnywhere, Category = "Inputs")
	FName Name;

	/** The type of this parameter */
	UPROPERTY(EditAnywhere, Category = "Inputs")
	FEdGraphPinType Type;
};

UCLASS(MinimalAPI)
class UAnimGraphNode_LinkedInputPose : public UAnimGraphNode_Base, public IClassVariableCreator
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Inputs")
	FAnimNode_LinkedInputPose Node;

	UPROPERTY(EditAnywhere, Category = "Inputs")
	TArray<FAnimBlueprintFunctionPinInfo> Inputs;

	/** Reference to the stub function we use to build our parameters */
	UPROPERTY()
	FMemberReference FunctionReference;

	/** The index of the input pose, used alongside FunctionReference to build parameters */
	UPROPERTY()
	int32 InputPoseIndex;

	UE_API UAnimGraphNode_LinkedInputPose();

	/** UObject interface */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** IClassVariableCreator interface */
	UE_API virtual void CreateClassVariablesFromBlueprint(IAnimBlueprintVariableCreationContext& InCreationContext) override;

	/** UEdGraphNode interface */
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual bool CanDuplicateNode() const override;
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;
	UE_API virtual bool HasExternalDependencies(TArray<UStruct*>* OptionalOutput) const override;

	/** UK2Node interface */
	UE_API virtual void ExpandNode(class FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph) override;

	/** UAnimGraphNode_Base interface */
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual bool ShouldShowAttributesOnPins() const override { return false; }
	UE_API virtual void OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;

	/** Make a name widget for this linked input pose node */
	UE_API TSharedRef<SWidget> MakeNameWidget(IDetailLayoutBuilder& DetailBuilder);

	/** @return whether this node is editable. Non-editable nodes are implemented from function interfaces */
	bool IsEditable() const { return CanUserDeleteNode(); }

	/** @return the number of parameter inputs this node provides. This is either determined via the Inputs array or via the FunctionReference. */
	UE_API int32 GetNumInputs() const;

	/** Promotes the node from being a part of an interface override to a full function that allows for parameter and result pin additions */
	UE_API void PromoteFromInterfaceOverride();

	/** Conform input pose name according to function */
	UE_API void ConformInputPoseName();

	/** Validate pose index against the function reference (used to determine whether we should exist or not) */
	UE_API bool ValidateAgainstFunctionReference() const;

	/** Helper function for iterating stub function parameters */
	UE_API void IterateFunctionParameters(TFunctionRef<void(const FName&, const FEdGraphPinType&)> InFunc) const;

private:
	friend class FAnimGraphDetails;
	friend class UAnimationGraph;
	
	// Helper function for common code in AllocateDefaultPins and ReallocatePinsDuringReconstruction
	UE_API void AllocatePinsInternal();

	/** Create pins from the user-defined Inputs array */
	UE_API void CreateUserDefinedPins();

	/** Create pins from the stub function FunctionReference */
	UE_API void CreatePinsFromStubFunction(const UFunction* Function);

	friend class UAnimBlueprintExtension_LinkedInputPose;
	// Called pre-compilation to evaluate is this input pose is used
	UE_API void AnalyzeLinks(TArrayView<UAnimGraphNode_Base*> InAnimNodes);

	/** Reconstruct any layer nodes in this BP post-edit */
	static UE_API void ReconstructLayerNodes(UBlueprint* InBlueprint);
	
private:
	/** UI helper functions */
	UE_API void HandleInputPoseArrayChanged();
	UE_API void HandleInputPinArrayChanged();
};

UE_DEPRECATED(4.24, "UAnimGraphNode_SubInput has been renamed to UAnimGraphNode_LinkedInputPose")
typedef UAnimGraphNode_LinkedInputPose UAnimGraphNode_SubInput;

#undef UE_API
