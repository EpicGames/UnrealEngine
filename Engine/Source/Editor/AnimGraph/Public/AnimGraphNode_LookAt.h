// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_LookAt.h"
#include "AnimGraphNode_LookAt.generated.h"

class FPrimitiveDrawInterface;
class USkeletalMeshComponent;

UCLASS(MinimalAPI, meta=(Keywords = "Look At, Follow, Trace, Track"))
class UAnimGraphNode_LookAt : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_LookAt Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FEditorModeID GetEditorMode() const override;
	virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) override;
	virtual void CopyPinDefaultsToNodeData(UEdGraphPin* InPin) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_SkeletalControlBase interface
	ANIMGRAPH_API virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const override;
	ANIMGRAPH_API virtual void GetOnScreenDebugInfo(TArray<FText>& DebugInfo, FAnimNode_Base* RuntimeAnimNode, USkeletalMeshComponent* PreviewSkelMeshComp) const override;
	// End of UAnimGraphNode_SkeletalControlBase interface

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface
private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
