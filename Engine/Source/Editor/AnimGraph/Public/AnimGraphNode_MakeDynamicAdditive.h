// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_MakeDynamicAdditive.h"
#include "AnimGraphNode_MakeDynamicAdditive.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_MakeDynamicAdditive : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_MakeDynamicAdditive Node;

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual FString GetNodeCategory() const override;
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	// End of UAnimGraphNode_Base interface
};
