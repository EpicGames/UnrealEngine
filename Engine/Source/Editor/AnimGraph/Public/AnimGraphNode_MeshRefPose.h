// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_RefPose.h"
#include "AnimGraphNode_MeshRefPose.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_MeshRefPose : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_MeshSpaceRefPose Node;

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual FString GetNodeCategory() const override;
	virtual void CreateOutputPins() override;
	// End of UAnimGraphNode_Base interface
};
