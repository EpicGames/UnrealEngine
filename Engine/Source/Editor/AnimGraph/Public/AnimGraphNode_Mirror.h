// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_Mirror.h"
#include "AnimGraphNode_Mirror.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_Mirror : public UAnimGraphNode_Base
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_Mirror Node;

	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	virtual void PostPlacedNewNode() override;
	virtual void PostLoad() override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreloadRequiredAssets() override;
	void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;

private:
	static FText GetTitleGivenAssetInfo(const FText& AssetName);
	void OnMirrorDataTableChanged(); 
	void BindMirrorDataTableChangedDelegate(); 
	bool HasMirrorDataTableForBlueprints(const TArray<UBlueprint*>& Blueprints) const;
	/** Used for filtering in the Blueprint context menu when the sequence asset this node uses is unloaded */
	FString UnloadedSkeletonName;
};
