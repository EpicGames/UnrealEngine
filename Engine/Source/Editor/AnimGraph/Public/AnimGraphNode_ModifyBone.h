// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneControllers/AnimNode_ModifyBone.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "AnimGraphNode_ModifyBone.generated.h"

#define UE_API ANIMGRAPH_API

class FCompilerResultsLog;

UCLASS(MinimalAPI, meta=(Keywords = "Modify Transform"))
class UAnimGraphNode_ModifyBone : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_ModifyBone Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_Base interface
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	UE_API virtual FEditorModeID GetEditorMode() const override;
	UE_API virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) override;
	UE_API virtual void CopyPinDefaultsToNodeData(UEdGraphPin* InPin) override;
	// End of UAnimGraphNode_Base interface

	// UAnimGraphNode_SkeletalControlBase interface
	UE_API virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};

#undef UE_API
