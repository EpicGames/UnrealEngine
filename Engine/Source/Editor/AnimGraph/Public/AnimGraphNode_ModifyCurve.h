// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_ModifyCurve.h"
#include "AnimGraphNode_ModifyCurve.generated.h"

class FMenuBuilder;

/** Easy way to modify curve values on a pose */
UCLASS(MinimalAPI)
class UAnimGraphNode_ModifyCurve : public UAnimGraphNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_ModifyCurve Node;

public:
	UAnimGraphNode_ModifyCurve();

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	// End of UAnimGraphNode_Base interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface

private:
	/** Remove a curve pin with the given name */
	void RemoveCurvePin(FName CurveName);
	/** Add a curve pin with the given name */
	void AddCurvePin(FName CurveName);
	/** Create submenu with options for curves to add */
	void GetAddCurveMenuActions(FMenuBuilder& MenuBuilder) const;
	/** Create submenu with options for curves to remove */
	void GetRemoveCurveMenuActions(FMenuBuilder& MenuBuilder) const;
};
