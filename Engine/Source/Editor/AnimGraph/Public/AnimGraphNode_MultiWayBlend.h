// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_MultiWayBlend.h"
#include "AnimGraphNode_MultiWayBlend.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_MultiWayBlend : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_MultiWayBlend Node;

	// Adds a new pose pin
	//@TODO: Generalize this behavior (returning a list of actions/delegates maybe?)
	ANIMGRAPH_API virtual void AddPinToBlendNode();
	ANIMGRAPH_API virtual void RemovePinFromBlendNode(UEdGraphPin* Pin);
	ANIMGRAPH_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PostPlacedNewNode() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	virtual FString GetNodeCategory() const override;
	//~ End UAnimGraphNode_Base Interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface

	// Begin UObject interface
	virtual void PostEditUndo() override;
	// End of UObject interface

private:
	int32 RemovedPinArrayIndex;

	// removes removed pins and adjusts array indices of remained pins
	void RemovePinsFromOldPins(TArray<UEdGraphPin*>& OldPins, int32 RemovedArrayIndex);
};
