// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_ObserveBone.h"
#include "AnimGraphNode_ObserveBone.generated.h"

#define UE_API ANIMGRAPH_API

class FCompilerResultsLog;
class IDetailLayoutBuilder;
class SGraphNode;

// This allows you to observe the state of a bone at a particular point in the graph, showing it in any space and optionally relative to the reference pose
UCLASS(MinimalAPI)
class UAnimGraphNode_ObserveBone : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_ObserveBone Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_Base interface
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	UE_API virtual FEditorModeID GetEditorMode() const override;
	// End of UAnimGraphNode_Base interface

	// UAnimGraphNode_SkeletalControlBase interface
	UE_API virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};

#undef UE_API
