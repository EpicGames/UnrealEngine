// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_PoseHandler.h"
#include "AnimNodes/AnimNode_PoseBlendNode.h"
#include "AnimGraphNode_PoseBlendNode.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_PoseBlendNode : public UAnimGraphNode_PoseHandler
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_PoseBlendNode Node;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode

	// UAnimGraphNode_Base interface
	// Interface to support transition getter
	virtual bool DoesSupportTimeForTransitionGetter() const override;
	virtual void GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	// End of UAnimGraphNode_Base

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface

private:
	FText GetNodeTitleForPoseAsset(ENodeTitleType::Type TitleType, UPoseAsset* InPoseAsset) const;

private:
	static FText GetTitleGivenAssetInfo(const FText& AssetName);

	/** Used for filtering in the Blueprint context menu when the pose asset this node uses is unloaded */
	FString UnloadedSkeletonName;

protected:
	virtual FAnimNode_PoseHandler* GetPoseHandlerNode() override { return &Node; }
	virtual const FAnimNode_PoseHandler* GetPoseHandlerNode() const override { return &Node; }
};
