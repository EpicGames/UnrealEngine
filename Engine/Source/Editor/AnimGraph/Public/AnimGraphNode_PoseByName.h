// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimNodes/AnimNode_PoseByName.h"
#include "AnimGraphNode_PoseByName.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_PoseByName : public UAnimGraphNode_AssetPlayerBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_PoseByName Node;

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode

	// UAnimGraphNode_Base interface
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual void PreloadRequiredAssets() override;
	// Interface to support transition getter
	virtual bool DoesSupportTimeForTransitionGetter() const override;
	virtual UAnimationAsset* GetAnimationAsset() const override;
	virtual void GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap) override;
	virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	// End of UAnimGraphNode_Base

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	// UAnimGraphNode_AssetPlayerBase interface
	virtual void SetAnimationAsset(UAnimationAsset* Asset) override;
	// End of UAnimGraphNode_AssetPlayerBase interface

private:
	FText GetNodeTitleForPoseAsset(ENodeTitleType::Type TitleType, UPoseAsset* InPoseAsset) const;

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};
