// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimNodes/AnimNode_PoseDriver.h"
#include "AnimGraphNode_PoseHandler.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_PoseDriver.generated.h"

#define UE_API ANIMGRAPH_API

class FCompilerResultsLog;

UCLASS(MinimalAPI, BlueprintType)
class UAnimGraphNode_PoseDriver : public UAnimGraphNode_PoseHandler
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FAnimNode_PoseDriver Node;

	/** Length of axis in world units used for debug drawing */
	UPROPERTY(EditAnywhere, Category = Debugging, meta = (DefaultValue = "20.0", UIMin = "1.0", UIMax = "100.0"))
	float AxisLength;

	/** Number of subdivisions / lines used when debug drawing a cone */
	UPROPERTY(EditAnywhere, Category = Debugging, meta = (DefaultValue = "32", UIMin = "6", UIMax = "128"))
	int32 ConeSubdivision;

	/** If checked the cones will be drawn in 3d for debugging */
	UPROPERTY(EditAnywhere, Category = Debugging, meta = (DefaultValue = "True"))
	bool bDrawDebugCones;

	/** Used to indicate selected target to edit mode drawing */
	int32 SelectedTargetIndex;
	/** Delegate to call when selection changes */
	FSimpleMulticastDelegate SelectedTargetChangeDelegate;

public:

	/** Get the current preview node instance */
	UE_API FAnimNode_PoseDriver* GetPreviewPoseDriverNode() const;

	/** Util to replace current contents of PoseTargets with info from assigned PoseAsset */
	UFUNCTION(BlueprintCallable, Category=PoseDriver)
	UE_API void CopyTargetsFromPoseAsset();

	/** Sets the pose-driver its source bones by name */
	UFUNCTION(BlueprintCallable, Category=PoseDriver)
	UE_API void SetSourceBones(const TArray<FName>& BoneNames);

	/** Returns the pose-driver its source bones by name */
	UFUNCTION(BlueprintPure, Category=PoseDriver)
	UE_API void GetSourceBoneNames(TArray<FName>& BoneNames);
 
	/** Set the pose-driver its driven bones by name */
	UFUNCTION(BlueprintCallable, Category=PoseDriver)
	UE_API void SetDrivingBones(const TArray<FName>& BoneNames);

	/** Returns the pose-driver its driven bones by name */
	UFUNCTION(BlueprintPure, Category=PoseDriver)
	UE_API void GetDrivingBoneNames(TArray<FName>& BoneNames);

	/**  */
	UFUNCTION(BlueprintCallable, Category=PoseDriver)
	UE_API void SetRBFParameters(FRBFParams Parameters);

	/**  */
	UFUNCTION(BlueprintPure, Category=PoseDriver)
	UE_API FRBFParams& GetRBFParameters();
	
	/**  */
	UFUNCTION(BlueprintCallable, Category=PoseDriver)
	UE_API void SetPoseDriverSource(EPoseDriverSource DriverSource);

	/**  */
	UFUNCTION(BlueprintPure, Category=PoseDriver)
	UE_API EPoseDriverSource& GetPoseDriverSource();
	
	/**  */
	UFUNCTION(BlueprintCallable, Category=PoseDriver)
	UE_API void SetPoseDriverOutput(EPoseDriverOutput DriverOutput);

	/**  */
	UFUNCTION(BlueprintPure, Category=PoseDriver)
	UE_API EPoseDriverOutput& GetPoseDriverOutput();
	
	/** Automatically modify TargetScale for each PoseTarget, based on distance to nearest neighbor */
	UE_API void AutoSetTargetScales(float& OutMaxDistance);

	/** Adds a new target, reallocating transforms array appropriately */
	UE_API void AddNewTarget();

	/** Reallocates transforms arrays as necessary to accommodate source bones */
	UE_API void ReserveTargetTransforms();

	/** Return the color for a given weight. Used for Details and EditMode */
	UE_API FLinearColor GetColorFromWeight(float InWeight);

	/** Used to refer back to preview instance in anim tools */
	UPROPERTY(Transient)
	TObjectPtr<USkeletalMeshComponent> LastPreviewComponent;

	// Begin UObject Interface.
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End UObject Interface.

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeBodyTintColor() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	UE_API virtual FEditorModeID GetEditorMode() const override;
	UE_API virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	UE_API virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) override;
	// End of UAnimGraphNode_Base interface

protected:
	// UAnimGraphNode_PoseHandler interface
	virtual bool IsPoseAssetRequired() override { return false; }
	virtual FAnimNode_PoseHandler* GetPoseHandlerNode() override { return &Node; }
	virtual const FAnimNode_PoseHandler* GetPoseHandlerNode() const override { return &Node; }
	// End of UAnimGraphNode_PoseHandler interface

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};

#undef UE_API
