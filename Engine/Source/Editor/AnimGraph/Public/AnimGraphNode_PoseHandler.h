// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimGraphNode_PoseHandler.generated.h"

struct FAnimNode_PoseHandler;

UCLASS(MinimalAPI, abstract)
class UAnimGraphNode_PoseHandler : public UAnimGraphNode_AssetPlayerBase
{
	GENERATED_UCLASS_BODY()

	// UAnimGraphNode_Base interface
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual void PreloadRequiredAssets() override;
	virtual UAnimationAsset* GetAnimationAsset() const override;
	virtual TSubclassOf<UAnimationAsset> GetAnimationAssetClass() const override;
	virtual void OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const override;
	// End of UAnimGraphNode_Base

	// UAnimGraphNode_AssetPlayerBase interface
	virtual void SetAnimationAsset(UAnimationAsset* Asset) override;
	// End of UAnimGraphNode_AssetPlayerBase interface

	// UAnimGraphNode_PoseHandler interface
	virtual bool IsPoseAssetRequired() { return true; }
	virtual FAnimNode_PoseHandler* GetPoseHandlerNode() PURE_VIRTUAL(UAnimGraphNode_PoseHandler::GetPoseHandlerNode, return nullptr;);
	virtual const FAnimNode_PoseHandler* GetPoseHandlerNode() const PURE_VIRTUAL(UAnimGraphNode_PoseHandler::GetPoseHandlerNode, return nullptr;);
	// End of UAnimGraphNode_PoseHandler interface
};
