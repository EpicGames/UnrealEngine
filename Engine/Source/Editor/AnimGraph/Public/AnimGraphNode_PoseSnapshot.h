// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_RefPoseBase.h"
#include "AnimNodes/AnimNode_PoseSnapshot.h"
#include "AnimGraphNode_PoseSnapshot.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_PoseSnapshot : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_PoseSnapshot Node;

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface
};
