// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_RandomPlayer.h"
#include "AnimGraphNode_RandomPlayer.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_RandomPlayer : public UAnimGraphNode_Base
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_RandomPlayer Node;

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual void PreloadRequiredAssets() override;
	// End of UAnimGraphNode_Base interface
};
