// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_RefPose.h"
#include "AnimGraphNode_RefPoseBase.generated.h"

UCLASS(MinimalAPI, abstract)
class UAnimGraphNode_RefPoseBase : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_RefPose Node;

	// UEdGraphNode interface
	virtual FString GetNodeCategory() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	// End of UEdGraphNode interface
};
