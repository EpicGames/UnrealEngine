// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_RemoveCurve.h"
#include "AnimGraphNode_RemoveCurve.generated.h"

///** Removes animation curves on a pose */
UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_RemoveCurve : public UAnimGraphNode_Base
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_RemoveCurve Node;

public:	

	UAnimGraphNode_RemoveCurve();

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	
};