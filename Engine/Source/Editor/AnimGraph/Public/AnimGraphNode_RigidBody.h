// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_RigidBody.h"
#include "Engine/PoseWatch.h"
#include "AnimGraphNode_RigidBody.generated.h"

UCLASS(MinimalAPI, meta=(Keywords = "Simulate Rigid Body Physics Ragdoll"))
class UAnimGraphNode_RigidBody : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_RigidBody Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetNodeCategory() const override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base
	virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp, const bool bIsSelected, const bool bIsPoseWatchEnabled) const override;
	virtual void OnPoseWatchChanged(const bool IsPoseWatchActive, TObjectPtr<UPoseWatch> InPoseWatch, FEditorModeTools& InModeTools, FAnimNode_Base* InRuntimeNode) override;

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	void ToggleBodyVisibility();
	void ToggleConstraintVisibility();

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

	bool AreAnyBodiesHidden() const;
	bool AreAnyConstraintsHidden() const;
	
	static EVisibility ShouldShowPoseScalePropagationSettings();

private:
	TWeakObjectPtr<UPoseWatchElement> PoseWatchElementBodies;
	TWeakObjectPtr<UPoseWatchElement> PoseWatchElementConstraints;
};
