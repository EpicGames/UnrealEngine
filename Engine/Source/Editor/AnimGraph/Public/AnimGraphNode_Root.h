// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_Root.h"
#include "AnimGraphNode_Root.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_Root : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_Root Node;

	//~ Begin UObject interface
	ANIMGRAPH_API virtual void Serialize(FArchive& Ar) override;
	//~ End of UObject interface

	//~ Begin UEdGraphNode Interface.
	ANIMGRAPH_API virtual FLinearColor GetNodeTitleColor() const override;
	ANIMGRAPH_API virtual FText GetTooltipText() const override;
	ANIMGRAPH_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	ANIMGRAPH_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsNodeRootSet() const override { return true; }
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	ANIMGRAPH_API virtual bool IsPoseWatchable() const override;
	ANIMGRAPH_API virtual bool IsSinkNode() const override;
	ANIMGRAPH_API virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	ANIMGRAPH_API virtual void OnCopyTermDefaultsToDefaultObject(IAnimBlueprintCopyTermDefaultsContext& InCompilationContext, IAnimBlueprintNodeCopyTermDefaultsContext& InPerNodeContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;

	// Get the link to the documentation
	ANIMGRAPH_API virtual FString GetDocumentationLink() const override;

	//~ End UAnimGraphNode_Base Interface
};
