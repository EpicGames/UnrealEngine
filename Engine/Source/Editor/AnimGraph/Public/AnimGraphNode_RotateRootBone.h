// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimNodes/AnimNode_RotateRootBone.h"
#include "AnimGraphNode_RotateRootBone.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_RotateRootBone : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_RotateRootBone Node;

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual FText GetMenuCategory() const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	//~ End UAnimGraphNode_Base Interface
};
