// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BlueprintNodeSignature.h"
#include "AnimGraphNode_BlendSpaceBase.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimNodes/AnimNode_RotationOffsetBlendSpace.h"
#include "AnimGraphNode_RotationOffsetBlendSpace.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_RotationOffsetBlendSpace: public UAnimGraphNode_BlendSpaceBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_RotationOffsetBlendSpace Node;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual UAnimationAsset* GetAnimationAsset() const override;
	virtual void OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const override;
	virtual void GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap) override;
	virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of UAnimGraphNode_Base interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	// End of UK2Node interface

	// UAnimGraphNode_AssetPlayerBase interface
	virtual void SetAnimationAsset(UAnimationAsset* Asset) override;
	// End of UAnimGraphNode_AssetPlayerBase interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
