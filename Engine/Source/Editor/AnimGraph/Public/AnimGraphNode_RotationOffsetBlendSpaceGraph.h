// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "AnimNodes/AnimNode_RotationOffsetBlendSpaceGraph.h"
#include "AnimGraphNode_RotationOffsetBlendSpaceGraph.generated.h"

#define UE_API ANIMGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FCompilerResultsLog;

UCLASS(MinimalAPI)
class UAnimGraphNode_RotationOffsetBlendSpaceGraph : public UAnimGraphNode_BlendSpaceGraphBase
{
	GENERATED_BODY()

private:
	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_RotationOffsetBlendSpaceGraph Node;

	// UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	
	// UK2Node interface
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;

	// UAnimGraphNode_Base interface
	UE_API virtual void BakeDataDuringCompilation(FCompilerResultsLog& MessageLog) override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

#undef UE_API
