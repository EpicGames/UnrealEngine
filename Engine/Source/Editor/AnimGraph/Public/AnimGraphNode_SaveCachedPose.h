// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Animation/AnimNode_SaveCachedPose.h"
#include "AnimGraphNode_SaveCachedPose.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;

UCLASS(MinimalAPI)
class UAnimGraphNode_SaveCachedPose : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_SaveCachedPose Node;

	UPROPERTY(EditAnywhere, Category=CachedPose)
	FString CacheName;

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual bool IsNodeRootSet() const override { return true; }
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostPasteNode() override;
	// End of UEdGraphNode interface

	// UK2Node interface.
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	// UAnimGraphNode_Base interface
	virtual FString GetNodeCategory() const override;
	virtual bool IsSinkNode() const override { return true; }
	virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;
	// End of UAnimGraphNode_Base interface

private:
	
	bool MakeUniqueCacheName();
	
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};
