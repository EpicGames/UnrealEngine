// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "BoneControllers/AnimNode_ScaleChainLength.h"
#include "AnimGraphNode_ScaleChainLength.generated.h"

class USkeletalMeshComponent;

UCLASS(MinimalAPI)
class UAnimGraphNode_ScaleChainLength : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_ScaleChainLength Node;

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	virtual FString GetNodeCategory() const override;
	//~ End UAnimGraphNode_Base Interface
};
