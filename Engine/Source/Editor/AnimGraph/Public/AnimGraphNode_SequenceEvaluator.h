// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimNodes/AnimNode_SequenceEvaluator.h"
#include "AnimGraphNode_SequenceEvaluator.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_SequenceEvaluator : public UAnimGraphNode_AssetPlayerBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_SequenceEvaluator Node;

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode

	// UAnimGraphNode_Base interface
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual void PreloadRequiredAssets() override;
	virtual void BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog) override;
	// Interface to support transition getter
	virtual bool DoesSupportTimeForTransitionGetter() const override;
	virtual UAnimationAsset* GetAnimationAsset() const override;
	virtual const TCHAR* GetTimePropertyName() const override;
	virtual UScriptStruct* GetTimePropertyStruct() const override;
	virtual void GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap) override;
	virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	virtual void OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UAnimGraphNode_Base

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	// UAnimGraphNode_AssetPlayerBase interface
	virtual void SetAnimationAsset(UAnimationAsset* Asset) override;
	virtual void CopySettingsFromAnimationAsset(UAnimationAsset* Asset) override;
	// End of UAnimGraphNode_AssetPlayerBase interface
	
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};
