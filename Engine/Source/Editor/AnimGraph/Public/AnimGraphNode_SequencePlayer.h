// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimGraphNode_SequencePlayer.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_SequencePlayer : public UAnimGraphNode_AssetPlayerBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_SequencePlayer Node;

	// UObject interface
	void Serialize(FArchive& Ar) override;

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual void PreloadRequiredAssets() override;		
	virtual void BakeDataDuringCompilation(class FCompilerResultsLog& MessageLog) override;
	virtual bool DoesSupportTimeForTransitionGetter() const override;
	virtual UAnimationAsset* GetAnimationAsset() const override;
	virtual const TCHAR* GetTimePropertyName() const override;
	virtual UScriptStruct* GetTimePropertyStruct() const override;
	virtual void GetAllAnimationSequencesReferred(TArray<UAnimationAsset*>& AnimationAssets) const override;
	virtual void ReplaceReferredAnimations(const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	virtual void OnOverrideAssets(IAnimBlueprintNodeOverrideAssetsContext& InContext) const override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual TSubclassOf<UAnimationAsset> GetAnimationAssetClass() const { return UAnimSequenceBase::StaticClass(); }
	// End of UAnimGraphNode_Base interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	// End of UK2Node interface

	// UAnimGraphNode_AssetPlayerBase interface
	virtual void SetAnimationAsset(UAnimationAsset* Asset) override;
	virtual void CopySettingsFromAnimationAsset(UAnimationAsset* Asset) override;
	// End of UAnimGraphNode_AssetPlayerBase interface
};
