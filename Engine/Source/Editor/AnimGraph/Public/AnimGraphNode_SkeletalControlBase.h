// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BonePose.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_SkeletalControlBase.generated.h"

#define UE_API ANIMGRAPH_API

class FCompilerResultsLog;
class USkeletalMeshComponent;
struct FAnimNode_SkeletalControlBase;
struct HActor;

/**
 * This is the base class for the 'source version' of all skeletal control animation graph nodes
 * (nodes that manipulate the pose rather than playing animations to create a pose or blending between poses)
 *
 * Concrete subclasses should contain a member struct derived from FAnimNode_SkeletalControlBase
 */
UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_SkeletalControlBase : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

public:
	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	// UEdGraphNode interface
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool ShowVisualWarning() const override;
	UE_API virtual FText GetVisualWarningTooltipText() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	UE_API virtual FString GetNodeCategory() const override;
	UE_API virtual void CreateOutputPins() override;
	UE_API virtual void ValidateAnimNodePostCompile(FCompilerResultsLog& MessageLog, UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex) override;
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of UAnimGraphNode_Base interface

	/**
	 * methods related to widget control
	 */
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual FVector GetWidgetLocation(const USkeletalMeshComponent* SkelComp, struct FAnimNode_SkeletalControlBase* AnimNode)
	{
		return FVector::ZeroVector;
	}
	
	UE_DEPRECATED(4.14, "This function is deprecated and is no longer used, please use CopyNodeDataToPreviewNode")
	virtual void CopyNodeDataTo(FAnimNode_Base* OutAnimNode){ CopyNodeDataToPreviewNode(OutAnimNode); }

	UE_DEPRECATED(4.14, "This function is deprecated and is no longer used")
	virtual void CopyNodeDataFrom(const FAnimNode_Base* NewAnimNode){}

	/** Are we currently showing this pin */
	UE_API bool IsPinShown(const FName PinName) const;

	// Returns the coordinate system that should be used for this bone
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API virtual int32 GetWidgetCoordinateSystem(const USkeletalMeshComponent* SkelComp);

	// return current widget mode this anim graph node supports
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API virtual int32 GetWidgetMode(const USkeletalMeshComponent* SkelComp);
	// called when the user changed widget mode by pressing "Space" key
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API virtual int32 ChangeToNextWidgetMode(const USkeletalMeshComponent* SkelComp, int32 CurWidgetMode);
	// called when the user set widget mode directly, returns true if InWidgetMode is available
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual bool SetWidgetMode(const USkeletalMeshComponent* SkelComp, int32 InWidgetMode) { return false; }

	// 
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API virtual FName FindSelectedBone();

	// if anim graph node needs other actors to select other bones, move actor's positions when this is called
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual void MoveSelectActorLocation(const USkeletalMeshComponent* SkelComp, FAnimNode_SkeletalControlBase* InAnimNode) {}

	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual bool IsActorClicked(HActor* ActorHitProxy) { return false; }
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual void ProcessActorClick(HActor* ActorHitProxy) {}
	// if it has select-actors, should hide all actors when de-select is called  
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual void DeselectActor(USkeletalMeshComponent* SkelComp){}

	// called when the widget is dragged in translation mode
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual void DoTranslation(const USkeletalMeshComponent* SkelComp, FVector& Drag, FAnimNode_Base* InOutAnimNode) {}
	// called when the widget is dragged in rotation mode
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual void DoRotation(const USkeletalMeshComponent* SkelComp, FRotator& Rotation, FAnimNode_Base* InOutAnimNode) {}
	// called when the widget is dragged in scale mode
	UE_DEPRECATED(4.14, "This function is deprecated. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	virtual void DoScale(const USkeletalMeshComponent* SkelComp, FVector& Scale, FAnimNode_Base* InOutAnimNode) {}


public:
	// set literal value for FVector
	UE_API void SetDefaultValue(const FName InDefaultValueName, const FVector& InValue);
	// get literal value for vector
	UE_API void GetDefaultValue(const FName UpdateDefaultValueName, FVector& OutVec);

	void GetDefaultValue(const FName PropName, FRotator& OutValue)
	{
		FVector Value;
		GetDefaultValue(PropName, Value);
		OutValue.Pitch = Value.X;
		OutValue.Yaw = Value.Y;
		OutValue.Roll = Value.Z;
	}

	template<class ValueType>
	ValueType GetNodeValue(const FName PropName, const ValueType& CompileNodeValue)
	{
		if (IsPinShown(PropName))
		{
			ValueType Val;
			GetDefaultValue(PropName, Val);
			return Val;
		}
		return CompileNodeValue;
	}

	void SetDefaultValue(const FName PropName, const FRotator& InValue)
	{
		FVector VecValue(InValue.Pitch, InValue.Yaw, InValue.Roll);
		SetDefaultValue(PropName, VecValue);
	}

	template<class ValueType>
	void SetNodeValue(const FName PropName, ValueType& CompileNodeValue, const ValueType& InValue)
	{
		if (IsPinShown(PropName))
		{
			SetDefaultValue(PropName, InValue);
		}
		CompileNodeValue = InValue;
	}

protected:
	// Returns the short descriptive name of the controller
	UE_API virtual FText GetControllerDescription() const;

	/**
	* helper functions for bone control preview
	*/
	// local conversion function for drawing
	UE_DEPRECATED(4.14, "This function is moved to FAnimNodeEditMode. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API void ConvertToComponentSpaceTransform(const USkeletalMeshComponent* SkelComp, const FTransform & InTransform, FTransform & OutCSTransform, int32 BoneIndex, EBoneControlSpace Space) const;
	// convert drag vector in component space to bone space 
	UE_DEPRECATED(4.14, "This function is moved to FAnimNodeEditMode. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API FVector ConvertCSVectorToBoneSpace(const USkeletalMeshComponent* SkelComp, FVector& InCSVector, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space);
	// convert rotator in component space to bone space 
	UE_DEPRECATED(4.14, "This function is moved to FAnimNodeEditMode. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API FQuat ConvertCSRotationToBoneSpace(const USkeletalMeshComponent* SkelComp, FRotator& InCSRotator, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space);
	// convert widget location according to bone control space
	UE_DEPRECATED(4.14, "This function is moved to FAnimNodeEditMode. Use a IAnimNodeEditMode-derived class to implement a new editor mode and override GetEditorMode to return its ID")
	UE_API FVector ConvertWidgetLocation(const USkeletalMeshComponent* InSkelComp, FCSPose<FCompactHeapPose>& InMeshBases, const FName& BoneName, const FVector& InLocation, const EBoneControlSpace Space);

	virtual const FAnimNode_SkeletalControlBase* GetNode() const PURE_VIRTUAL(UAnimGraphNode_SkeletalControlBase::GetNode, return nullptr;);

	UE_API FAnimNode_SkeletalControlBase* GetDebuggedNode() const;
};

#undef UE_API
