// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_SplineIK.h"
#include "EdGraph/EdGraphNodeUtils.h" // for FNodeTitleTextTable
#include "AnimGraphNode_SplineIK.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_SplineIK : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_SplineIK Node;

public:

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;

	// UAnimGraphNode_Base interface
	virtual FEditorModeID GetEditorMode() const;

	// UAnimGraphNode_SkeletalControlBase interface
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
