// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_SpringBone.h"
#include "AnimGraphNode_SpringBone.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_SpringBone : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_SpringBone Node;

	//~ Begin UObject Interface
	virtual void PostLoad();
	//~ End UObject Interface

public:
	//~ Begin UEdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	//~ End UEdGraphNode Interface

protected:
	//~ Begin UAnimGraphNode_SkeletalControlBase Interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	//~ End UAnimGraphNode_SkeletalControlBase Interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
