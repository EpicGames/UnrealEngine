// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNode_StateMachine.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimGraphNode_StateMachine.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_StateMachine : public UAnimGraphNode_StateMachineBase
{
	GENERATED_UCLASS_BODY()

	// Runtime state machine representation (empty; populated during compilation)
	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_StateMachine Node;

	// UAnimGraphNode_StateMachineBase interface
	virtual FAnimNode_StateMachine& GetNode() override { return Node; }
	// End of UAnimGraphNode_StateMachineBase interface
};
