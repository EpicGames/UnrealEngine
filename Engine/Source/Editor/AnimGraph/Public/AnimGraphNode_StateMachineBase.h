// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Animation/AnimNode_StateMachine.h"
#include "AnimGraphNode_StateMachineBase.generated.h"

#define UE_API ANIMGRAPH_API

class INameValidatorInterface;

UCLASS(MinimalAPI, Abstract)
class UAnimGraphNode_StateMachineBase : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	// Editor state machine representation
	UPROPERTY()
	TObjectPtr<class UAnimationStateMachineGraph> EditorStateMachineGraph;

	// UEdGraphNode interface
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual void DestroyNode() override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	UE_API virtual FString GetDocumentationLink() const override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;
	UE_API virtual TArray<UEdGraph*> GetSubGraphs() const override;
	UE_API virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	UE_API virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	UE_API virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;
	// End of UAnimGraphNode_Base interface

	//  @return the name of this state machine
	UE_API FString GetStateMachineName();

	// Interface for derived classes to implement
	virtual FAnimNode_StateMachine& GetNode() PURE_VIRTUAL(UAnimGraphNode_StateMachineBase::GetNode, static FAnimNode_StateMachine Dummy; return Dummy;);
	// End of my interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedFullTitle;
};

#undef UE_API
