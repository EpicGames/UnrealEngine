// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_StateResult.h"
#include "AnimGraphNode_StateResult.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_StateResult : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_StateResult Node;

	// UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	//~ Begin UEdGraphNode Interface.
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsNodeRootSet() const override { return true; }
	//~ End UEdGraphNode Interface.

	//~ Begin UAnimGraphNode_Base Interface
	virtual bool IsSinkNode() const override;
	virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	virtual void GetBoundFunctionsInfo(TArray<TPair<FName, FName>>& InOutBindingsInfo) override;
	
	// UK2Node interface
	virtual bool ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const override;

	// Get the link to the documentation
	virtual FString GetDocumentationLink() const override;

	//~ End UAnimGraphNode_Base Interface

	/** Function called when the owning state is entered, meaning it becomes the state machine's current state. */
	UPROPERTY(EditAnywhere, Category = "Functions", meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnStateEntry"), DisplayName="On State Entry")
	FMemberReference StateEntryFunction;

	/** Function called when the owning state is fully blended in.
	 *
	 * Notes:
	 * - This is only called for the state machine's current state since its the most recent transition's target state.
	 * - This will not be called if the state is skipped. This can happen when the flag bSkipFirstUpdateTransition on the state machine node is set to true.
	 */
	UPROPERTY(EditAnywhere, Category = "Functions", meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnStateFullyBlendedIn"), DisplayName="On State Fully Blended In")
	FMemberReference StateFullyBlendedInFunction;

	/** Function called when the owning state is exited, meaning it stops being the state machine's current state. */
	UPROPERTY(EditAnywhere, Category = "Functions", meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnStateExit"), DisplayName="On State Exit")
	FMemberReference StateExitFunction;

	/** Function called when the owning state is fully blended out.
	 *
	 * Notes:
	 * - This will be called for any states that had weight.
	 * - This will not be called if the state is skipped. This can happen when the flag bSkipFirstUpdateTransition on the state machine node is set to true.
	 */
	UPROPERTY(EditAnywhere, Category = "Functions", meta=(FunctionReference, AllowFunctionLibraries, PrototypeFunction="/Script/AnimGraphRuntime.AnimExecutionContextLibrary.Prototype_ThreadSafeAnimUpdateCall", DefaultBindingName="OnStateFullyBlendedOut"), DisplayName="On State Fully Blended Out")
	FMemberReference StateFullyBlendedOutFunction;
};
