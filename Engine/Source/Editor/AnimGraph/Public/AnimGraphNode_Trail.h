// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_Trail.h"
#include "AnimGraphNode_Trail.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_Trail : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_Trail Node;

public:
	// UObject interface
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
//	virtual FString GetNodeNativeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UEdGraphNode Interface.

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent * PreviewSkelMeshComp) const override;
	// End of UAnimGraphNode_SkeletalControlBase interface

	//~ Begin UAnimGraphNode_Base Interface
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual void OnNodeSelected(bool bInIsSelected, class FEditorModeTools& InModeTools, struct FAnimNode_Base* InRuntimeNode);
	//~ End UAnimGraphNode_Base Interface
private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
	void DrawAngularLimits(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp, const FAnimNode_Trail& NodeToVisualize, const TArray<FName>& TrailBoneList) const;
};
