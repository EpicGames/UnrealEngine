// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Visibility.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_TransitionPoseEvaluator.h"
#include "AnimGraphNode_TransitionPoseEvaluator.generated.h"

class FBlueprintActionDatabaseRegistrar;
class IDetailLayoutBuilder;

UCLASS(MinimalAPI)
class UAnimGraphNode_TransitionPoseEvaluator : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FAnimNode_TransitionPoseEvaluator Node;
	
	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	virtual bool CanDuplicateNode() const override { return false; }
	virtual bool CanUserDeleteNode() const override;
	virtual FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UAnimGraphNode_Base interface

	// UK2Node interface.
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;

private:

	// Details customization helpers
	static EVisibility GetCacheFramesVisibility(IDetailLayoutBuilder* DetailLayoutBuilder);


	// End of UK2Node interface
};
