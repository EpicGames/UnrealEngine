// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_TransitionResult.h"
#include "AnimGraphNode_TransitionResult.generated.h"

class FBlueprintActionDatabaseRegistrar;

UCLASS(MinimalAPI)
class UAnimGraphNode_TransitionResult : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_TransitionResult Node;

	// UEdGraphNode interface
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool IsNodeRootSet() const override { return true; }
	virtual bool ShowVisualWarning() const override;
	virtual FText GetVisualWarningTooltipText() const override;
	// End of UEdGraphNode interface

	// UK2Node interface.
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	// UAnimGraphNode_Base interface
	virtual bool IsSinkNode() const override { return true; }
	// End of UAnimGraphNode_Base interface
};
