// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_TwistCorrectiveNode.h"
#include "AnimGraphNode_TwistCorrectiveNode.generated.h"

#define UE_API ANIMGRAPH_API

class FCompilerResultsLog;

/**
 * This is the 'source version' of a bone driven controller, which maps part of the state from one bone to another (e.g., 2 * source.x -> target.z)
 */
UCLASS(MinimalAPI)
class UAnimGraphNode_TwistCorrectiveNode : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_TwistCorrectiveNode Node;

public:

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

protected:

	// UAnimGraphNode_SkeletalControlBase protected interface
	UE_API virtual FText GetControllerDescription() const override;
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase protected interface

};

#undef UE_API
