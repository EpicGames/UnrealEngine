// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/TargetPoint.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_TwoBoneIK.h"
#include "AnimGraphNode_TwoBoneIK.generated.h"

// actor class used for bone selector
#define ABoneSelectActor ATargetPoint

class FTwoBoneIKDelegate;
class IDetailLayoutBuilder;

UCLASS(MinimalAPI)
class UAnimGraphNode_TwoBoneIK : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_TwoBoneIK Node;

	/** Enable drawing of the debug information of the node */
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bEnableDebugDraw;

	// just for refreshing UIs when bone space was changed
	static TSharedPtr<class FTwoBoneIKDelegate> TwoBoneIKDelegate;

public:
	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	virtual FEditorModeID GetEditorMode() const;
	virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) override;
	virtual void CopyPinDefaultsToNodeData(UEdGraphPin* InPin) override;
	// End of UAnimGraphNode_Base interface

	// UAnimGraphNode_SkeletalControlBase interface
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

	IDetailLayoutBuilder* DetailLayout;

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* SkelMeshComp) const override;
	virtual FText GetControllerDescription() const override;
	// End of UAnimGraphNode_SkeletalControlBase interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
