// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimNode_UseCachedPose.h"
#include "AnimGraphNode_UseCachedPose.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UAnimGraphNode_SaveCachedPose;

UCLASS(MinimalAPI)
class UAnimGraphNode_UseCachedPose : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY()
	FAnimNode_UseCachedPose Node;

	UPROPERTY()
	mutable TWeakObjectPtr<UAnimGraphNode_SaveCachedPose> SaveCachedPoseNode;

public:
	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual void JumpToDefinition() const override;
	// End of UEdGraphNode interface

	// UK2Node interface.
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	// End of UK2Node interface

	// UAnimGraphNode_Base interface
	virtual FString GetNodeCategory() const override;
	virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	virtual void OnProcessDuringCompilation(IAnimBlueprintCompilationContext& InCompilationContext, IAnimBlueprintGeneratedClassCompiledData& OutCompiledData) override;
	virtual void GetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const override;
	// End of UAnimGraphNode_Base interface

private:
	UPROPERTY()
	mutable FString NameOfCache;
};
