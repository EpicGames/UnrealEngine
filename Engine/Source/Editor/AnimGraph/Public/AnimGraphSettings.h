// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "AnimGraphSettings.generated.h"

UCLASS(config=Editor)
class UAnimGraphSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	//~ Begin UDeveloperSettings interface
	virtual FName GetContainerName() const override;
	virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FText GetSectionDescription() const override;
#endif
	//~ End UDeveloperSettings interface

	/**
	 * If true, populates the blueprint action menu with pre-bound blend-by-enum nodes for supported enums.
	 * For large projects this can clutter the context menu so it may be preferable to hide those entries.
	 * */
	UPROPERTY(EditAnywhere, config, Category = "Workflow")
	bool bShowInstancedEnumBlendAnimNodeBlueprintActions = true;

protected:
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
};