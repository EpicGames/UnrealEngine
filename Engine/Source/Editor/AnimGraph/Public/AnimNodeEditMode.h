// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"
#include "BonePose.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "IAnimNodeEditMode.h"
#include "IPersonaPreviewScene.h"
#include "InputCoreTypes.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Sphere.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/NameTypes.h"
#include "UnrealWidgetFwd.h"

#define UE_API ANIMGRAPH_API

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FText;
class FViewport;
class HHitProxy;
class UAnimGraphNode_Base;
class USkeletalMeshComponent;
struct FAnimNode_Base;
struct FBoneSocketTarget;
struct FCompactHeapPose;
struct FViewportClick;
template <class PoseType> struct FCSPose;

/** Base implementation for anim node edit modes */
class FAnimNodeEditMode : public IAnimNodeEditMode
{
public:
	UE_API FAnimNodeEditMode();

	/** IAnimNodeEditMode interface */
	UE_API virtual ECoordSystem GetWidgetCoordinateSystem() const override;
	UE_API virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	UE_API virtual UE::Widget::EWidgetMode ChangeToNextWidgetMode(UE::Widget::EWidgetMode CurWidgetMode) override;
	UE_API virtual bool SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode) override;
	UE_API virtual FName GetSelectedBone() const override;
	UE_API virtual void DoTranslation(FVector& InTranslation) override;
	UE_API virtual void DoRotation(FRotator& InRotation) override;
	UE_API virtual void DoScale(FVector& InScale) override;
	UE_API virtual void EnterMode(class UAnimGraphNode_Base* InEditorNode, struct FAnimNode_Base* InRuntimeNode) override;
	UE_API virtual void ExitMode() override;
	virtual bool SupportsPoseWatch() override { return false; };

	/** IPersonaEditMode interface */
	UE_API virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	UE_API virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	UE_API virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override;

	/** FEdMode interface */
	UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	UE_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	UE_API virtual FVector GetWidgetLocation() const override;
	UE_API virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	UE_API virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	UE_API virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	UE_API virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	UE_API virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	UE_API virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	UE_API virtual bool ShouldDrawWidget() const override;
	UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	UE_API virtual void Exit() override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	UE_API virtual void RegisterPoseWatchedNode(UAnimGraphNode_Base* InEditorNode, FAnimNode_Base* InRuntimeNode);

	struct EditorRuntimeNodePair
	{
		EditorRuntimeNodePair(UAnimGraphNode_Base* InEditorAnimNode, FAnimNode_Base* InRuntimeAnimNode)
			: EditorAnimNode(InEditorAnimNode)
			, RuntimeAnimNode(InRuntimeAnimNode)
		{}

		/** The node we are operating on */
		UAnimGraphNode_Base* EditorAnimNode;

		/** The runtime node in the preview scene */
		FAnimNode_Base* RuntimeAnimNode;
	};

	// Start IGizmoEdModeInterface overrides
	UE_API virtual bool BeginTransform(const FGizmoState& InState) override;
	UE_API virtual bool EndTransform(const FGizmoState& InState) override;
	// End IGizmoEdModeInterface overrides
	
protected:
	// local conversion functions for drawing
	static UE_API void ConvertToComponentSpaceTransform(const USkeletalMeshComponent* SkelComp, const FTransform & InTransform, FTransform & OutCSTransform, int32 BoneIndex, EBoneControlSpace Space);
	static UE_API void ConvertToBoneSpaceTransform(const USkeletalMeshComponent* SkelComp, const FTransform & InCSTransform, FTransform & OutBSTransform, int32 BoneIndex, EBoneControlSpace Space);
	// convert drag vector in component space to bone space 
	static UE_API FVector ConvertCSVectorToBoneSpace(const USkeletalMeshComponent* SkelComp, FVector& InCSVector, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space);
	static UE_API FVector ConvertCSVectorToBoneSpace(const USkeletalMeshComponent* SkelComp, FVector& InCSVector, FCSPose<FCompactHeapPose>& MeshBases, const FBoneSocketTarget& InTarget, const EBoneControlSpace Space);
	// convert rotator in component space to bone space 
	static UE_API FQuat ConvertCSRotationToBoneSpace(const USkeletalMeshComponent* SkelComp, FRotator& InCSRotator, FCSPose<FCompactHeapPose>& MeshBases, const FName& BoneName, const EBoneControlSpace Space);
	// convert widget location according to bone control space
	static UE_API FVector ConvertWidgetLocation(const USkeletalMeshComponent* InSkelComp, FCSPose<FCompactHeapPose>& InMeshBases, const FName& BoneName, const FVector& InLocation, const EBoneControlSpace Space);
	static UE_API FVector ConvertWidgetLocation(const USkeletalMeshComponent* InSkelComp, FCSPose<FCompactHeapPose>& InMeshBases, const FBoneSocketTarget& Target, const FVector& InLocation, const EBoneControlSpace Space);

	UE_API virtual UAnimGraphNode_Base* GetActiveWidgetAnimNode() const; // Return the editor node associated with the selected widget. All widget operations are performed on this node.
	UE_API virtual FAnimNode_Base*	GetActiveWidgetRuntimeAnimNode() const; // Return the runtime node associated with the selected widget. All widget operations are performed on this node.

	const bool IsManipulatingWidget() const { return bManipulating; }

	// Manage the start and end of a transform action in the viewport.
	UE_API bool HandleBeginTransform();
	UE_API bool HandleEndTransform();
	
	TArray< EditorRuntimeNodePair > SelectedAnimNodes;	// Selected Anim Graph Nodes
	TArray< EditorRuntimeNodePair > PoseWatchedAnimNodes; 	// Pose Watched Anim Graph Nodes. 

private:
	bool bManipulating;

	bool bInTransaction;
};

#undef UE_API
