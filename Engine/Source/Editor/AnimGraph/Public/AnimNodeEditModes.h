// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor.h"

#define UE_API ANIMGRAPH_API

struct AnimNodeEditModes
{
	UE_API const static FEditorModeID AnimDynamics;
	UE_API const static FEditorModeID AnimNode;
	UE_API const static FEditorModeID TwoBoneIK;
	UE_API const static FEditorModeID ObserveBone;
	UE_API const static FEditorModeID ModifyBone;
	UE_API const static FEditorModeID Fabrik;
	UE_API const static FEditorModeID CCDIK;
	UE_API const static FEditorModeID PoseDriver;
	UE_API const static FEditorModeID SplineIK;
	UE_API const static FEditorModeID LookAt;
};

#undef UE_API
