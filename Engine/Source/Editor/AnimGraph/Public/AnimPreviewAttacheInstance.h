// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimCustomInstanceHelper.h"
#include "Animation/AnimInstanceProxy.h"
#include "AnimNodes/AnimNode_CopyPoseFromMesh.h"
#include "AnimPreviewAttacheInstance.generated.h"

#define UE_API ANIMGRAPH_API

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FAnimPreviewAttacheInstanceProxy : public FAnimInstanceProxy
{
	GENERATED_BODY()

public:
	FAnimPreviewAttacheInstanceProxy()
	{
	}

	FAnimPreviewAttacheInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
	{
	}

	virtual void Initialize(UAnimInstance* InAnimInstance) override;
	virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	virtual bool Evaluate(FPoseContext& Output) override;

private:
	/** Pose blend node for evaluating pose assets (for previewing curve sources) */
	FAnimNode_CopyPoseFromMesh CopyPoseFromMesh;
};

/**
 * This Instance only contains one AnimationAsset, and produce poses
 * Used by Preview in AnimGraph, Playing single animation in Kismet2 and etc
 */

UCLASS(MinimalAPI, transient, NotBlueprintable, noteditinlinenew)
class UAnimPreviewAttacheInstance : public UAnimInstance
{
	GENERATED_UCLASS_BODY()

	//~ Begin UAnimInstance Interface
	UE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
	//~ End UAnimInstance Interface
};



#undef UE_API
