// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BonePose.h"
#include "BoneControllers/AnimNode_ModifyBone.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimSingleNodeInstanceProxy.h"
#include "AnimNodes/AnimNode_CurveSource.h"
#include "AnimNodes/AnimNode_PoseBlendNode.h"
#include "AnimNodes/AnimNode_CopyPoseFromMesh.h"
#include "AnimPreviewInstance.generated.h"

#define UE_API ANIMGRAPH_API

/** Enum to know how montage is being played */
UENUM()
enum EMontagePreviewType : int
{
	/** Playing montage in usual way. */
	EMPT_Normal, 
	/** Playing all sections. */
	EMPT_AllSections,
	EMPT_MAX,
};


/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FAnimPreviewInstanceProxy : public FAnimSingleNodeInstanceProxy
{
	GENERATED_BODY()

public:
	FAnimPreviewInstanceProxy()
	{
		bCanProcessAdditiveAnimations = true;
	}

	FAnimPreviewInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimSingleNodeInstanceProxy(InAnimInstance)
		, SkeletalControlAlpha(1.0f)
		, bEnableControllers(true)
		, bSetKey(false)
	{
		bCanProcessAdditiveAnimations = true;
	}

	UE_API virtual void Initialize(UAnimInstance* InAnimInstance) override;
	UE_API virtual void Update(float DeltaSeconds) override;
	UE_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	UE_API virtual bool Evaluate(FPoseContext& Output) override;
	UE_API virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	UE_API virtual void SetAnimationAsset(UAnimationAsset* NewAsset, USkeletalMeshComponent* MeshComponent, bool bIsLooping, float InPlayRate) override;

	UE_API void ResetModifiedBone(bool bCurveController = false);

	UE_API FAnimNode_ModifyBone* FindModifiedBone(const FName& InBoneName, bool bCurveController = false);	

	UE_API FAnimNode_ModifyBone& ModifyBone(const FName& InBoneName, bool bCurveController = false);

	UE_API void RemoveBoneModification(const FName& InBoneName, bool bCurveController = false);

	void EnableControllers(bool bEnable)
	{
		bEnableControllers = bEnable;
	}

	void SetSkeletalControlAlpha(float InSkeletalControlAlpha)
	{
		SkeletalControlAlpha = FMath::Clamp<float>(InSkeletalControlAlpha, 0.f, 1.f);
	}

#if WITH_EDITOR	
	void SetKey()
	{
		bSetKey = true;
	}

	FDelegateHandle AddKeyCompleteDelegate(FSimpleMulticastDelegate::FDelegate InOnSetKeyCompleteDelegate)
	{
		return OnSetKeyCompleteDelegate.Add(InOnSetKeyCompleteDelegate);
	}

	void RemoveKeyCompleteDelegate(FDelegateHandle InDelegateHandle)
	{
		OnSetKeyCompleteDelegate.Remove(InDelegateHandle);
	}
#endif

	UE_API void RefreshCurveBoneControllers(UAnimationAsset* AssetToRefreshFrom);

	TArray<FAnimNode_ModifyBone>& GetBoneControllers()
	{
		return BoneControllers;
	}

	TArray<FAnimNode_ModifyBone>& GetCurveBoneControllers()
	{
		return CurveBoneControllers;
	}

	virtual void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None) {}

	/** Sets an external debug skeletal mesh component to use to debug */
	UE_API void SetDebugSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent);

	/** Gets the external debug skeletal mesh component we are debugging */
	UE_API USkeletalMeshComponent* GetDebugSkeletalMeshComponent() const;

private:
	UE_API void UpdateCurveController();

	UE_API void ApplyBoneControllers(TArray<FAnimNode_ModifyBone> &InBoneControllers, FComponentSpacePoseContext& ComponentSpacePoseContext);

	UE_API void SetKeyImplementation(const FCompactPose& PreControllerInLocalSpace, const FCompactPose& PostControllerInLocalSpace);

	UE_API void AddKeyToSequence(UAnimSequence* Sequence, float Time, const FName& BoneName, const FTransform& AdditiveTransform);

private:
	/** Controllers for individual bones */
	TArray<FAnimNode_ModifyBone> BoneControllers;

	/** Curve modifiers */
	TArray<FAnimNode_ModifyBone> CurveBoneControllers;

	/** External curve for in-editor curve sources (such as audio) */
	FAnimNode_CurveSource CurveSource;

	/** Pose blend node for evaluating pose assets (for previewing curve sources) */
	FAnimNode_PoseBlendNode PoseBlendNode;

	/** Allows us to copy a pose from the mesh being debugged */
	FAnimNode_CopyPoseFromMesh CopyPoseNode;

	/**
	 * Delegate to call after Key is set
	 */
	FSimpleMulticastDelegate OnSetKeyCompleteDelegate;

	/** Shared parameters for previewing blendspace or animsequence **/
	float SkeletalControlAlpha;
	
	/*
	 * Used to determine if controller has to be applied or not
	 * Used to disable controller during editing
	 */
	bool bEnableControllers;

	/* 
	 * When this flag is true, it sets key
	 */
	bool bSetKey;
};

/**
 * This Instance only contains one AnimationAsset, and produce poses
 * Used by Preview in AnimGraph, Playing single animation in Kismet2 and etc
 */

UCLASS(MinimalAPI, transient, NotBlueprintable, noteditinlinenew)
class UAnimPreviewInstance : public UAnimSingleNodeInstance
{
	GENERATED_UCLASS_BODY()

		/** Shared parameters for previewing blendspace or animsequence **/
	UPROPERTY(transient)
	TEnumAsByte<enum EMontagePreviewType> MontagePreviewType;

	UPROPERTY(transient)
	int32 MontagePreviewStartSectionIdx;

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UAnimInstance Interface
	UE_API virtual void NativeInitializeAnimation() override;
	UE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
	virtual bool CanRunParallelWork() const { return false; }
protected:
	UE_API virtual void Montage_Advance(float DeltaTime) override;
	//~ End UAnimInstance Interface

public:
	/** Set SkeletalControl Alpha**/
	UE_API void SetSkeletalControlAlpha(float SkeletalControlAlpha);

	UE_API UAnimSequence* GetAnimSequence();

	//~ Begin UAnimSingleNodeInstance Interface
	UE_API virtual void RestartMontage(UAnimMontage* Montage, FName FromSection = FName()) override;
	UE_API virtual void SetAnimationAsset(UAnimationAsset* NewAsset, bool bIsLooping = true, float InPlayRate = 1.f) override;
	//~ End UAnimSingleNodeInstance Interface

	/** Montage preview functions */
	UE_API void MontagePreview_JumpToStart();
	UE_API void MontagePreview_JumpToEnd();
	UE_API void MontagePreview_JumpToPreviewStart();
	UE_API void MontagePreview_Restart();
	UE_API void MontagePreview_PreviewNormal(int32 FromSectionIdx = INDEX_NONE, bool bPlay = true);
	UE_API void MontagePreview_SetLoopNormal(bool bIsLooping, int32 PreferSectionIdx = INDEX_NONE);
	UE_API void MontagePreview_PreviewAllSections(bool bPlay = true);
	UE_API void MontagePreview_SetLoopAllSections(bool bIsLooping);
	UE_API void MontagePreview_SetLoopAllSetupSections(bool bIsLooping);
	UE_API void MontagePreview_ResetSectionsOrder();
	UE_API void MontagePreview_SetLooping(bool bIsLooping);
	UE_API void MontagePreview_SetPlaying(bool bIsPlaying);
	UE_API void MontagePreview_SetReverse(bool bInReverse);
	UE_API void MontagePreview_StepForward();
	UE_API void MontagePreview_StepBackward();
	UE_API void MontagePreview_JumpToPosition(float NewPosition);
	UE_API int32 MontagePreview_FindFirstSectionAsInMontage(int32 AnySectionIdx);
	UE_API int32 MontagePreview_FindLastSection(int32 StartSectionIdx);
	UE_API float MontagePreview_CalculateStepLength();
	UE_API void MontagePreview_RemoveBlendOut();
	bool IsPlayingMontage() { return GetActiveMontageInstance() != NULL; }

	/** 
	 * Finds an already modified bone 
	 * @param	InBoneName	The name of the bone modification to find
	 * @return the bone modification or NULL if no current modification was found
	 */
	UE_API FAnimNode_ModifyBone* FindModifiedBone(const FName& InBoneName, bool bCurveController=false);

	/** 
	 * Modifies a single bone. Create a new FAnimNode_ModifyBone if one does not exist for the passed-in bone.
	 * @param	InBoneName	The name of the bone to modify
	 * @return the new or existing bone modification
	 */
	UE_API FAnimNode_ModifyBone& ModifyBone(const FName& InBoneName, bool bCurveController=false);

	/**
	 * Removes an existing bone modification
	 * @param	InBoneName	The name of the existing modification to remove
	 */
	UE_API void RemoveBoneModification(const FName& InBoneName, bool bCurveController=false);

	/**
	 * Reset all bone modified
	 */
	UE_API void ResetModifiedBone(bool bCurveController=false);

	/**
	 * Returns all currently active bone controllers on this instance's proxy
	 */
	UE_API const TArray<FAnimNode_ModifyBone>& GetBoneControllers();

#if WITH_EDITOR	
	/**
	 * Convert current modified bone transforms (BoneControllers) to transform curves (CurveControllers)
	 * it does based on CurrentTime. This function does not set key directly here. 
	 * It does wait until next update, and it gets the delta of transform before applying curves, and 
	 * creates curves from it, so you'll need delegate if you'd like to do something after (set with SetKeyCompleteDelegate)
	 */
	UE_API void SetKey();

	/**
	 * Add the delegate to be called when a key is set.
	 * 
	 * @param Delegate To be called once set key is completed
	 */
	UE_API FDelegateHandle AddKeyCompleteDelegate(FSimpleMulticastDelegate::FDelegate InOnSetKeyCompleteDelegate);

	/**
	 * Add the delegate to be called when a key is set.
	 * 
	 * @param Delegate To be called once set key is completed
	 */
	UE_API void RemoveKeyCompleteDelegate(FDelegateHandle InDelegateHandle);
#endif

	/** 
	 * Refresh Curve Bone Controllers based on TransformCurves from Animation data
	 */
	UE_API void RefreshCurveBoneControllers();

	/** 
	 * Enable Controllers
	 * This is used by when editing, when controller has to be disabled
	 */
	UE_API void EnableControllers(bool bEnable);

	/** Preview physics interaction */
	UE_API void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None);

	/** Sets an external debug skeletal mesh component to use to debug */
	UE_API void SetDebugSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent);

	/** Gets the external debug skeletal mesh component we are debugging */
	UE_API USkeletalMeshComponent* GetDebugSkeletalMeshComponent() const;
};



#undef UE_API
