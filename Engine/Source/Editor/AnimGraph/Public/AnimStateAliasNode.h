// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimStateNodeBase.h"
#include "Animation/AnimTypes.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "AnimStateAliasNode.generated.h"

class FArchive;
class UAnimStateNode;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UAnimStateAliasNode : public UAnimStateNodeBase
{
	GENERATED_UCLASS_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State Alias")
	bool bGlobalAlias;

	UPROPERTY()
	FString StateAliasName;

	// UObject
	virtual void Serialize(FArchive& Ar) override;

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool CanDuplicateNode() const override { return true; }
	virtual void OnRenameNode(const FString& NewName) override;
	virtual void PostPasteNode() override;
	virtual void PostPlacedNewNode() override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UAnimStateNodeBase Interface
	virtual UEdGraphPin* GetInputPin() const override;
	virtual UEdGraphPin* GetOutputPin() const override;
	virtual FString GetStateName() const override;
	virtual FString GetDesiredNewNodeName() const override;
	UObject* GetJumpTargetForDoubleClick() const;
	//~ End UAnimStateNodeBase Interface
	
	ANIMGRAPH_API const TSet<TWeakObjectPtr<UAnimStateNodeBase>>& GetAliasedStates() const;
	ANIMGRAPH_API TSet<TWeakObjectPtr<UAnimStateNodeBase>>& GetAliasedStates();

	// Returns null if aliasing more than one state.
	ANIMGRAPH_API UAnimStateNodeBase* GetAliasedState() const;

	static FName GetAliasedStateNodesPropertyName() { return GET_MEMBER_NAME_CHECKED(UAnimStateAliasNode, AliasedStateNodes); }

private:

	void RebuildAliasedStateNodeReferences();

	UPROPERTY()
	TSet<TWeakObjectPtr<UAnimStateNodeBase>> AliasedStateNodes;
};
