// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimStateNodeBase.h"
#include "AnimStateConduitNode.generated.h"

class UEdGraph;
class UEdGraphPin;

UCLASS(MinimalAPI)
class UAnimStateConduitNode : public UAnimStateNodeBase
{
	GENERATED_UCLASS_BODY()
public:

	// The transition graph for this conduit; it's a logic graph, not an animation graph
	UPROPERTY()
	TObjectPtr<class UEdGraph> BoundGraph;

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool CanDuplicateNode() const override { return true; }
	virtual void PostPasteNode() override;
	virtual void PostPlacedNewNode() override;
	virtual void DestroyNode() override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual TArray<UEdGraph*> GetSubGraphs() const override { return TArray<UEdGraph*>( { BoundGraph } ); }
	//~ End UEdGraphNode Interface

	//~ Begin UAnimStateNodeBase Interface
	virtual UEdGraphPin* GetInputPin() const override;
	virtual UEdGraphPin* GetOutputPin() const override;
	virtual FString GetStateName() const override;
	virtual FString GetDesiredNewNodeName() const override;
	virtual UEdGraph* GetBoundGraph() const override { return BoundGraph; }
	virtual void ClearBoundGraph() override { BoundGraph = nullptr; }
	//~ End UAnimStateNodeBase Interface
};
