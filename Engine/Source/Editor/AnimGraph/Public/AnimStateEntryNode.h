// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AnimStateEntryNode.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UAnimStateEntryNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()


	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	//~ End UEdGraphNode Interface

	ANIMGRAPH_API UEdGraphNode* GetOutputNode() const;
	ANIMGRAPH_API UEdGraphPin* GetOutputPin() const;
};
