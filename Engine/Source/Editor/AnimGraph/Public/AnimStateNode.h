// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "AnimStateNodeBase.h"
#include "AnimStateNode.generated.h"

class UEdGraph;
class UEdGraphPin;
class UAnimGraphNode_StateResult;

UENUM()
enum EAnimStateType : int
{
	AST_SingleAnimation UMETA(DisplayName="Single animation"),
	AST_BlendGraph UMETA(DisplayName="Blend graph"),
};


UCLASS(MinimalAPI)
class UAnimStateNode : public UAnimStateNodeBase
{
	GENERATED_UCLASS_BODY()
public:

	// The animation graph for this state
	UPROPERTY()
	TObjectPtr<class UEdGraph> BoundGraph;

	// The type of the contents of this state
	UPROPERTY()
	TEnumAsByte<EAnimStateType> StateType;

	UPROPERTY(EditAnywhere, Category = "Animation State")
	FAnimNotifyEvent StateEntered;
	
	UPROPERTY(EditAnywhere, Category = "Animation State")
	FAnimNotifyEvent StateLeft;

	UPROPERTY(EditAnywhere, Category = "Animation State")
	FAnimNotifyEvent StateFullyBlended;

	// Whether or not this state will ALWAYS reset it's state on reentry, regardless of remaining weight
	UPROPERTY(EditAnywhere, Category = "Animation State")
	bool bAlwaysResetOnEntry;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool CanDuplicateNode() const override { return true; }
	virtual void PostPasteNode() override;
	virtual void PostPlacedNewNode() override;
	virtual void DestroyNode() override;
	virtual TArray<UEdGraph*> GetSubGraphs() const override { return TArray<UEdGraph*>( { BoundGraph } ); }
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	//~ End UEdGraphNode Interface
	
	//~ Begin UAnimStateNodeBase Interface
	virtual UEdGraphPin* GetInputPin() const override;
	virtual UEdGraphPin* GetOutputPin() const override;
	virtual FString GetStateName() const override;
	//~ End UAnimStateNodeBase Interface

	// @return the pose pin of the state sink node within the anim graph of this state
	ANIMGRAPH_API UEdGraphPin* GetPoseSinkPinInsideState() const;

	// @return the state sink node within the anim graph of this state
	ANIMGRAPH_API UAnimGraphNode_StateResult* GetResultNodeInsideState() const;

public:
	virtual UEdGraph* GetBoundGraph() const override { return BoundGraph; }
	virtual void ClearBoundGraph() override { BoundGraph = nullptr; }
};
