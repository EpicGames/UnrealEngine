// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "AnimStateNodeBase.generated.h"

class INameValidatorInterface;
class UAnimBlueprint;
class UEdGraph;
class UEdGraphPin;
class UEdGraphSchema;

UCLASS(MinimalAPI, abstract)
class UAnimStateNodeBase : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

	// UEdGraphNode interface
	virtual void PostPasteNode() override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual bool CanJumpToDefinition() const override;
	virtual void JumpToDefinition() const override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	// End of UEdGraphNode interface

	// @return the input pin for this state
	virtual UEdGraphPin* GetInputPin() const { return NULL; }

	// @return the output pin for this state
	virtual UEdGraphPin* GetOutputPin() const { return NULL; }

	// @return the name of this state
	virtual FString GetStateName() const { return TEXT("BaseState"); }

	// Populates the OutTransitions array with a list of transition nodes connected to this state
	ANIMGRAPH_API virtual void GetTransitionList(TArray<class UAnimStateTransitionNode*>& OutTransitions, bool bWantSortedList = false) const;

	virtual UEdGraph* GetBoundGraph() const { return NULL; }
	virtual void ClearBoundGraph() {}

	ANIMGRAPH_API UAnimBlueprint* GetAnimBlueprint() const;

protected:
	// Name used as a seed when pasting nodes
	virtual FString GetDesiredNewNodeName() const { return TEXT("State"); }
	
public:
	// Gets the animation state node documentation link
	virtual FString GetDocumentationLink() const override;
};
