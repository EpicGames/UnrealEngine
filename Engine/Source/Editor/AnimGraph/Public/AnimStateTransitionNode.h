// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimStateMachineTypes.h"
#include "Animation/BlendProfile.h"
#include "AnimStateNodeBase.h"

#include "AnimStateTransitionNode.generated.h"

class UCurveFloat;
class UEdGraph;
class UEdGraphPin;

UCLASS(MinimalAPI, config=Editor)
class UAnimStateTransitionNode : public UAnimStateNodeBase
{
	GENERATED_UCLASS_BODY()

	// The transition logic graph for this transition (returning a boolean)
	UPROPERTY()
	TObjectPtr<class UEdGraph> BoundGraph;

	// The animation graph for this transition if it uses custom blending (returning a pose)
	UPROPERTY()
	TObjectPtr<class UEdGraph> CustomTransitionGraph;

	// The priority order of this transition. If multiple transitions out of a state go
	// true at the same time, the one with the smallest priority order will take precedent
	UPROPERTY(EditAnywhere, Category=Transition)
	int32 PriorityOrder;

	// The duration to cross-fade for
	UPROPERTY(EditAnywhere, Config, Category=Transition, Meta=(ClampMin="0.0", ForceUnits="s"))
	float CrossfadeDuration;

	// The type of blending to use in the crossfade
	UPROPERTY()
	TEnumAsByte<ETransitionBlendMode::Type> CrossfadeMode_DEPRECATED;

	UPROPERTY(EditAnywhere, Category=Transition)
	EAlphaBlendOption BlendMode;

	UPROPERTY(EditAnywhere, Category=Transition)
	TObjectPtr<UCurveFloat> CustomBlendCurve;

	// The blend profile to use to evaluate this transition per-bone
	UE_DEPRECATED(5.6, "Use the interfaced blend profile instead")
	UPROPERTY()
	TObjectPtr<UBlendProfile> BlendProfile_DEPRECATED;

	// The blend profile to use to evaluate this transition per-bone
	UPROPERTY(EditAnywhere, Category=Transition, meta=(UseAsBlendProfile=true))
	FBlendProfileInterfaceWrapper BlendProfileWrapper;

	// Try setting the rule automatically based on most relevant asset player node's remaining time and the Automatic Rule Trigger Time of the transition, ignoring the internal time
	UPROPERTY(EditAnywhere, Category=Transition)
	bool bAutomaticRuleBasedOnSequencePlayerInState;

	// When should the automatic transition rule trigger relative to the time remaining on the relevant asset player:
	//  < 0 means trigger the transition 'Crossfade Duration' seconds before the end of the asset player, so a standard blend would finish just as the asset player ends
	// >= 0 means trigger the transition 'Automatic Rule Trigger Time' seconds before the end of the asset player
	UPROPERTY(EditAnywhere, Category=Transition, meta=(ForceUnits="s"))
	float AutomaticRuleTriggerTime;

	// The minimum time elapsed (in seconds) before the target state can be re-entered via this transition 
	// Has no effect when set to -1.0.
	// When set to zero, wait at least one frame before re-entry via this transition is allowed.
	UPROPERTY(EditAnywhere, Category=Transition, DisplayName="Min Time Before Re-entry", meta=(ForceUnits="s"))
	float MinTimeBeforeReentry;

	// If SyncGroupName is specified, Transition will only be taken if the SyncGroup has valid markers (other transition rules still apply in addition to that).
	UPROPERTY(EditAnywhere, Category = Transition)
	FName SyncGroupNameToRequireValidMarkersRule;

	// What transition logic to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transition)
	TEnumAsByte<ETransitionLogicType::Type> LogicType;

	UPROPERTY(EditAnywhere, Category=Events)
	FAnimNotifyEvent TransitionStart;

	UPROPERTY(EditAnywhere, Category=Events)
	FAnimNotifyEvent TransitionEnd;

	UPROPERTY(EditAnywhere, Category=Events)
	FAnimNotifyEvent TransitionInterrupt;

	/** Whether to fall back to inertialization/dead blending when reentering an already-active state. This can avoid pops. 
		The target state must enable bAlwaysResetOnEntry for the inertial blend to trigger. */
	UPROPERTY(EditAnywhere, Category="Transition|Experimental")
	bool bAllowInertializationForSelfTransitions;

	/** This transition can go both directions */
	UPROPERTY(EditAnywhere, Category=Transition)
	bool Bidirectional;

	/** Whether or not to disable this transition from being enterable */
	UPROPERTY(EditAnywhere, Category=Transition)
	bool bDisabled;

	/** If true, this transition rule will not be evaluated when the state machine's update context is inactive (e.g. when blending out). */
	UPROPERTY(EditAnywhere, Category=Transition)
	bool bOnlyEvaluateWhenActive;

	/** The rules for this transition may be shared with other transition nodes */
	UPROPERTY()
	bool bSharedRules;

	/** The cross-fade settings of this node may be shared */
	UPROPERTY()
	bool bSharedCrossfade;

	/** What we call this transition if we are shared ('Transition X to Y' can't be used since its used in multiple places) */
	UPROPERTY()
	FString	SharedRulesName;

	/** Shared rules guid useful when copying between different state machines */
	UPROPERTY()
	FGuid SharedRulesGuid;

	/** Color we draw in the editor as if we are shared */
	UPROPERTY()
	FLinearColor SharedColor;

	UPROPERTY()
	FString SharedCrossfadeName;

	UPROPERTY()
	FGuid SharedCrossfadeGuid;

	UPROPERTY()
	int32 SharedCrossfadeIdx;

	UPROPERTY()
	FVector2D CachedRotation;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual bool CanDuplicateNode() const override { return true; }
	virtual void PrepareForCopying() override;
	virtual void PostPasteNode() override;
	virtual void PostPlacedNewNode() override;
	virtual void DestroyNode() override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual TArray<UEdGraph*> GetSubGraphs() const override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UAnimStateNodeBase Interface
	virtual UEdGraph* GetBoundGraph() const override { return BoundGraph; }
	virtual void ClearBoundGraph() override { BoundGraph = nullptr; }
	virtual UEdGraphPin* GetInputPin() const override { return Pins[0]; }
	virtual UEdGraphPin* GetOutputPin() const override { return Pins[1]; }
	//~ End UAnimStateNodeBase Interface

	// @return the name of this state
	ANIMGRAPH_API FString GetStateName() const override;

	ANIMGRAPH_API UAnimStateNodeBase* GetPreviousState() const;
	ANIMGRAPH_API UAnimStateNodeBase* GetNextState() const;
	ANIMGRAPH_API void CreateConnections(UAnimStateNodeBase* PreviousState, UAnimStateNodeBase* NextState);

	/**
	 * Relink transition head (where the arrow head is of a state transition) to a new state.
	 * @param[in] NewTargetState The new transition target.
	 */
	ANIMGRAPH_API void RelinkHead(UAnimStateNodeBase* NewTargetState);

	/**
	 * Relink transition tail (where the arrow base is of a state transition) to a new state.
	 * @param[in] NewSourceState The new transition source.
	 */
	ANIMGRAPH_API void RelinkTail(UAnimStateNodeBase* NewSourceState);

	/**
	 * Reverse the direction of this transition
	 */
	ANIMGRAPH_API void Reverse();
	
	/**
	 * Helper function to gather the transition nodes to be relinked by taking the graph selection into account as well.
	 * For example when relinking a transition holding several transition nodes but only a few are selected to be relinked.
	 */
	ANIMGRAPH_API static TArray<UAnimStateTransitionNode*> GetListTransitionNodesToRelink(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes);
	
	ANIMGRAPH_API bool IsBoundGraphShared() const;

	ANIMGRAPH_API void MakeRulesShareable(FString ShareName);
	ANIMGRAPH_API void MakeCrossfadeShareable(FString ShareName);

	ANIMGRAPH_API void UnshareRules();
	ANIMGRAPH_API void UnshareCrossade();

	ANIMGRAPH_API void UseSharedRules(const UAnimStateTransitionNode* Node);
	ANIMGRAPH_API void UseSharedCrossfade(const UAnimStateTransitionNode* Node);
	
	void CopyCrossfadeSettings(const UAnimStateTransitionNode* SrcNode);
	void PropagateCrossfadeSettings();

	ANIMGRAPH_API bool IsReverseTrans(const UAnimStateNodeBase* Node);
protected:
	void CreateBoundGraph();
	void CreateCustomTransitionGraph();

public:
	UEdGraph* GetCustomTransitionGraph() const { return CustomTransitionGraph; }

	/** Validates any currently set blend profile, ensuring it's valid for the current skeleton */
	ANIMGRAPH_API bool ValidateBlendProfile();
};
