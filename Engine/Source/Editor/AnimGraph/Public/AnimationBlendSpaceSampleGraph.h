// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationGraph.h"
#include "AnimationBlendSpaceSampleGraph.generated.h"

class UAnimGraphNode_BlendSpaceSampleResult;

UCLASS(MinimalAPI)
class UAnimationBlendSpaceSampleGraph : public UAnimationGraph
{
	GENERATED_BODY()

public:
	// Result node within the state's animation graph
	UPROPERTY()
	TObjectPtr<UAnimGraphNode_BlendSpaceSampleResult> ResultNode;
};
