// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraphSchema_K2.h"
#include "AnimationConduitGraphSchema.generated.h"

//@TODO: Should this derive from AnimationTransitionSchema (with appropriate suppression of state-based queries)
UCLASS(MinimalAPI)
class UAnimationConduitGraphSchema : public UEdGraphSchema_K2
{
	GENERATED_UCLASS_BODY()

	// UEdGraphSchema interface
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const override { return false; }
	virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return true; }
	virtual void HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const override;
	// End of UEdGraphSchema interface

	// UEdGraphSchema_K2 interface
	virtual bool DoesSupportCollapsedNodes() const override { return false; }
	virtual bool DoesSupportEventDispatcher() const	override { return false; }
	// End of UEdGraphSchema_K2 interface
};
