// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationGraph.h"
#include "AnimationCustomTransitionGraph.generated.h"

UCLASS(MinimalAPI)
class UAnimationCustomTransitionGraph : public UAnimationGraph
{
	GENERATED_UCLASS_BODY()

	// Result node within the state's animation graph
	UPROPERTY()
	TObjectPtr<class UAnimGraphNode_CustomTransitionResult> MyResultNode;

	//@TODO: Document
	ANIMGRAPH_API class UAnimGraphNode_CustomTransitionResult* GetResultNode();
};

