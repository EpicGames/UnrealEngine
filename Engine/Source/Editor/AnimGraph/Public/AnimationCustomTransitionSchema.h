// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationGraphSchema.h"
#include "AnimationCustomTransitionSchema.generated.h"

UCLASS(MinimalAPI)
class UAnimationCustomTransitionSchema : public UAnimationGraphSchema
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphSchema Interface.
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	virtual void HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const override;
	//~ End UEdGraphSchema Interface.
};
