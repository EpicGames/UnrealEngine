// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraph.h"
#include "Animation/AnimClassInterface.h"
#include "AnimGraphNode_Base.h"
#include "AnimationGraph.generated.h"

#define UE_API ANIMGRAPH_API

class UEdGraphPin;

/** Delegate fired when a pin's default value is changed */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPinDefaultValueChanged, UEdGraphPin* /*InPinThatChanged*/)

UCLASS(MinimalAPI, BlueprintType)
class UAnimationGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

	/** Delegate fired when a pin's default value is changed */
	FOnPinDefaultValueChanged OnPinDefaultValueChanged;

	/** Blending options for animation graphs in Linked Animation Blueprints. */
	UPROPERTY(EditAnywhere, Category = GraphBlending, meta = (ShowOnlyInnerProperties))
	FAnimGraphBlendOptions BlendOptions;

	/** Returns contained graph nodes of the specified (or child) class */
	UFUNCTION(BlueprintCallable, Category=AnimationGraph)
	UE_API void GetGraphNodesOfClass(TSubclassOf<UAnimGraphNode_Base> NodeClass, TArray<UAnimGraphNode_Base*>& GraphNodes, bool bIncludeChildClasses = true);

private:
	// UObject interface
	UE_API virtual void PostEditUndo() override;

	// Reconstruct layer nodes post-undo
	UE_API void ReconstructLayerNodes() const;
};

#undef UE_API
