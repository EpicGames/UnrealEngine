// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "AnimationGraphSchema.generated.h"

class FMenuBuilder;
class IAssetReferenceFilter;
class UAnimationAsset;
class UAnimBlueprint;
class UPhysicsAsset;
struct FBPInterfaceDescription;
class UAnimGraphNode_Base;
class UAnimGraphNode_LinkedAnimGraphBase;

UCLASS(MinimalAPI)
class UAnimationGraphSchema : public UEdGraphSchema_K2
{
	GENERATED_UCLASS_BODY()
	// Common PinNames
	UPROPERTY()
	FString PN_SequenceName;    // PC_Object+PSC_Sequence

	UPROPERTY()
	FName NAME_NeverAsPin;

	UPROPERTY()
	FName NAME_PinHiddenByDefault;

	UPROPERTY()
	FName NAME_PinShownByDefault;

	UPROPERTY()
	FName NAME_AlwaysAsPin;

	UPROPERTY()
	FName NAME_CustomizeProperty;

	UPROPERTY()
	FName NAME_OnEvaluate;
	
	UPROPERTY()
	FName DefaultEvaluationHandlerName;

	//~ Begin UEdGraphSchema Interface.
	ANIMGRAPH_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	ANIMGRAPH_API virtual EGraphType GetGraphType(const UEdGraph* TestEdGraph) const override;
	ANIMGRAPH_API virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	ANIMGRAPH_API virtual void HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const override;
	ANIMGRAPH_API virtual bool CreateAutomaticConversionNodeAndConnections(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	ANIMGRAPH_API virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	ANIMGRAPH_API virtual void DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;
	ANIMGRAPH_API virtual void DroppedAssetsOnPin(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphPin* Pin) const override;
	ANIMGRAPH_API virtual void GetAssetsNodeHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const override;
	ANIMGRAPH_API virtual void GetAssetsPinHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphPin* HoverPin, FString& OutTooltipText, bool& OutOkIcon) const override;
	ANIMGRAPH_API virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	ANIMGRAPH_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	ANIMGRAPH_API virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	ANIMGRAPH_API virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return true; }
	ANIMGRAPH_API virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	ANIMGRAPH_API virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	ANIMGRAPH_API virtual bool CanGraphBeDropped(TSharedPtr<FEdGraphSchemaAction> InAction) const override;
	ANIMGRAPH_API virtual FReply BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction, const FPointerEvent& MouseEvent = FPointerEvent()) const override;
	//~ End UEdGraphSchema Interface.

	//~ Begin UEdGraphSchema_K2 Interface
	ANIMGRAPH_API virtual const FPinConnectionResponse DetermineConnectionResponseOfCompatibleTypedPins(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const override;
	ANIMGRAPH_API virtual TOptional<FSearchForAutocastFunctionResults> SearchForAutocastFunction(const FEdGraphPinType& OutputPinType, const FEdGraphPinType& InputPinType) const;
	ANIMGRAPH_API virtual bool ArePinsCompatible(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UClass* CallingContext = NULL, bool bIgnoreArray = false) const override;
	virtual bool DoesSupportCollapsedNodes() const override { return false; }
	virtual bool DoesSupportEventDispatcher() const	override { return false; }
	ANIMGRAPH_API virtual bool DoesSupportAnimNotifyActions() const override;
	ANIMGRAPH_API virtual void CreateFunctionGraphTerminators(UEdGraph& Graph, UClass* Class) const override;
	ANIMGRAPH_API virtual bool CanShowDataTooltipForPin(const UEdGraphPin& Pin) const override;
	//~ End UEdGraphSchema_K2 Interface

	/** Spawn the correct node in the Animation Graph using the given AnimationAsset at the supplied location */
	static void SpawnNodeFromAsset(UAnimationAsset* Asset, const UE::Slate::FDeprecateVector2DParameter& GraphPosition, UEdGraph* Graph, UEdGraphPin* PinIfAvailable);

	/** Spawn a rigid body node if we drop a physics asset on the graph */
	static void SpawnRigidBodyNodeFromAsset(UPhysicsAsset* Asset, const UE::Slate::FDeprecateVector2DParameter& GraphPosition, UEdGraph* Graph);

	/** Update the specified node to a new asset */
	static void UpdateNodeWithAsset(class UK2Node* K2Node, UAnimationAsset* Asset);

	/** Auto-arranges a graph's inputs and outputs. Does nothing to nodes that are not roots or inputs */
	ANIMGRAPH_API static void AutoArrangeInterfaceGraph(UEdGraph& Graph);

	/** Checks to see whether the passed-in pin type is a pose pin (local or component space) */
	ANIMGRAPH_API static bool IsPosePin(const FEdGraphPinType& PinType);

	/** Checks to see whether the passed-in pin type is a local space pose pin */
	ANIMGRAPH_API static bool IsLocalSpacePosePin(const FEdGraphPinType& PinType);

	/** Checks to see whether the passed-in pin type is a component space pose pin */
	ANIMGRAPH_API static bool IsComponentSpacePosePin(const FEdGraphPinType& PinType);

	/** Makes a local space pose pin type */
	ANIMGRAPH_API static FEdGraphPinType MakeLocalSpacePosePin();

	/** Makes a component space pose pin type */
	ANIMGRAPH_API static FEdGraphPinType MakeComponentSpacePosePin();

	/** Conforms an anim graph to an interface function */
	ANIMGRAPH_API static void ConformAnimGraphToInterface(UBlueprint* InBlueprint, UEdGraph& InGraph, UFunction* InFunction);

	/** Conforms anim layer nodes to an interface desc by GUID */
	ANIMGRAPH_API static void ConformAnimLayersByGuid(const UAnimBlueprint* InAnimBlueprint, const FBPInterfaceDescription& CurrentInterfaceDesc);

	UE_DEPRECATED(4.24, "Function renamed, please use GetPositionForNewLinkedInputPoseNode")
	static FVector2D GetPositionForNewSubInputNode(UEdGraph& InGraph) { return GetPositionForNewLinkedInputPoseNode(InGraph); }

	/** Find a position for a newly created linked input pose */
	ANIMGRAPH_API static FVector2D GetPositionForNewLinkedInputPoseNode(UEdGraph& InGraph);

	/** Create a binding widget for the specified named pin on the specified anim graph nodes */
	ANIMGRAPH_API static TSharedPtr<SWidget> MakeBindingWidgetForPin(const TArray<UAnimGraphNode_Base*>& InAnimGraphNodes, FName InPinName, bool bInOnGraphNode, TAttribute<bool> bInIsEnabled);
	
	/** Unexpose pins that are unused on a linked anim graph node */
	ANIMGRAPH_API static void HideUnboundPropertyPins(UAnimGraphNode_LinkedAnimGraphBase* Node);

private:
	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};



