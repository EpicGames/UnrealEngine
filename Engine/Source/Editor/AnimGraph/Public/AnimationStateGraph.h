// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationGraph.h"
#include "AnimationStateGraph.generated.h"

UCLASS(MinimalAPI)
class UAnimationStateGraph : public UAnimationGraph
{
	GENERATED_UCLASS_BODY()

	// Result node within the state's animation graph
	UPROPERTY()
	TObjectPtr<class UAnimGraphNode_StateResult> MyResultNode;

	ANIMGRAPH_API class UAnimGraphNode_StateResult* GetResultNode();
};

