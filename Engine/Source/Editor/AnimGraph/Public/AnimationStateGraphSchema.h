// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationGraphSchema.h"
#include "AnimationStateGraphSchema.generated.h"

UCLASS(MinimalAPI)
class UAnimationStateGraphSchema : public UAnimationGraphSchema
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphSchema Interface.
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const override;;
	//~ End UEdGraphSchema Interface.
};
