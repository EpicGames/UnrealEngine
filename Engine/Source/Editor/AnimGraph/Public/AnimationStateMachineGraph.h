// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AnimationStateMachineGraph.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UAnimationStateMachineGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

	// Entry node within the state machine
	UPROPERTY()
	TObjectPtr<class UAnimStateEntryNode> EntryNode;

	// Parent instance node
	UPROPERTY()
	TObjectPtr<class UAnimGraphNode_StateMachineBase> OwnerAnimGraphNode;
};

