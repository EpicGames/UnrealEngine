// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"
#include "AnimationStateMachineSchema.generated.h"

#define UE_API ANIMGRAPH_API

class FMenuBuilder;
class IAssetReferenceFilter;
class UEdGraph;

/** Action to add a node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_NewStateNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	TObjectPtr<UEdGraphNode> NodeTemplate;

	FEdGraphSchemaAction_NewStateNode() 
		: FEdGraphSchemaAction()
		, NodeTemplate(nullptr)
	{}

	FEdGraphSchemaAction_NewStateNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NodeTemplate(nullptr)
	{}

	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;

	template <typename NodeType>
	static NodeType* SpawnNodeFromTemplate(class UEdGraph* ParentGraph, NodeType* InTemplateNode, const UE::Slate::FDeprecateVector2DParameter Location = FVector2f(0.0f, 0.0f), bool bSelectNewNode = true)
	{
		FEdGraphSchemaAction_NewStateNode Action;
		Action.NodeTemplate = InTemplateNode;

		return Cast<NodeType>(Action.PerformAction(ParentGraph, NULL, Location, bSelectNewNode));
	}
};

/** Action to create new comment */
USTRUCT()
struct FEdGraphSchemaAction_NewStateComment : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	FEdGraphSchemaAction_NewStateComment() 
		: FEdGraphSchemaAction()
	{}

	FEdGraphSchemaAction_NewStateComment(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};


UCLASS(MinimalAPI)
class UAnimationStateMachineSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	ANIMGRAPH_API static const FName PC_Exec;

	//~ Begin UEdGraphSchema Interface
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	virtual bool CreateAutomaticConversionNodeAndConnections(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	virtual bool TryRelinkConnectionTarget(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, UEdGraphPin* NewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const override;
	virtual bool IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const override;
	virtual const FPinConnectionResponse CanRelinkConnectionToPin(const UEdGraphPin* OldSourcePin, const UEdGraphPin* TargetPinCandidate) const override;
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual EGraphType GetGraphType(const UEdGraph* TestEdGraph) const override;
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	virtual void GetAssetsNodeHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const override;

	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	//~ End UEdGraphSchema Interface

private:
	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};

#undef UE_API
