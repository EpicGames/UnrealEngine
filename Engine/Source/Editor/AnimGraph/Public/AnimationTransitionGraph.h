// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationGraph.h"
#include "AnimationTransitionGraph.generated.h"

UCLASS(MinimalAPI)
class UAnimationTransitionGraph : public UAnimationGraph
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UAnimGraphNode_TransitionResult> MyResultNode;

	ANIMGRAPH_API class UAnimGraphNode_TransitionResult* GetResultNode();
};

