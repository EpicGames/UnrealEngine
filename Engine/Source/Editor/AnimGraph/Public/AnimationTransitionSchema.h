// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraphSchema_K2.h"
#include "AnimationTransitionSchema.generated.h"

class UAnimStateNode;
class UAnimStateTransitionNode;
struct FAnimBlueprintDebugData;

// This class is the schema for transition rule graphs in animation state machines
UCLASS(MinimalAPI)
class UAnimationTransitionSchema : public UEdGraphSchema_K2
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphSchema Interface.
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const override { return false; }
	virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return true; }
	virtual void HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const override;
	//~ End UEdGraphSchema Interface.

	//~ Begin UEdGraphSchema_K2 Interface.
	virtual bool DoesSupportCollapsedNodes() const override { return false; }
	virtual bool DoesSupportEventDispatcher() const	override { return false; }
	//~ End UEdGraphSchema_K2 Interface.

private:
	static UAnimStateTransitionNode* GetTransitionNodeFromGraph(const FAnimBlueprintDebugData& DebugData, const UEdGraph* Graph);

	static UAnimStateNode* GetStateNodeFromGraph(const FAnimBlueprintDebugData& DebugData, const UEdGraph* Graph);

};
