// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "BlendSpaceGraph.generated.h"

class UBlendSpace;
class UObject;

// Dummy graph to hold sub-graphs for blendspaces. Not edited directly.
UCLASS(MinimalAPI)
class UBlendSpaceGraph : public UEdGraph
{
	GENERATED_BODY()

private:
	// UObject interface
	virtual void PostLoad() override;

public:
	// Blendspace that we wrap
	UPROPERTY(VisibleAnywhere, Category="BlendSpace", Instanced, NoClear, meta = (ShowOnlyInnerProperties))
	TObjectPtr<UBlendSpace> BlendSpace;
};