// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FCompilerResultsLog;
class UAnimGraphNode_Base;

// Interface passed to start/end compilation delegates
class IAnimBlueprintCompilationBracketContext
{
public:	
	virtual ~IAnimBlueprintCompilationBracketContext() = default;

	// Get the message log for the current compilation
	FCompilerResultsLog& GetMessageLog() const { return GetMessageLogImpl(); }

	// Index of the nodes (must match up with the runtime discovery process of nodes, which runs thru the property chain)
	const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndices() const { return GetAllocatedAnimNodeIndicesImpl(); }

	// Map of anim node indices to node handler properties in sparse class data struct
	const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedHandlerPropertiesByNode() const { return GetAllocatedHandlerPropertiesByNodeImpl(); }

protected:
	// Get the message log for the current compilation
	virtual FCompilerResultsLog& GetMessageLogImpl() const = 0;

	// Map of anim node properties to original anim graph node
	virtual const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndicesImpl() const = 0;

	// Map of anim node indices to node handler properties in sparse class data struct
	virtual const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedHandlerPropertiesByNodeImpl() const = 0;
};
