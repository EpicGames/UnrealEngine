// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetCompiler.h"
#include "Containers/ArrayView.h"

#define UE_API ANIMGRAPH_API

class UEdGraphNode;
class UEdGraph;
class UEdGraphPin;
struct FPoseLinkMappingRecord;
class UAnimGraphNode_Base;
class FCompilerResultsLog;
class UBlueprint;
class UAnimBlueprint;
class FProperty;
struct FKismetCompilerOptions;

#define ANIM_FUNC_DECORATOR	TEXT("__AnimFunc")

/** Interface to the anim BP compiler context for use while compilation is in progress */
class IAnimBlueprintCompilationContext
{
public:
	// Record of a property that can be folded into the class members/constant blocks
	struct FFoldedPropertyRecord
	{
		FFoldedPropertyRecord(UAnimGraphNode_Base* InAnimGraphNode, FStructProperty* InAnimNodeProperty, FProperty* InProperty, bool bInIsOnClass)
			: AnimGraphNode(InAnimGraphNode)
			, AnimNodeProperty(InAnimNodeProperty)
			, Property(InProperty)
			, GeneratedProperty(nullptr)
			, FoldIndex(INDEX_NONE)
			, PropertyIndex(INDEX_NONE)
			, bIsOnClass(bInIsOnClass)
		{
		}

		// The anim graph node that this property record came from
		UAnimGraphNode_Base* AnimGraphNode = nullptr;

		// The property of the FAnimNode_Base-derived structure within the anim graph node
		FStructProperty* AnimNodeProperty = nullptr;

		// The original property within the FAnimNode_Base
		FProperty* Property = nullptr;

		// The generated property within the respective data area (either constants or mutables)
		FProperty* GeneratedProperty = nullptr;

		// The index that this property was folded to. INDEX_NONE if it was not folded.
		int32 FoldIndex = INDEX_NONE;

		// The index of the property in its respective data area (either constant or mutable)
		// This will be INDEX_NONE if the property was folded.
		int32 PropertyIndex = INDEX_NONE;

		// Whether this property will be held on the class (constants on sparse class data), or on the instance (mutables struct)
		bool bIsOnClass = false;
	};

	virtual ~IAnimBlueprintCompilationContext() = default;

	// Get a compilation context from a kismet compiler context assuming that it is an FAnimBlueprintCompilerContext
	static UE_API TUniquePtr<IAnimBlueprintCompilationContext> Get(FKismetCompilerContext& InKismetCompiler);

	// Spawns an intermediate node associated with the source node (for error purposes)
	template <typename NodeType>
	NodeType* SpawnIntermediateNode(UEdGraphNode* SourceNode, UEdGraph* ParentGraph = nullptr)
	{
		return GetKismetCompiler()->SpawnIntermediateNode<NodeType>(SourceNode, ParentGraph);
	}

	// Find a property in the currently-compiled class
	template <typename FieldType>
	FieldType* FindClassFProperty(const TCHAR* InFieldPath) const
	{
		return FindFProperty<FieldType>(GetKismetCompiler()->NewClass, InFieldPath);
	}

	// Adds a pose link mapping record
	void AddPoseLinkMappingRecord(const FPoseLinkMappingRecord& InRecord) { AddPoseLinkMappingRecordImpl(InRecord); }

	// Process the passed-in list of nodes
	void ProcessAnimationNodes(TArray<UAnimGraphNode_Base*>& AnimNodeList) { ProcessAnimationNodesImpl(AnimNodeList); }

	// Prunes any nodes that aren't reachable via a pose link
	void PruneIsolatedAnimationNodes(const TArray<UAnimGraphNode_Base*>& RootSet, TArray<UAnimGraphNode_Base*>& GraphNodes) { PruneIsolatedAnimationNodesImpl(RootSet, GraphNodes); }

	// Perform an expansion step for the specified graph
	void ExpansionStep(UEdGraph* Graph, bool bAllowUbergraphExpansions) { ExpansionStepImpl(Graph, bAllowUbergraphExpansions); }

	// Get the message log for the current compilation
	FCompilerResultsLog& GetMessageLog() const { return GetMessageLogImpl(); }

	// Performs standard validation on the graph (outputs point to inputs, no more than one connection to each input, types match on both ends, etc...)
	bool ValidateGraphIsWellFormed(UEdGraph* Graph) const { return ValidateGraphIsWellFormedImpl(Graph); }

	// Returns the allocation index of the specified node, processing it if it was pending
	int32 GetAllocationIndexOfNode(UAnimGraphNode_Base* VisualAnimNode) const { return GetAllocationIndexOfNodeImpl(VisualAnimNode); }

	// Get the currently-compiled blueprint
	const UBlueprint* GetBlueprint() const { return GetBlueprintImpl(); }

	// Get the currently-compiled anim blueprint
	const UAnimBlueprint* GetAnimBlueprint() const { return GetAnimBlueprintImpl(); }

	// Get the consolidated uber graph during compilation
	UEdGraph* GetConsolidatedEventGraph() const { return GetConsolidatedEventGraphImpl(); }

	// Gets all anim graph nodes that are piped into the provided node (traverses input pins)
	void GetLinkedAnimNodes(UAnimGraphNode_Base* InGraphNode, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const { return GetLinkedAnimNodesImpl(InGraphNode, LinkedAnimNodes); }

	// Index of the nodes (must match up with the runtime discovery process of nodes, which runs thru the property chain)
	const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndices() const { return GetAllocatedAnimNodeIndicesImpl(); }

	// Map of true source objects (user edited ones) to the cloned ones that are actually compiled
	const TMap<UAnimGraphNode_Base*, UAnimGraphNode_Base*>& GetSourceNodeToProcessedNodeMap() const { return GetSourceNodeToProcessedNodeMapImpl(); }

	// Map of anim node indices to node properties
	const TMap<int32, FProperty*>& GetAllocatedPropertiesByIndex() const { return GetAllocatedPropertiesByIndexImpl(); }

	// Map of anim node indices to node properties
	const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedPropertiesByNode() const { return GetAllocatedPropertiesByNodeImpl(); }

	// Map of anim node indices to node handler properties in sparse class data struct
	const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedHandlerPropertiesByNode() const { return GetAllocatedHandlerPropertiesByNodeImpl(); }

	// Add the specified compiled-in attribute uniquely to the specified node
	void AddAttributesToNode(UAnimGraphNode_Base* InNode, TArrayView<const FName> InAttributes) const { AddAttributesToNodeImpl(InNode, InAttributes); }

	// Get the current compiled-in attributes uniquely assigned to the specified node
	TArrayView<const FName> GetAttributesFromNode(UAnimGraphNode_Base* InNode) const { return GetAttributesFromNodeImpl(InNode); }

	// Check whether an anim node participates in constant folding
	bool IsAnimGraphNodeFolded(UAnimGraphNode_Base* InNode) const { return IsAnimGraphNodeFoldedImpl(InNode); }

	// Get the folded property record, if any, for the supplied node & named property
	const FFoldedPropertyRecord* GetFoldedPropertyRecord(UAnimGraphNode_Base* InNode, FName InPropertyName) const { return GetFoldedPropertyRecordImpl(InNode, InPropertyName); }

	// Get the generated property of the class that mutable data is added to
	virtual const FStructProperty* GetMutableDataProperty() const { return GetMutableDataPropertyImpl(); }
	
protected:
	// Adds a pose link mapping record
	virtual void AddPoseLinkMappingRecordImpl(const FPoseLinkMappingRecord& InRecord) = 0;

	// Process the passed-in list of nodes
	virtual void ProcessAnimationNodesImpl(TArray<UAnimGraphNode_Base*>& AnimNodeList) = 0;

	// Prunes any nodes that aren't reachable via a pose link
	virtual void PruneIsolatedAnimationNodesImpl(const TArray<UAnimGraphNode_Base*>& RootSet, TArray<UAnimGraphNode_Base*>& GraphNodes) = 0;

	// Perform an expansion step for the specified graph
	virtual void ExpansionStepImpl(UEdGraph* Graph, bool bAllowUbergraphExpansions) = 0;

	// Get the message log for the current compilation
	virtual FCompilerResultsLog& GetMessageLogImpl() const = 0;

	// Performs standard validation on the graph (outputs point to inputs, no more than one connection to each input, types match on both ends, etc...)
	virtual bool ValidateGraphIsWellFormedImpl(UEdGraph* Graph) const = 0;

	// Returns the allocation index of the specified node, processing it if it was pending
	virtual int32 GetAllocationIndexOfNodeImpl(UAnimGraphNode_Base* VisualAnimNode) const = 0;

	// Get the currently-compiled blueprint
	virtual const UBlueprint* GetBlueprintImpl() const = 0;

	// Get the currently-compiled anim blueprint
	virtual const UAnimBlueprint* GetAnimBlueprintImpl() const = 0;

	// Get the consolidated uber graph during compilation
	virtual UEdGraph* GetConsolidatedEventGraphImpl() const = 0;

	// Gets all anim graph nodes that are piped into the provided node (traverses input pins)
	virtual void GetLinkedAnimNodesImpl(UAnimGraphNode_Base* InGraphNode, TArray<UAnimGraphNode_Base*>& LinkedAnimNodes) const = 0;

	// Index of the nodes (must match up with the runtime discovery process of nodes, which runs thru the property chain)
	virtual const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndicesImpl() const = 0;

	// Map of true source objects (user edited ones) to the cloned ones that are actually compiled
	virtual const TMap<UAnimGraphNode_Base*, UAnimGraphNode_Base*>& GetSourceNodeToProcessedNodeMapImpl() const = 0;

	// Map of anim node indices to node properties
	virtual const TMap<int32, FProperty*>& GetAllocatedPropertiesByIndexImpl() const = 0;

	// Map of anim node indices to node properties
	virtual const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedPropertiesByNodeImpl() const = 0;

	// Map of anim node indices to node handler properties in sparse class data struct
	virtual const TMap<UAnimGraphNode_Base*, FProperty*>& GetAllocatedHandlerPropertiesByNodeImpl() const = 0;

	// Add the specified compiled-in attribute uniquely to the specified node
	virtual void AddAttributesToNodeImpl(UAnimGraphNode_Base* InNode, TArrayView<const FName> InAttributes) const = 0;

	// Get the current compiled-in attributes uniquely assigned to the specified node
	virtual TArrayView<const FName> GetAttributesFromNodeImpl(UAnimGraphNode_Base* InNode) const = 0;

	// Check whether an anim node participates in constant folding
	virtual bool IsAnimGraphNodeFoldedImpl(UAnimGraphNode_Base* InNode) const = 0;

	// Get the folded property record, if any, for the supplied node & named property
	virtual const FFoldedPropertyRecord* GetFoldedPropertyRecordImpl(UAnimGraphNode_Base* InNode, FName InPropertyName) const = 0;

	// Get the generated property of the class that mutable data is added to
	virtual const FStructProperty* GetMutableDataPropertyImpl() const = 0;

	// Get the compiler as a base class to avoid circular include issues with templated functions/classes
	virtual FKismetCompilerContext* GetKismetCompiler() const = 0;
};

#undef UE_API
