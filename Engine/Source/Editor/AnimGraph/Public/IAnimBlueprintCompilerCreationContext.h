// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"

class UEdGraphSchema;

/** Interface to the creation of an anim BP compiler context */
class IAnimBlueprintCompilerCreationContext
{
public:
	virtual ~IAnimBlueprintCompilerCreationContext() = default;

	/** Registers a graphs schema class to the anim BP compiler so that default function processing is not performed on it */
	virtual void RegisterKnownGraphSchema(TSubclassOf<UEdGraphSchema> InGraphSchemaClass) = 0;
};

