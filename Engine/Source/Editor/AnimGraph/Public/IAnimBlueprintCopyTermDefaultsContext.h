// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"

class FCompilerResultsLog;
class UAnimBlueprint;

// Interface passed to CopyTermDefaults delegate
class IAnimBlueprintCopyTermDefaultsContext
{
public:	
	virtual ~IAnimBlueprintCopyTermDefaultsContext() = default;

	// Get the message log for the current compilation
	FCompilerResultsLog& GetMessageLog() const { return GetMessageLogImpl(); }

	// Get the currently-compiled anim blueprint
	const UAnimBlueprint* GetAnimBlueprint() const { return GetAnimBlueprintImpl(); }

protected:
	// Get the message log for the current compilation
	virtual FCompilerResultsLog& GetMessageLogImpl() const = 0;

	// Get the currently-compiled anim blueprint
	virtual const UAnimBlueprint* GetAnimBlueprintImpl() const = 0;
};


// Interface passed to per-node CopyTermDefaults override point
class IAnimBlueprintNodeCopyTermDefaultsContext
{
public:	
	virtual ~IAnimBlueprintNodeCopyTermDefaultsContext() = default;

	// Get the CDO that we are writing to
	UObject* GetClassDefaultObject() const { return GetClassDefaultObjectImpl(); }
	
	// Get the property that we are writing to
	const FProperty* GetTargetProperty() const { return GetTargetPropertyImpl(); }

	// Get the destination ptr (the node) that we are writing to
	uint8* GetDestinationPtr() const { return GetDestinationPtrImpl(); }

	// Get the source ptr (the node in the anim graph node) that we are reading from
	const uint8* GetSourcePtr() const { return GetSourcePtrImpl(); }

	// Get the property index for this node
	int32 GetNodePropertyIndex() const { return GetNodePropertyIndexImpl(); }

	// Get the source node cast to the correct type
	template <typename NodeType>
	const NodeType& GetSourceNode() const
	{
		const FStructProperty* TargetProperty = CastFieldChecked<FStructProperty>(GetTargetProperty());
		check(TargetProperty->Struct->IsChildOf(NodeType::StaticStruct()));
		return *reinterpret_cast<const NodeType*>(GetSourcePtr());
	}

	// Get the destination node cast to the correct type
	template <typename NodeType>
	NodeType& GetDestinationNode() const
	{
		const FStructProperty* TargetProperty = CastFieldChecked<FStructProperty>(GetTargetProperty());
		check(TargetProperty->Struct->IsChildOf(NodeType::StaticStruct()));
		return *reinterpret_cast<NodeType*>(GetDestinationPtr());
	}

protected:
	// Get the CDO that we are writing to
	virtual UObject* GetClassDefaultObjectImpl() const = 0;
	
	// Get the property that we are writing to
	virtual const FProperty* GetTargetPropertyImpl() const = 0;

	// Get the destination ptr (the node) that we are writing to
	virtual uint8* GetDestinationPtrImpl() const = 0;

	// Get the source ptr (the node in the anim graph node) that we are reading from
	virtual const uint8* GetSourcePtrImpl() const = 0;

	// Get the property index for this node
	virtual int32 GetNodePropertyIndexImpl() const  = 0;
};

// Interface passed to per-extension CopyTermDefaults override point
class IAnimBlueprintExtensionCopyTermDefaultsContext
{
public:	
	virtual ~IAnimBlueprintExtensionCopyTermDefaultsContext() = default;

	// Get the CDO that we are writing to
	UObject* GetClassDefaultObject() const { return GetClassDefaultObjectImpl(); }
	
	// Get the property that we are writing to
	const FProperty* GetTargetProperty() const { return GetTargetPropertyImpl(); }

	// Get the destination ptr (the node) that we are writing to
	uint8* GetDestinationPtr() const { return GetDestinationPtrImpl(); }

	// Get the source ptr (the node in the anim graph node) that we are reading from
	const uint8* GetSourcePtr() const { return GetSourcePtrImpl(); }

	// Get the property index for this node
	int32 GetNodePropertyIndex() const { return GetNodePropertyIndexImpl(); }

protected:
	// Get the CDO that we are writing to
	virtual UObject* GetClassDefaultObjectImpl() const = 0;
	
	// Get the property that we are writing to
	virtual const FProperty* GetTargetPropertyImpl() const = 0;

	// Get the destination ptr (the node) that we are writing to
	virtual uint8* GetDestinationPtrImpl() const = 0;

	// Get the source ptr (the node in the anim graph node) that we are reading from
	virtual const uint8* GetSourcePtrImpl() const = 0;

	// Get the property index for this node
	virtual int32 GetNodePropertyIndexImpl() const = 0;
};
