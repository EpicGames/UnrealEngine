// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimStateMachineTypes.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimNodeBase.h"
#include "PropertyAccess.h"

struct FBlueprintDebugData;
struct FAnimBlueprintDebugData;

/** Interface to the writable parts of the generated class that handlers can operate on */
class IAnimBlueprintGeneratedClassCompiledData
{
public:
	// Get the baked state machines data for the currently-compiled class
	virtual TArray<FBakedAnimationStateMachine>& GetBakedStateMachines() const = 0;

	// Get the saved pose indices map data for the currently-compiled class
	virtual TMap<FName, FCachedPoseIndices>& GetOrderedSavedPoseIndicesMap() const = 0;

	// Get the debug data for the currently-compiled class
	virtual FBlueprintDebugData& GetBlueprintDebugData() const = 0;

	// Get the currently-compiled classes anim notifies
	virtual TArray<FAnimNotifyEvent>& GetAnimNotifies() const = 0;

	// Finds a notify event or adds if it doesn't already exist
	virtual int32 FindOrAddNotify(FAnimNotifyEvent& Notify) const = 0;

	// Get the anim debug data for the currently-compiled class
	virtual FAnimBlueprintDebugData& GetAnimBlueprintDebugData() const = 0;

	// Get the currently-compiled classes graph asset player information
	virtual TMap<FName, FGraphAssetPlayerInformation>& GetGraphAssetPlayerInformation() const = 0;

	// Add a class-internal blendspace
	// @param	InSourceBlendSpace	The blendspace to duplicate
	// @return a duplicate of the blendspace outered to the class
	virtual UBlendSpace* AddBlendSpace(UBlendSpace* InSourceBlendSpace) = 0;
};
