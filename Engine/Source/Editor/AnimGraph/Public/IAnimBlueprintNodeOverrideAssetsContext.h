// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"

struct FAnimNode_Base;

// Interface passed to per-node OverrideAssets override point
class IAnimBlueprintNodeOverrideAssetsContext
{
public:
	virtual ~IAnimBlueprintNodeOverrideAssetsContext() = default;

	// Get the anim node to override
	template<typename AnimNodeType>
	AnimNodeType& GetAnimNode() const
	{
		check(GetAnimNodeTypeImpl()->IsChildOf(AnimNodeType::StaticStruct()));
		return static_cast<AnimNodeType&>(*GetAnimNodeImpl());
	}
	
	// Get all of the assets to override with
	TArrayView<UObject* const> GetAssets() const { return GetAssetsImpl(); }

protected:
	// Get the anim node to override
	virtual FAnimNode_Base* GetAnimNodeImpl() const = 0;

	// Get the type of the anim node to override
	virtual UScriptStruct* GetAnimNodeTypeImpl() const = 0;
	
	// Get all of the assets to override with
	virtual TArrayView<UObject* const> GetAssetsImpl() const = 0;
};