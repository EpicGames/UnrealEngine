// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FCompilerResultsLog;
class UEdGraph;
class UAnimGraphNode_Base;
struct FKismetCompilerOptions;

// Interface passed to PostExpansionStep delegate
class IAnimBlueprintPostExpansionStepContext
{
public:
	virtual ~IAnimBlueprintPostExpansionStepContext() = default;

	// Get the message log for the current compilation
	FCompilerResultsLog& GetMessageLog() const { return GetMessageLogImpl(); }

	// Index of the nodes (must match up with the runtime discovery process of nodes, which runs thru the property chain)
	const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndices() const { return GetAllocatedAnimNodeIndicesImpl(); }

	// Get the consolidated uber graph during compilation
	UEdGraph* GetConsolidatedEventGraph() const { return GetConsolidatedEventGraphImpl(); }

	// Get the compiler options we are currently using
	const FKismetCompilerOptions& GetCompileOptions() const { return GetCompileOptionsImpl(); }

protected:
	// Get the message log for the current compilation
	virtual FCompilerResultsLog& GetMessageLogImpl() const = 0;

	virtual const TMap<UAnimGraphNode_Base*, int32>& GetAllocatedAnimNodeIndicesImpl() const = 0;

	// Get the consolidated uber graph during compilation
	virtual UEdGraph* GetConsolidatedEventGraphImpl() const = 0;

	// Get the compiler options we are currently using
	virtual const FKismetCompilerOptions& GetCompileOptionsImpl() const = 0;
};
