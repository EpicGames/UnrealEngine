// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeatures.h"
#include "Templates/SubclassOf.h"

class UAnimBlueprintExtension;

class ANIMGRAPH_API IAnimGraph_SequencerMixerTargetConnector : public IModularFeature
{
public:

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("AnimGraph_SequencerMixerTargetConnector"));
		return FeatureName;
	}

	static const IAnimGraph_SequencerMixerTargetConnector* Get() 
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(IAnimGraph_SequencerMixerTargetConnector::GetModularFeatureName()))
		{
			return &IModularFeatures::Get().GetModularFeature<IAnimGraph_SequencerMixerTargetConnector>(IAnimGraph_SequencerMixerTargetConnector::GetModularFeatureName());
		}
		return nullptr;
	}

	virtual ~IAnimGraph_SequencerMixerTargetConnector() = default;
	
	virtual void GetSequencerMixerTargetRequiredExtensions(TArray<TSubclassOf<UAnimBlueprintExtension>>& OutExtensions) const = 0;
};