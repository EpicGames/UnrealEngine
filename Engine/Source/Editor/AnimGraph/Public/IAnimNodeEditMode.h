// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UnrealWidgetFwd.h"
#include "AnimationEditMode.h"

class UAnimGraphNode_Base;
struct FAnimNode_Base;

/** Base interface for skeletal control edit modes */
class IAnimNodeEditMode : public FAnimationEditMode
{
public:
	/** Returns the coordinate system that should be used for this bone */
	virtual ECoordSystem GetWidgetCoordinateSystem() const = 0;

	/** @return current widget mode this anim graph node supports */
	virtual UE::Widget::EWidgetMode GetWidgetMode() const = 0;

	/** Called when the user changed widget mode by pressing "Space" key */
	virtual UE::Widget::EWidgetMode ChangeToNextWidgetMode(UE::Widget::EWidgetMode CurWidgetMode) = 0;

	/** Called when the user set widget mode directly, returns true if InWidgetMode is available */
	virtual bool SetWidgetMode(UE::Widget::EWidgetMode InWidgetMode) = 0;

	/** Get the bone that the skeletal control is manipulating */
	virtual FName GetSelectedBone() const = 0;

	/** Called when the widget is dragged in translation mode */
	virtual void DoTranslation(FVector& InTranslation) = 0;

	/** Called when the widget is dragged in rotation mode */
	virtual void DoRotation(FRotator& InRotation) = 0;

	/** Called when the widget is dragged in scale mode */
	virtual void DoScale(FVector& InScale) = 0;

	/** Called when entering this edit mode */
	virtual void EnterMode(UAnimGraphNode_Base* InEditorNode, FAnimNode_Base* InRuntimeNode) = 0;

	/** Called when exiting this edit mode */
	virtual void ExitMode() = 0;

	/** Called to determine whether this edit mode should be drawn when nodes edited by this edit mode are pose watched */
	virtual bool SupportsPoseWatch() = 0;

	/** Called when a Pose Watch is created on a node edited by this edit mode */
	virtual void RegisterPoseWatchedNode(UAnimGraphNode_Base* InEditorNode, FAnimNode_Base* InRuntimeNode) = 0;
};
