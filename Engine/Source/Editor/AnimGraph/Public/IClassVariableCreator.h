// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "IClassVariableCreator.generated.h"

class FKismetCompilerContext;
class FProperty;
struct FEdGraphPinType;

/** Context passed to IClassVariableCreator::CreateClassVariablesFromBlueprint */
class IAnimBlueprintVariableCreationContext
{
public:
	/** Create a class variable in the current class. Note that no name confick resolution is performed, if a unique name is needed, use CreateUniqueVariable */
	virtual FProperty* CreateVariable(const FName Name, const FEdGraphPinType& Type) = 0;
	
	/** Create a uniquely named variable corresponding to an object in the current class. */
	virtual FProperty* CreateUniqueVariable(UObject* InForObject, const FEdGraphPinType& Type) = 0;
};

UINTERFACE(MinimalAPI)
class UClassVariableCreator : public UInterface
{
	GENERATED_BODY()
};

class IClassVariableCreator
{
	GENERATED_BODY()

public:
	/** 
	 * Implement this in a graph node and the anim BP compiler will call this expecting to generate
	 * class variables.
	 * @param	InVariableCreator	The variable creation context for the current BP compilation
	 */
	virtual void CreateClassVariablesFromBlueprint(IAnimBlueprintVariableCreationContext& InCreationContext) = 0;
};