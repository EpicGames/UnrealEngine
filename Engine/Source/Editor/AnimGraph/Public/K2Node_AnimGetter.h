// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AnimGetter.generated.h"

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class UAnimBlueprint;
class UAnimGraphNode_Base;
class UAnimStateNodeBase;
class UClass;
class UEdGraphSchema;
class UField;
class UFunction;
class UObject;

USTRUCT()
struct FNodeSpawnData
{
	GENERATED_BODY()

	FNodeSpawnData();

	// Title to use for the spawned node
	UPROPERTY()
	FText CachedTitle;

	// The node the spawned getter accesses, if any
	UPROPERTY()
	TObjectPtr<UAnimGraphNode_Base> SourceNode;

	// The state node the spawned getter accesses
	UPROPERTY()
	TObjectPtr<UAnimStateNodeBase> SourceStateNode;

	// The instance class the spawned getter is defined on
	UPROPERTY()
	TObjectPtr<UClass> AnimInstanceClass;

	// The blueprint the getter is valid within
	UPROPERTY()
	TObjectPtr<const UAnimBlueprint> SourceBlueprint;

	// The UFunction (as a UField) 
	UPROPERTY()
	TObjectPtr<UField> Getter;

	// String of combined valid contexts for the spawned getter
	UPROPERTY()
	FString GetterContextString;
};

UCLASS(MinimalAPI)
class UK2Node_AnimGetter : public UK2Node_CallFunction
{
	GENERATED_BODY()
public:

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostPasteNode() override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual bool IsActionFilteredOut(FBlueprintActionFilter const& Filter) override;
	virtual void PostPlacedNewNode();
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	// end of UK2Node interface

	// The node that is required for the getter
	UPROPERTY()
	TObjectPtr<UAnimGraphNode_Base> SourceNode;

	// UAnimStateNode doesn't use the same hierarchy so we need to have a seperate property here to handle
	// those.
	UPROPERTY()
	TObjectPtr<UAnimStateNodeBase> SourceStateNode;

	// The UAnimInstance derived class that implements the getter we are running
	UPROPERTY()
	TObjectPtr<UClass> GetterClass;

	// The anim blueprint that generated this getter
	UPROPERTY()
	TObjectPtr<const UAnimBlueprint> SourceAnimBlueprint;

	// Cached node title
	UPROPERTY()
	FText CachedTitle;

	// List of valid contexts for the node
	UPROPERTY()
	TArray<FString> Contexts;

protected:
	
	//UFunction* GetSourceBlueprintFunction() const;

	// Fixes the SourceStateNode to be the state of the node's owner
	void RestoreStateMachineState();

	// Fixes the SourceNode to be the state machine owner of SourceStateNode (if it is not null)
	void RestoreStateMachineNode();

	/** Returns whether or not the provided UFunction requires the named parameter */
	static bool GetterRequiresParameter(const UFunction* Getter, FString ParamName);

	/** Checks the cached context strings to make sure this getter is valid within the provided schema */
	bool IsContextValidForSchema(const UEdGraphSchema* Schema) const;

	/** Passed to blueprint spawners to configure spawned nodes */
	void PostSpawnNodeSetup(UEdGraphNode* NewNode, bool bIsTemplateNode, FNodeSpawnData SpawnData);

	/** Recache the title, used if the source node or source state changes */
	void UpdateCachedTitle();

	/** Sets CachedTitle for a new node to be created */
	static void UpdateCachedTitle(FNodeSpawnData& SpawnData);

	/** Generates a title for the node based on its function and the context it is in */
	static FText GenerateTitle(UFunction* Getter, UAnimStateNodeBase* SourceStateNode, UAnimGraphNode_Base* SourceNode);

	// UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	// End of UK2Node_CallFunction interface

	virtual void RegisterDelegates();
	virtual void UnregisterDelegates();

private:
	FDelegateHandle GraphRenameHandle;
};
