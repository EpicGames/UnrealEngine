// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "BlueprintActionFilter.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AnimNodeReference.generated.h"

#define UE_API ANIMGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UObject;
struct FSearchTagDataPair;

UCLASS(MinimalAPI)
class UK2Node_AnimNodeReference : public UK2Node
{
public:
	GENERATED_BODY()

	// Get the text used for the node's label
	UE_API FText GetLabelText() const;
	
	// Get the tag for this node, if any
	FName GetTag() const { return Tag; }

	// Set the tag for this node
	void SetTag(FName InTag) { Tag = InTag; }
private:
	/** The node tag we reference */
	UPROPERTY()
	FName Tag;

	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void AddSearchMetaDataInfo(TArray<FSearchTagDataPair>& OutTaggedMetaData) const override;
	UE_API virtual bool IsCompatibleWithGraph(UEdGraph const* TargetGraph) const override;
	
	// UK2Node interface
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool IsNodePure() const override { return true; }
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	UE_API virtual FText GetMenuCategory() const override;
	virtual bool DrawNodeAsVariable() const override { return true; }
	UE_API virtual FText GetTooltipText() const override;
};

#undef UE_API
