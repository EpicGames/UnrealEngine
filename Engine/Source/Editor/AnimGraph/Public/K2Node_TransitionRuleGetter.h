// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_TransitionRuleGetter.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FBlueprintActionFilter;
class UAnimBlueprint;
class UAnimGraphNode_Base;
class UAnimStateNode;
class UEdGraphPin;
class UEdGraphSchema;
class UObject;

UENUM()
namespace ETransitionGetter
{
	enum Type : int
	{
		AnimationAsset_GetCurrentTime,
		AnimationAsset_GetLength,
		AnimationAsset_GetCurrentTimeFraction,
		AnimationAsset_GetTimeFromEnd,
		AnimationAsset_GetTimeFromEndFraction,
		CurrentState_ElapsedTime,
		CurrentState_GetBlendWeight,
		CurrentTransitionDuration,
		ArbitraryState_GetBlendWeight
	};
}


UCLASS(MinimalAPI)
class UK2Node_TransitionRuleGetter : public UK2Node
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TEnumAsByte<ETransitionGetter::Type> GetterType;

	UPROPERTY()
	TObjectPtr<UAnimGraphNode_Base> AssociatedAnimAssetPlayerNode;

	UPROPERTY()
	TObjectPtr<UAnimStateNode> AssociatedStateNode;

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual void PreloadRequiredAssets() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool ShowPaletteIconOnNode() const override { return false; }
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual bool IsActionFilteredOut(FBlueprintActionFilter const& Filter) override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsNodePure() const override { return true; }
	// end of UK2Node interface

	// @todo document
	ANIMGRAPH_API UEdGraphPin* GetOutputPin() const;

	// @todo document
	ANIMGRAPH_API static FText GetFriendlyName(ETransitionGetter::Type TypeID);

private:
	/** Helper function for GetStateSpecificMenuActions, finds actions for AnimBlueprint states that can be used in AnimGraphSchema graphs */
	void GetStateSpecificAnimGraphSchemaMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const UAnimBlueprint* AnimBlueprint, UAnimStateNode* StateNode) const;

	/** Helper function for GetStateSpecificMenuActions, finds actions for AnimBlueprint states that can be used in AnimTransitionSchema graphs */
	void GetStateSpecificAnimTransitionSchemaMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const UAnimBlueprint* AnimBlueprint, UAnimStateNode* StateNode) const;

	/** Helper function for GetMenuActions, goes through all states in an AnimBlueprint and finds possible actions */
	void GetStateSpecificMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const UAnimBlueprint* AnimBlueprint) const;

	/** Helper function for GetMenuActions, generates non-state specific actions that use this node */
	void GetNonStateSpecificMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const;
};

